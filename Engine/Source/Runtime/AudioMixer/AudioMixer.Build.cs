// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AudioMixer : ModuleRules
	{
		public AudioMixer(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateIncludePathModuleNames.Add("TargetPlatform");
			PublicIncludePathModuleNames.Add("TargetPlatform");

			PublicIncludePathModuleNames.Add("Engine");

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"AudioLinkEngine",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Engine",
					"NonRealtimeAudioRenderer",
					"AudioMixerCore",
					"SignalProcessing",
					"AudioPlatformConfiguration",
					"SoundFieldRendering",
					"AudioExtensions",
					"AudioLinkCore",
					"HeadMountedDisplay",
					"TraceLog"
				}
			);

			AddEngineThirdPartyPrivateStaticDependencies(Target,
					"UEOgg",
					"Vorbis",
					"VorbisFile",
					"libOpus",
					"UELibSampleRate"
					);

			// Circular references that need to be cleaned up
			CircularlyReferencedDependentModules.AddRange(
				new string[] {
					"NonRealtimeAudioRenderer",
					"SoundFieldRendering"
				}
			);
		}
	}
}
