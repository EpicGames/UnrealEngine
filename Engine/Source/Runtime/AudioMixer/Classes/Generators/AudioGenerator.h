// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"

#include "AudioGenerator.generated.h"

struct FAudioGeneratorHandle
{
	int32 Id;

	FAudioGeneratorHandle()
		: Id(INDEX_NONE)
	{}
};

typedef TFunction<void(const float * InAudio, int32 NumSamples)> FOnAudioGenerate;

UCLASS(ClassGroup = (Custom), meta = (BlueprintSpawnableComponent), MinimalAPI)
class UAudioGenerator : public UObject
{
	GENERATED_BODY()

public:
	AUDIOMIXER_API UAudioGenerator();
	AUDIOMIXER_API virtual ~UAudioGenerator();

	// Returns the sample rate of the generator
	int32 GetSampleRate() const { return SampleRate; }

	// Returns the number of channels of the generator
	int32 GetNumChannels() const { return NumChannels; }

	// Adds a generator delegate. Returns a handle for the generator delegate, so it can be removed.
	AUDIOMIXER_API FAudioGeneratorHandle AddGeneratorDelegate(FOnAudioGenerate InFunction);

	// Removes the given audio generator delegate handle
	AUDIOMIXER_API void RemoveGeneratorDelegate(FAudioGeneratorHandle InHandle);

protected:

	// Called by derived classes to initialize the sample rate and num channels of the generator
	AUDIOMIXER_API void Init(int32 InSampleRate, int32 InNumChannels);

	// Called by derived classes when new audio is generated
	AUDIOMIXER_API void OnGeneratedAudio(const float* InAudio, int32 NumSamples);

	FCriticalSection CritSect;
	int32 SampleRate;
	int32 NumChannels;
	TMap<uint32, FOnAudioGenerate> OnGeneratedMap;
};
