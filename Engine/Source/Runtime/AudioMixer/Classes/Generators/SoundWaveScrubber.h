// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DSP/AlignedBuffer.h"
#include "DSP/GrainEnvelope.h"
#include "DSP/SampleRateConverter.h"
#include "Sound/SoundBase.h"
#include "Sound/SoundGenerator.h"
#include "Sound/SoundWave.h"
#include "AudioDynamicParameter.h"

#include "SoundWaveScrubber.generated.h"

#define UE_API AUDIOMIXER_API

class FSoundWaveProxyPlayer;

namespace Audio
{

// Class manages the runtime generation of scrubbed audio from a reference sound wave
// It does this using a simple granulation technique of overlapping grains using a Hanning window.
class FSoundWaveScrubber
{
public:
	UE_API FSoundWaveScrubber();
	UE_API ~FSoundWaveScrubber();

	// Initialize the sound wave scrubber
	UE_API void Init(TSharedRef<const FSoundWaveProxy> InSoundWaveProxyRef, float InSampleRate, int32 InNumChannels, float InPlayheadTimeSeconds = 0.0f);

	UE_DEPRECATED(5.8, "Please use the TSharedRef<const FSoundWaveProxy> overload instead")
	UE_API void Init(FSoundWaveProxyPtr InSoundWaveProxyPtr, float InSampleRate, int32 InNumChannels, float InPlayheadTimeSeconds = 0.0f);

	// Sets whether or not to scrub the audio file. If false, then the output of the scrubber will be silence.
	UE_API void SetIsScrubbing(bool bInIsScrubbing);

	// Sets if the scrubber should scrub while playhead is stationary (after it hits it's target playhead)
	UE_API void SetIsScrubbingWhileStationary(bool bInIsScrubWhileStationary);

	// Sets the scrubbing location in seconds
	UE_API void SetPlayheadTime(float InPlayheadTimeSeconds);

	// Sets the grain duration range in seconds (clamped in range of of 0.01 to 0.5 seconds)
	// The grain duration used during grain spawn is based the scrubbing speed. Sloweer the speed, the longer the grain.
	UE_API void SetGrainDurationRange(const FVector2D& InGrainDurationRange);

	// Renders the audio from the sound wave scrubber into the output audio buffer.
	UE_API int32 RenderAudio(TArrayView<float>& OutAudio);

private:

	// ----------------
	// General state and/or settings
	float AudioMixerSampleRate = 0.0f;
	float SourceFileSampleRate = 0.0f;
	float SourceFileDurationSeconds = 0.0f;
	int32 NumChannels = 0;

	// How much audio to decode per decode block
	static constexpr float DecodedAudioSizeInSeconds = 1.0f;

	// The current target playhead time in seconds
	// The scrubber will interpolate to this if the current playhead time is less than a given threshold,
	// otherwise the scrubber will "jump" to the playhead time.
	float TargetPlayheadTimeSeconds = 0.0f;

	// How long we've been stationary
	float TimeSincePlayheadHasNotChanged = 0.0f;

	// Interpolated playhead time
	FDynamicParameter CurrentPlayheadTimeSeconds;

	// ----------------
	// Data for managing the decoded audio
	struct FDecodedDataChunk
	{
		// The start frame of the decoded audio chunk
		int32 FrameStart = INDEX_NONE;

		// The actual decoded audio. 
		// Size is NumFrames * NumChannels * AudioMixerSampleRate (after SRC)
		Audio::FAlignedFloatBuffer PCMAudio;

		// Count of the number of grains actively using this chunk
		int32 NumGrainsUsingChunk = 0;
	};
	// Place to store decoded chunks. We need 2 chunks for actively playing grains plus a free chunk.
	TArray<FDecodedDataChunk> DecodedChunks;

	// Sound wave data to safely query and use the referenced sound wave asset
	TSharedPtr<const FSoundWaveData> SoundWaveDataPtr;
	// Sound wave proxy player is a decoder we can use to decode audio into chunks
	TUniquePtr<FSoundWaveProxyPlayer> SoundWaveProxyPlayerPtr;
	// Simple SRC interface. Decoded audio is SRC'd to match the audio mixer sample rate
	TUniquePtr<Audio::ISampleRateConverter> SRC;

	// Function utility to decode audio from the current sate to the given decoded data chunk
	void DecodeToDataChunk(FDecodedDataChunk& InOutDataChunk, float InDecoderSeekTimeSeconds);

	// This retrieves decoded data for the given read frame index. 
	// Returns INDEX_NONE if no decoded data was available for the given read index
	int32 GetDecodedDataChunkIndexForCurrentReadIndex(int32 InReadFrameIndex);

	// This function will decode the audio and cache it.
	// Returns the index for the decoded chunk
	int32 DecodeDataChunkIndexForCurrentReadIndex(int32 InReadFrameIndex);

	// ----------------
	// Data for granulation
	// Envelope shared across grains
	Audio::Grain::FEnvelope GrainEnvelope;

	// The grain duration in frames
	FVector2D TargetGrainDurationRange;
	FVector2D GrainDurationRange;
	float GrainDurationSeconds = 0.0f;

	// Grain duration as computed by scrub velocity based on the grain duration ranges
	int32 CurrentGrainDurationFrames = 0;
	int32 CurrentHalfGrainDurationFrames = 0;

	// Utility data useful for debugging the scrubber.
	// The grain count is a running tally of the numebr of grains rendered
	int32 GrainCount = 0;
	// This should never go above 2
	int32 NumActiveGrains = 0;

	// Active grain data for a single grain
	struct FGrain
	{
		// The number of frames this grain has rendered
		// If this is larger than GrainDurationFrames it's an inactive grain
		// this is used to look up the grain envelope to find the amplitude of the grain per frame
		int32 CurrentRenderedFramesCount = INDEX_NONE;

		// Index into the decoded data array
		int32 DecodedDataChunkIndex = 0;

		// The current read frame of the grain
		int32 CurrentReadFrame = INDEX_NONE;

		// The duration of this grain. Set when the grain spawns. 
		int32 GrainDurationFrames = 0;
	};
	// The grain data used to render the granular audio
	TArray<FGrain> ActiveGrains;

	// This is a frame count until we need to spawn another grain
	int32 NumFramesTillNextGrainSpawn = 0;

	// Spawns a new grain at the value of the current playhead
	FGrain SpawnGrain();

	// Makes sure the grain is referencing valid decoded data
	void UpdateGrainDecodeData(FGrain& InGrain);

	// Renders the grain, mixing it into the output buffer. Returns the frame index that is half
	void RenderActiveGrains(TArrayView<float>& OutAudio, int32 InStartFrame, int32 InNumFramesToRender);

	// Used for setting user parameters
	FCriticalSection CritSect;

	// Whether or not we're actively scrubbing audio playback. 
	bool bIsScrubbing = false;
	bool bIsScrubbingDueToBeingStationary = true;

	// Whether or not we scrub while the playhead doesn't move
	bool bIsScrubbingWhileStationary = true;

	// Whether or not we have an error with the decoder
	bool bHasErrorWithDecoder = false;
};


class FSoundWaveScrubberGenerator : public ISoundGenerator
{
public:
	UE_API void Init(TSharedRef<const FSoundWaveProxy> InSoundWaveProxyRef, float InSampleRate, int32 InNumChannels, float InPlayheadTimeSeconds = 0.0f);

	UE_DEPRECATED(5.8, "Please use the TSharedRef<const FSoundWaveProxy> overload instead")
	UE_API void Init(FSoundWaveProxyPtr InSoundWaveProxyPtr, float InSampleRate, int32 InNumChannels, float InPlayheadTimeSeconds = 0.0f);

	// ~ISoundGenerator Begin
	UE_API virtual int32 OnGenerateAudio(float* OutAudio, int32 NumSamples) override;
	UE_API virtual int32 GetDesiredNumSamplesToRenderPerCallback() const override;
	UE_API virtual bool IsFinished() const override;
	// ~ISoundGenerator End

	// Sets whether or not to scrub the audio file. If false, then the output of the scrubber will be silence.
	UE_API void SetIsScrubbing(bool bInIsScrubbing);

	// Sets if the scrubber should scrub while playhead is stationary (after it hits it's target playhead)
	UE_API void SetIsScrubbingWhileStationary(bool bInScrubWhileStationary);

	// Sets the scrubbing location in seconds
	UE_API void SetPlayheadTime(float InPlayheadTimeSeconds);

	// Sets the grain duration range in seconds (clamped in range of of 0.01 to 0.5 seconds)
	// The grain duration used during grain spawn is based the scrubbing speed. Sloweer the speed, the longer the grain.
	UE_API void SetGrainDurationRange(const FVector2D& InGrainDurationRange);

private:
	int32 NumChannels = 0;
	FSoundWaveScrubber SoundWaveScrubber;

};

}


UCLASS(MinimalAPI)
class UScrubbedSound : public USoundWave
{
	GENERATED_UCLASS_BODY()
public:

	// ~USoundBase
	virtual ISoundGeneratorPtr CreateSoundGenerator(const FSoundGeneratorInitParams& InParams) override;
	// ~USoundBase

	ISoundGeneratorPtr GetSoundGenerator() { return SoundWaveScrubber; }
	
	// Sets the sound wave to scrub
	UFUNCTION(BlueprintCallable, Category = "Scrubbing")
	AUDIOMIXER_API void SetSoundWave(USoundWave* InSoundWave);
	
	// Sets the scrub time in seconds
	UFUNCTION(BlueprintCallable, Category = "Scrubbing")
	AUDIOMIXER_API void SetPlayheadTime(float InPlayheadTimeSeconds);

	// Returns the current playhead time
	UFUNCTION(BlueprintCallable, Category = "Scrubbing")
	float GetPlayheadTime() const { return PlayheadTimeSeconds; }

	// Sets the scrub grain duration range.
	UFUNCTION(BlueprintCallable, Category = "Scrubbing")
	AUDIOMIXER_API void SetGrainDurationRange(const FVector2D& InGrainDurationRangeSeconds);

	// Sets if the scrubber is actively scrubbing or not
	UFUNCTION(BlueprintCallable, Category = "Scrubbing")
	AUDIOMIXER_API void SetIsScrubbing(bool bInIsScrubbing);

	// Sets if the scrubber should scrub while playhead is stationary (after it hits it's target playhead)
	UFUNCTION(BlueprintCallable, Category = "Scrubbing")
	AUDIOMIXER_API void SetIsScrubbingWhileStationary(bool bInScrubWhileStationary);

private:
	float PlayheadTimeSeconds = 0.0f;
	FVector2D GrainDurationRange = { 0.4f, 0.05f };

	bool bIsScrubbing = false;
	bool bScrubWhileStationary = true;
	float StationaryTimeSeconds = 0.1f;

	ISoundGeneratorPtr SoundWaveScrubber;

	UPROPERTY(Transient)
	TObjectPtr<USoundWave> SoundWaveToScrub;
	
};

#undef UE_API
