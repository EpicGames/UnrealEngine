// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioDevice.h"
#include "Delegates/IDelegateInstance.h"
#include "DSP/DynamicsProcessor.h"
#include "DSP/MultithreadedPatching.h"
#include "Misc/ScopeLock.h"
#include "Sound/SoundEffectSubmix.h"
#include "Sound/SoundSubmix.h"
#include "Sound/SoundSubmixSend.h"
#include "Stats/Stats.h"

#include "AudioMixerSubmixEffectDynamicsProcessor.generated.h"

// The time it takes to process the master dynamics.
DECLARE_CYCLE_STAT_EXTERN(TEXT("Submix Dynamics"), STAT_AudioMixerSubmixDynamics, STATGROUP_AudioMixer, AUDIOMIXER_API);

namespace Audio
{
	// Forward Declarations
	class FMixerDevice;
}

UENUM(BlueprintType)
enum class ESubmixEffectDynamicsProcessorType : uint8
{
	Compressor = 0,
	Limiter,
	Expander,
	Gate,
	UpwardsCompressor,
	Count UMETA(Hidden)
};

UENUM(BlueprintType)
enum class ESubmixEffectDynamicsPeakMode : uint8
{
	MeanSquared = 0,
	RootMeanSquared,
	Peak,
	Count UMETA(Hidden)
};

UENUM(BlueprintType)
enum class ESubmixEffectDynamicsChannelLinkMode : uint8
{
	Disabled = 0,
	Average,
	Peak,
	Count UMETA(Hidden)
};

UENUM(BlueprintType)
enum class ESubmixEffectDynamicsKeySource : uint8
{
	// Defaults to use local submix (input) as key
	Default = 0,

	// Uses audio bus as key
	AudioBus,

	// Uses external submix as key
	Submix,

	Count UMETA(Hidden)
};

class FKeySource
{
	ESubmixEffectDynamicsKeySource Type = ESubmixEffectDynamicsKeySource::Default;
	int32 NumChannels = 0;
	uint32 ObjectId = INDEX_NONE;

	mutable FCriticalSection MutateSourceCritSection;

public:
	Audio::FPatchOutputStrongPtr Patch;

	void Reset()
	{
		Patch.Reset();

		{
			const FScopeLock ScopeLock(&MutateSourceCritSection);
			NumChannels = 0;
			ObjectId = INDEX_NONE;
			Type = ESubmixEffectDynamicsKeySource::Default;
		}
	}

	uint32 GetObjectId() const
	{
		const FScopeLock ScopeLock(&MutateSourceCritSection);
		return ObjectId;
	}

	int32 GetNumChannels() const
	{
		const FScopeLock ScopeLock(&MutateSourceCritSection);
		return NumChannels;
	}

	ESubmixEffectDynamicsKeySource GetType() const
	{
		const FScopeLock ScopeLock(&MutateSourceCritSection);
		return Type;
	}

	void SetNumChannels(const int32 InNumChannels)
	{
		const FScopeLock ScopeLock(&MutateSourceCritSection);
		NumChannels = InNumChannels;
	}

	void Update(ESubmixEffectDynamicsKeySource InType, uint32 InObjectId, int32 InNumChannels = 0)
	{
		bool bResetPatch = false;

		{
			const FScopeLock ScopeLock(&MutateSourceCritSection);
			if (Type != InType || ObjectId != InObjectId || NumChannels != InNumChannels)
			{
				Type = InType;
				ObjectId = InObjectId;
				NumChannels = InNumChannels;

				bResetPatch = true;
			}
		}

		if (bResetPatch)
		{
			Patch.Reset();
		}
	}
};

USTRUCT(BlueprintType)
struct FSubmixEffectDynamicProcessorFilterSettings
{
	GENERATED_USTRUCT_BODY()

	// Whether or not filter is enabled
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Filter, meta = (DisplayName = "Enabled"))
	uint8 bEnabled : 1;

	// The cutoff frequency of the HPF applied to key signal
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Filter, meta = (DisplayName = "Cutoff (Hz)", EditCondition = "bEnabled", ClampMin = "20.0", ClampMax = "20000.0", UIMin = "20.0", UIMax = "20000.0"))
	float Cutoff;

	// The gain of the filter shelf applied to the key signal
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Filter, meta = (DisplayName = "Gain (dB)", EditCondition = "bEnabled", ClampMin = "-60.0", ClampMax = "6.0", UIMin = "-60.0", UIMax = "6.0"))
	float GainDb;

	FSubmixEffectDynamicProcessorFilterSettings()
		: bEnabled(false)
		, Cutoff(20.0f)
		, GainDb(0.0f)
	{
	}
};

// Submix dynamics processor settings
USTRUCT(BlueprintType)
struct FSubmixEffectDynamicsProcessorSettings
{
	GENERATED_USTRUCT_BODY()

	// Type of processor to apply
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = General, meta = (DisplayName = "Type"))
	ESubmixEffectDynamicsProcessorType DynamicsProcessorType = ESubmixEffectDynamicsProcessorType::Compressor;

	// Mode of peak detection used on input key signal
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Dynamics, meta = (EditCondition = "!bBypass"))
	ESubmixEffectDynamicsPeakMode PeakMode = ESubmixEffectDynamicsPeakMode::Peak;

	// Mode of peak detection if key signal is multi-channel
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Dynamics, meta = (EditCondition = "!bBypass"))
	ESubmixEffectDynamicsChannelLinkMode LinkMode = ESubmixEffectDynamicsChannelLinkMode::Average;

	// The input gain of the dynamics processor
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = General, meta = (DisplayName = "Input Gain (dB)", UIMin = "-12.0", UIMax = "20.0", EditCondition = "!bBypass"))
	float InputGainDb = 0.0f;

	// The threshold at which to perform a dynamics processing operation
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Dynamics, meta = (DisplayName = "Threshold (dB)", ClampMin = "-60.0", ClampMax = "0.0", UIMin = "-60.0", UIMax = "0.0", EditCondition = "!bBypass"))
	float ThresholdDb = -6.0f;

	// The dynamics processor ratio used for compression/expansion
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Dynamics, meta = (
		EditCondition = "!bBypass && DynamicsProcessorType == ESubmixEffectDynamicsProcessorType::Compressor || DynamicsProcessorType == ESubmixEffectDynamicsProcessorType::Expander ||  DynamicsProcessorType == ESubmixEffectDynamicsProcessorType::UpwardsCompressor",
		ClampMin = "1.0", ClampMax = "20.0", UIMin = "1.0", UIMax = "20.0"))
	float Ratio = 1.5f;

	// The knee bandwidth of the processor to use
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Dynamics, meta = (DisplayName = "Knee (dB)", ClampMin = "0.0", ClampMax = "20.0", UIMin = "0.0", UIMax = "20.0", EditCondition = "!bBypass"))
	float KneeBandwidthDb = 10.0f;

	// The amount of time to look ahead of the current audio (Allows for transients to be included in dynamics processing)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Response,  meta = (DisplayName = "Look Ahead (ms)", ClampMin = "0.0", ClampMax = "50.0", UIMin = "0.0", UIMax = "50.0", EditCondition = "!bBypass"))
	float LookAheadMsec = 3.0f;

	// The amount of time to ramp into any dynamics processing effect
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Response, meta = (DisplayName = "AttackTime (ms)", ClampMin = "1.0", ClampMax = "300.0", UIMin = "1.0", UIMax = "200.0", EditCondition = "!bBypass"))
	float AttackTimeMsec = 10.0f;

	// The amount of time to release the dynamics processing effect
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Response, meta = (DisplayName = "Release Time (ms)", ClampMin = "20.0", ClampMax = "5000.0", UIMin = "20.0", UIMax = "5000.0", EditCondition = "!bBypass"))
	float ReleaseTimeMsec = 100.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Sidechain, meta = (EditCondition = "!bBypass"))
	ESubmixEffectDynamicsKeySource KeySource = ESubmixEffectDynamicsKeySource::Default;

	// If set, uses output of provided audio bus as modulator of input signal for dynamics processor (Uses input signal as default modulator)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Sidechain, meta = (EditCondition = "!bBypass && KeySource == ESubmixEffectDynamicsKeySource::AudioBus", EditConditionHides))
	TObjectPtr<UAudioBus> ExternalAudioBus = nullptr;

	// If set, uses output of provided submix as modulator of input signal for dynamics processor (Uses input signal as default modulator)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Sidechain, meta = (EditCondition = "!bBypass && KeySource == ESubmixEffectDynamicsKeySource::Submix", EditConditionHides))
	TObjectPtr<USoundSubmix> ExternalSubmix = nullptr;

	UPROPERTY()
	uint8 bChannelLinked_DEPRECATED : 1;

	// Toggles treating the attack and release envelopes as analog-style vs digital-style (Analog will respond a bit more naturally/slower)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Response, meta = (EditCondition = "!bBypass"))
	uint8 bAnalogMode : 1;

	// Whether or not to bypass effect
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = General, meta = (DisplayName = "Bypass", DisplayAfter = "DynamicsProcessorType"))
	uint8 bBypass : 1;

	// Audition the key modulation signal, bypassing enveloping and processing the input signal.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Sidechain, meta = (DisplayName = "Key Audition", EditCondition = "!bBypass"))
	uint8 bKeyAudition : 1;

	// Gain to apply to key signal if key source not set to default (input).
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Sidechain, meta = (
		DisplayName = "External Input Gain (dB)",
		EditCondition = "!bBypass && KeySource != ESubmixEffectDynamicsKeySource::Default",
		UIMin = "-60.0", UIMax = "30.0")
	)
	float KeyGainDb = 0.0f;

	// The output gain of the dynamics processor
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Output, meta = (DisplayName = "Output Gain (dB)", UIMin = "-60.0", UIMax = "30.0", EditCondition = "!bBypass"))
	float OutputGainDb = 0.0f;

	// High Shelf filter settings for key signal (external signal if supplied or input signal if not)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Sidechain, meta = (DisplayName = "Key Highshelf", EditCondition = "!bBypass"))
	FSubmixEffectDynamicProcessorFilterSettings KeyHighshelf;

	// Low Shelf filter settings for key signal (external signal if supplied or input signal if not)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Sidechain, meta = (DisplayName = "Key Lowshelf", EditCondition = "!bBypass"))
	FSubmixEffectDynamicProcessorFilterSettings KeyLowshelf;

	FSubmixEffectDynamicsProcessorSettings()
		: bChannelLinked_DEPRECATED(true)
		, bAnalogMode(true)
		, bBypass(false)
		, bKeyAudition(false)
	{
		KeyLowshelf.Cutoff = 20000.0f;
	}
};


class FSubmixEffectDynamicsProcessor : public FSoundEffectSubmix
{
public:
	AUDIOMIXER_API FSubmixEffectDynamicsProcessor();

	AUDIOMIXER_API virtual ~FSubmixEffectDynamicsProcessor();

	// Gets the effect's deviceId that owns it
	AUDIOMIXER_API Audio::FDeviceId GetDeviceId() const;

	// Called on an audio effect at initialization on audio thread before audio processing begins.
	AUDIOMIXER_API virtual void Init(const FSoundEffectSubmixInitData& InInitData) override;

	// Process the input block of audio. Called on audio render thread.
	AUDIOMIXER_API virtual void OnProcessAudio(const FSoundEffectSubmixInputData& InData, FSoundEffectSubmixOutputData& OutData) override;

	// Called when an audio effect preset is changed
	AUDIOMIXER_API virtual void OnPresetChanged() override;


protected:
	AUDIOMIXER_API Audio::FMixerDevice* GetMixerDevice();

	AUDIOMIXER_API void ResetKey();
	AUDIOMIXER_API void UpdateKeyFromSettings(const FSubmixEffectDynamicsProcessorSettings& InSettings);
	AUDIOMIXER_API bool UpdateKeySourcePatch();

	AUDIOMIXER_API void OnDeviceCreated(Audio::FDeviceId InDeviceId);
	AUDIOMIXER_API void OnDeviceDestroyed(Audio::FDeviceId InDeviceId);
	
	Audio::FAlignedFloatBuffer AudioExternal;

	Audio::FDeviceId DeviceId = INDEX_NONE;

	bool bBypass = false;

private:
	FKeySource KeySource;
	Audio::FDynamicsProcessor DynamicsProcessor;

	FDelegateHandle DeviceCreatedHandle;
	FDelegateHandle DeviceDestroyedHandle;

	friend class USubmixEffectDynamicsProcessorPreset;
};

UCLASS(ClassGroup = AudioSourceEffect, meta = (BlueprintSpawnableComponent), MinimalAPI)
class USubmixEffectDynamicsProcessorPreset : public USoundEffectSubmixPreset
{
	GENERATED_BODY()

public:
	EFFECT_PRESET_METHODS(SubmixEffectDynamicsProcessor)

	AUDIOMIXER_API virtual void OnInit() override;

	AUDIOMIXER_API virtual void Serialize(FStructuredArchive::FRecord Record) override;

#if WITH_EDITOR
	AUDIOMIXER_API virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& InChainEvent) override;
#endif // WITH_EDITOR

	UFUNCTION(BlueprintCallable, Category = "Audio|Effects")
	AUDIOMIXER_API void ResetKey();

	// Sets the source key input as the provided AudioBus' output.  If no object is provided, key is set
	// to effect's input.
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects")
	AUDIOMIXER_API void SetAudioBus(UAudioBus* AudioBus);

	// Sets the source key input as the provided Submix's output.  If no object is provided, key is set
	// to effect's input.
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects")
	AUDIOMIXER_API void SetExternalSubmix(USoundSubmix* Submix);

	UFUNCTION(BlueprintCallable, Category = "Audio|Effects")
	AUDIOMIXER_API void SetSettings(const FSubmixEffectDynamicsProcessorSettings& Settings);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SubmixEffectPreset, meta = (ShowOnlyInnerProperties))
	FSubmixEffectDynamicsProcessorSettings Settings;

private:
	void SetKey(ESubmixEffectDynamicsKeySource InKeySource, UObject* InObject, int32 InNumChannels = 0);
};
