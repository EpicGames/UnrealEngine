// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DSP/EQ.h"
#include "Sound/SoundEffectSubmix.h"
#include "Sound/SoundMix.h"
#include "Stats/Stats.h"

#include "AudioMixerSubmixEffectEQ.generated.h"

// The time it takes to process the master EQ effect.
DECLARE_CYCLE_STAT_EXTERN(TEXT("Submix EQ"), STAT_AudioMixerSubmixEQ, STATGROUP_AudioMixer, AUDIOMIXER_API);

// A multiband EQ submix effect.
USTRUCT(BlueprintType)
struct FSubmixEffectEQBand
{
	GENERATED_USTRUCT_BODY()

	// Cutoff frequency (in hz) of the EQ band
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Equalizer, meta = (EditCondition = "bEnabled", DisplayName = "Cutoff Frequency (hz)", ClampMin = "20.0", ClampMax = "20000.0", UIMin = "20.0", UIMax = "15000.0"))
	float Frequency = 500.0f;

	// The bandwidth, in octaves, of the EQ band
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Equalizer, meta = (EditCondition = "bEnabled", DisplayName = "Bandwidth (octaves)", ClampMin = "0.1", ClampMax = "2.0", UIMin = "0.1", UIMax = "2.0"))
	float Bandwidth = 2.0f;

	// The gain in decibels of the EQ band
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Equalizer, meta = (EditCondition = "bEnabled", DisplayName = "Gain (dB)", ClampMin = "-90.0", ClampMax = "20.0", UIMin = "-90.0", UIMax = "20.0"))
	float GainDb = 0.0f;

	// Whether or not the EQ band is enabled
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Equalizer)
	uint32 bEnabled : 1;

	FSubmixEffectEQBand()
		: bEnabled(false)
	{
	}
};

// EQ submix effect
USTRUCT(BlueprintType)
struct FSubmixEffectSubmixEQSettings
{
	GENERATED_USTRUCT_BODY()

	// The EQ bands to use. 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "SubmixEffect|Preset")
	TArray<FSubmixEffectEQBand> EQBands;
};

class FSubmixEffectSubmixEQ : public FSoundEffectSubmix
{
public:
	AUDIOMIXER_API FSubmixEffectSubmixEQ();

	// Called on an audio effect at initialization on main thread before audio processing begins.
	AUDIOMIXER_API virtual void Init(const FSoundEffectSubmixInitData& InSampleRate) override;

	// Process the input block of audio. Called on audio thread.
	AUDIOMIXER_API virtual void OnProcessAudio(const FSoundEffectSubmixInputData& InData, FSoundEffectSubmixOutputData& OutData) override;

	// Sets the effect parameters using the old audio engine preset setting object
	AUDIOMIXER_API virtual bool SetParameters(const FAudioEffectParameters& InParameters) override;

	virtual bool SupportsDefaultEQ() const override
	{
		return true;
	}

	// Called when an audio effect preset is changed
	AUDIOMIXER_API virtual void OnPresetChanged() override;

protected:
	AUDIOMIXER_API void UpdateParameters(const int32 NumOutputChannels);

	// An EQ effect is a bank of biquad filters
	struct FEQ
	{
		bool bEnabled;
		TArray<Audio::FBiquadFilter> Bands;

		FEQ()
			: bEnabled(true)
		{}
	};

	// Each of these filters is a 2 channel biquad filter. 1 for each stereo pair
	TArray<FEQ> FiltersPerChannel;

	float ScratchInBuffer[2];
	float ScratchOutBuffer[2];
	float SampleRate;
	float NumOutputChannels;
	bool bEQSettingsSet;

	// A pending eq setting change
	Audio::TParams<FSubmixEffectSubmixEQSettings> PendingSettings;

	// Game thread copy of the eq setting
	FSubmixEffectSubmixEQSettings GameThreadEQSettings;

	// Audio render thread copy of the eq setting
	FSubmixEffectSubmixEQSettings RenderThreadEQSettings;
};

UCLASS(ClassGroup = AudioSourceEffect, meta = (BlueprintSpawnableComponent), MinimalAPI)
class USubmixEffectSubmixEQPreset : public USoundEffectSubmixPreset
{
	GENERATED_BODY()

public:

	EFFECT_PRESET_METHODS(SubmixEffectSubmixEQ)

	UFUNCTION(BlueprintCallable, Category = "Audio|Effects")
	AUDIOMIXER_API void SetSettings(const FSubmixEffectSubmixEQSettings& InSettings);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SubmixEffectPreset, meta = (ShowOnlyInnerProperties))
	FSubmixEffectSubmixEQSettings Settings;
};
