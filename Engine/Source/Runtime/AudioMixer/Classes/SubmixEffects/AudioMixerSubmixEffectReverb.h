// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioEffect.h"
#include "Curves/RichCurve.h"
#include "AudioEffect.h"
#include "Curves/RichCurve.h"
#include "DSP/Amp.h"
#include "DSP/ReverbFast.h"
#include "DSP/BufferVectorOperations.h"
#include "Sound/SoundEffectSubmix.h"
#include "Stats/Stats.h"

#include "AudioMixerSubmixEffectReverb.generated.h"

struct FAudioEffectParameters;

// The time it takes to process the master reverb.
DECLARE_CYCLE_STAT_EXTERN(TEXT("Submix Reverb"), STAT_AudioMixerSubmixReverb, STATGROUP_AudioMixer, AUDIOMIXER_API);

USTRUCT(BlueprintType)
struct FSubmixEffectReverbSettings
{
	GENERATED_USTRUCT_BODY()

	/** Bypasses early reflections */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = EarlyReflections)
	bool bBypassEarlyReflections = false;

	/** Reflections Delay - 0.0 < 0.007 < 0.3 Seconds - the time between the listener receiving the direct path sound and the first reflection */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = EarlyReflections, meta = (DisplayName = "Reflections Delay (seconds)", ClampMin = "0.0", ClampMax = "0.3", EditCondition = "!bBypass && !bBypassEarlyReflections"))
	float ReflectionsDelay = 0.007f;

	/** Reverb Gain High Frequency - 0.0 < 0.89 < 1.0 - attenuates the high frequency reflected sound */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = EarlyReflections, meta = (DisplayName = "High Frequency Gain", ClampMin = "0.0", ClampMax = "1.0", EditCondition = "!bBypass && !bBypassEarlyReflections"))
	float GainHF = 0.89f;

	/** Reflections Gain - 0.0 < 0.05 < 3.16 - controls the amount of initial reflections */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = EarlyReflections, meta = (ClampMin = "0.0", ClampMax = "3.16", EditCondition = "!bBypass && !bBypassEarlyReflections"))
	float ReflectionsGain = 0.05f;

	/** Bypasses late reflections. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = LateReflections)
	bool bBypassLateReflections = false;

	/** Late Reverb Delay - 0.0 < 0.011 < 0.1 Seconds - time difference between late reverb and first reflections */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = LateReflections, meta = (DisplayName = "Late Delay (seconds)", ClampMin = "0.0", ClampMax = "0.1", EditCondition = "!bBypass && !bBypassLateReflections"))
	float LateDelay = 0.1f;

	/** Decay Time - 0.1 < 1.49 < 20.0 Seconds - larger is more reverb */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = LateReflections, meta = (DisplayName = "Decay Time (seconds)", ClampMin = "0.1", ClampMax = "20.0", EditCondition = "!bBypass && !bBypassLateReflections"))
	float DecayTime = 1.49f;

	/** Density - 0.0 < 0.85 < 1.0 - Coloration of the late reverb - lower value is more grainy */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = LateReflections, meta = (ClampMin = "0.0", ClampMax = "1.0", EditCondition = "!bBypass && !bBypassLateReflections"))
	float Density = 0.85f;

	/** Diffusion - 0.0 < 0.85 < 1.0 - Echo density in the reverberation decay - lower is more grainy */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = LateReflections, meta = (ClampMin = "0.0", ClampMax = "1.0", EditCondition = "!bBypass && !bBypassLateReflections"))
	float Diffusion = 0.85f;

	/** Air Absorption - 0.0 < 0.994 < 1.0 - lower value means more absorption */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = LateReflections, meta = (DisplayName = "High Frequency Air Absorption Gain", ClampMin = "0.0", ClampMax = "1.0", EditCondition = "!bBypass && !bBypassLateReflections"))
	float AirAbsorptionGainHF = 0.994f;

	/** Decay High Frequency Ratio - 0.1 < 0.83 < 2.0 - how much quicker or slower the high frequencies decay relative to the lower frequencies. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = LateReflections, meta = (DisplayName = "High Frequency Decay Ratio", ClampMin = "0.1", ClampMax = "2.0", EditCondition = "!bBypass && !bBypassLateReflections"))
	float DecayHFRatio = 0.83f;

	/** Late Reverb Gain - 0.0 < 1.26 < 10.0 - gain of the late reverb */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = LateReflections, meta = (DisplayName = "Late Reflections Gain", ClampMin = "0.0", ClampMax = "10.0", EditCondition = "!bBypass && !bBypassLateReflections"))
	float LateGain = 1.26f;

	/** Reverb Gain - 0.0 < 0.32 < 1.0 - overall reverb gain - master volume control */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = LateReflections, meta = (ClampMin = "0.0", ClampMax = "1.0", EditCondition = "!bBypass && !bBypassLateReflections"))
	float Gain = 0.0f;

	// Overall wet level of the reverb effect
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Routing, meta = (EditCondition = "!bBypass", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "10.0"))
	float WetLevel = 0.3f;

	// Overall dry level of the reverb effect
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Routing, meta = (EditCondition = "!bBypass", ClampMin = "0.0", ClampMax = "1.0"))
	float DryLevel = 0.0f;

	/** Bypasses reverb */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = General)
	bool bBypass = false;
};

class FSubmixEffectReverb : public FSoundEffectSubmix
{
public:
	AUDIOMIXER_API FSubmixEffectReverb();

	// Called on an audio effect at initialization on main thread before audio processing begins.
	AUDIOMIXER_API virtual void Init(const FSoundEffectSubmixInitData& InSampleRate) override;
	
	// Called when an audio effect preset is changed
	AUDIOMIXER_API virtual void OnPresetChanged() override;

	// Forces receiving downmixed submix audio to stereo input for the reverb effect
	virtual uint32 GetDesiredInputChannelCountOverride() const override { return 2; }

	// Process the input block of audio. Called on audio thread.
	AUDIOMIXER_API virtual void OnProcessAudio(const FSoundEffectSubmixInputData& InData, FSoundEffectSubmixOutputData& OutData) override;

	// Sets the reverb effect parameters based from audio thread code
	AUDIOMIXER_API virtual bool SetParameters(const FAudioEffectParameters& InParameters) override;

	// Whether this effect supports the default reverb system
	virtual bool SupportsDefaultReverb() const override
	{
		return true;
	}

	// Returns the drylevel of the effect
	virtual float GetDryLevel() const override { return CurrentWetDry.DryLevel; }

private:

	static AUDIOMIXER_API const float MinWetness;
	static AUDIOMIXER_API const float MaxWetness;

	AUDIOMIXER_API void UpdateParameters();

	// The reverb effect
	TUniquePtr<Audio::FPlateReverbFast> PlateReverb;

	// The reverb effect params
	Audio::TParams<Audio::FPlateReverbFastSettings> ReverbParams;

	// Settings for wet and dry signal to be consumed on next buffer
	Audio::TParams<Audio::FWetDry> WetDryParams;

	// Level of wet/dry signal on current buffer
	Audio::FWetDry CurrentWetDry;

	Audio::FAlignedFloatBuffer WetInputBuffer;

	// Curve which maps old reverb times to new decay value
	FRichCurve DecayCurve;

	float SampleRate = 1.f;
};

UCLASS(MinimalAPI)
class USubmixEffectReverbPreset : public USoundEffectSubmixPreset
{
	GENERATED_BODY()

public:
	EFFECT_PRESET_METHODS(SubmixEffectReverb)

	AUDIOMIXER_API virtual void PostInitProperties() override;

	UFUNCTION(BlueprintCallable, Category = "Audio|Effects")
	AUDIOMIXER_API void SetSettings(const FSubmixEffectReverbSettings& InSettings);

	UFUNCTION(BlueprintCallable, Category = "Audio|Effects")
	AUDIOMIXER_API void SetSettingsWithReverbEffect(const UReverbEffect* InReverbEffect, const float WetLevel, const float DryLevel = 0.0f);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SubmixEffectPreset, meta = (ShowOnlyInnerProperties))
	FSubmixEffectReverbSettings Settings;
};
