// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioBusSubsystem.h"
#include "AudioMixerDevice.h"
#include "AudioMixerSourceManager.h"
#include "DSP/MultithreadedPatching.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioBusSubsystem)

std::atomic<uint32> Audio::FAudioBusKey::InstanceIdCounter = 0;

UAudioBusSubsystem::UAudioBusSubsystem()
{
}

bool UAudioBusSubsystem::ShouldCreateSubsystem(UObject* Outer) const
{
	return !IsRunningDedicatedServer();
}

void UAudioBusSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	UE_LOGF(LogAudioMixer, Log, "Initializing Audio Bus Subsystem for audio device with ID %d", GetMixerDevice()->DeviceID);
	InitDefaultAudioBuses();
}

void UAudioBusSubsystem::Deinitialize()
{
	UE_LOGF(LogAudioMixer, Log, "Deinitializing Audio Bus Subsystem for audio device with ID %d", GetMixerDevice() ? GetMixerDevice()->DeviceID : -1);
	ShutdownDefaultAudioBuses();
}

void UAudioBusSubsystem::StartAudioBus(Audio::FAudioBusKey InAudioBusKey, int32 InNumChannels, bool bInIsAutomatic)
{
	StartAudioBus(InAudioBusKey, FString(), InNumChannels, bInIsAutomatic);
}

void UAudioBusSubsystem::StartAudioBus(Audio::FAudioBusKey InAudioBusKey, const FString& InAudioBusName, int32 InNumChannels, bool bInIsAutomatic)
{
	if (IsInGameThread())
	{
		if (ActiveAudioBuses_GameThread.Contains(InAudioBusKey))
		{
			return;
		}

		FActiveBusData BusData;
		BusData.BusKey = InAudioBusKey;
		BusData.NumChannels = InNumChannels;
		BusData.bIsAutomatic = bInIsAutomatic;

		ActiveAudioBuses_GameThread.Add(InAudioBusKey, BusData);

		FAudioThread::RunCommandOnAudioThread([this, InAudioBusKey, InAudioBusName, InNumChannels, bInIsAutomatic]()
		{
			if (Audio::FMixerSourceManager* MixerSourceManager = GetMutableSourceManager())
			{
				MixerSourceManager->StartAudioBus(InAudioBusKey, InAudioBusName, InNumChannels, bInIsAutomatic);
			}
		});
	}
	else
	{
		// If we're not the game thread, this needs to be on the game thread, so queue up a command to execute it on the game thread
		if (Audio::FMixerDevice* MixerDevice = GetMutableMixerDevice())
		{
			MixerDevice->GameThreadMPSCCommand([this, InAudioBusKey, InAudioBusName, InNumChannels, bInIsAutomatic]
			{
				StartAudioBus(InAudioBusKey, InAudioBusName, InNumChannels, bInIsAutomatic);
			});
		}
	}
}

void UAudioBusSubsystem::StopAudioBus(Audio::FAudioBusKey InAudioBusKey)
{
	if (IsInGameThread())
	{
		if (!ActiveAudioBuses_GameThread.Contains(InAudioBusKey))
		{
			return;
		}

		ActiveAudioBuses_GameThread.Remove(InAudioBusKey);

		FAudioThread::RunCommandOnAudioThread([this, InAudioBusKey]()
		{
			if (Audio::FMixerSourceManager* MixerSourceManager = GetMutableSourceManager())
			{
				MixerSourceManager->StopAudioBus(InAudioBusKey);
			}
		});
	}
	else
	{
		// If we're not the game thread, this needs to be on the game thread, so queue up a command to execute it on the game thread
		if (Audio::FMixerDevice* MixerDevice = GetMutableMixerDevice())
		{
			MixerDevice->GameThreadMPSCCommand([this, InAudioBusKey]
			{
				StopAudioBus(InAudioBusKey);
			});
		}
	}
}

bool UAudioBusSubsystem::IsAudioBusActive(Audio::FAudioBusKey InAudioBusKey) const
{
	if (IsInGameThread())
	{
		return ActiveAudioBuses_GameThread.Contains(InAudioBusKey);
	}

	check(IsInAudioThread());
	if (const Audio::FMixerSourceManager* MixerSourceManager = GetSourceManager())
	{
		return MixerSourceManager->IsAudioBusActive(InAudioBusKey);
	}
	return false;
}

Audio::FPatchInput UAudioBusSubsystem::AddPatchInputForAudioBus(Audio::FAudioBusKey InAudioBusKey, int32 InFrames, int32 InChannels, float InGain)
{
	Audio::FMixerSourceManager* SourceManager = GetMutableSourceManager();
	check(SourceManager);

	Audio::FMixerDevice* MixerDevice = GetMutableMixerDevice();
	if (!MixerDevice)
	{
		return Audio::FPatchInput();
	}

	Audio::FPatchInput PatchInput = MixerDevice->MakePatch(InFrames, InChannels, InGain);
	SourceManager->AddPendingAudioBusConnection(InAudioBusKey, InChannels, false, PatchInput);
	return PatchInput;
}

Audio::FPatchOutputStrongPtr UAudioBusSubsystem::AddPatchOutputForAudioBus(Audio::FAudioBusKey InAudioBusKey, int32 InFrames, int32 InChannels, float InGain)
{
	Audio::FMixerSourceManager* SourceManager = GetMutableSourceManager();
	check(SourceManager);

	Audio::FMixerDevice* MixerDevice = GetMutableMixerDevice();
	if (!MixerDevice)
	{
		return nullptr;
	}

	Audio::FPatchOutputStrongPtr PatchOutput = MixerDevice->MakePatch(InFrames, InChannels, InGain);
	SourceManager->AddPendingAudioBusConnection(InAudioBusKey, InChannels, false, PatchOutput);
	return PatchOutput;
}

Audio::FPatchInput UAudioBusSubsystem::AddPatchInputForSoundAndAudioBus(uint64 SoundInstanceID, Audio::FAudioBusKey AudioBusKey, int32 InFrames, int32 NumChannels, float InGain)
{
	Audio::FMixerDevice* MixerDevice = GetMutableMixerDevice();
	if (!MixerDevice)
	{
		return {};
	}

	if (Audio::FPatchOutputStrongPtr PatchOutput = MixerDevice->MakePatch(InFrames, NumChannels, InGain))
	{
		Audio::FPatchInput PatchInput = MoveTemp(PatchOutput);
		AddPendingConnection(SoundInstanceID, FPendingConnection{ .PatchVariant = FPendingConnection::FPatchVariant(TInPlaceType<Audio::FPatchInput>(), PatchInput),
																  .AudioBusKey = MoveTemp(AudioBusKey),
																  .BlockSizeFrames = InFrames,
																  .NumChannels = NumChannels,
#if UE_AUDIO_PROFILERTRACE_ENABLED
																  .InstanceID = SoundInstanceID 
#endif // UE_AUDIO_PROFILERTRACE_ENABLED
		});

		return PatchInput;
	}

	return {};
}

Audio::FPatchOutputStrongPtr UAudioBusSubsystem::AddPatchOutputForSoundAndAudioBus(uint64 SoundInstanceID, Audio::FAudioBusKey AudioBusKey, int32 InFrames, int32 NumChannels, float InGain)
{
	Audio::FMixerDevice* MixerDevice = GetMutableMixerDevice();
	if (!MixerDevice)
	{
		return {};
	}

	if (Audio::FPatchOutputStrongPtr PatchOutput = MixerDevice->MakePatch(InFrames, NumChannels, InGain))
	{
		AddPendingConnection(SoundInstanceID, FPendingConnection{ .PatchVariant = FPendingConnection::FPatchVariant(TInPlaceType<Audio::FPatchOutputStrongPtr>(), PatchOutput),
																  .AudioBusKey = MoveTemp(AudioBusKey),
																  .BlockSizeFrames = InFrames,
																  .NumChannels = NumChannels,
#if UE_AUDIO_PROFILERTRACE_ENABLED
																  .InstanceID = SoundInstanceID
#endif // UE_AUDIO_PROFILERTRACE_ENABLED
		});

		return PatchOutput;
	}

	return {};
}

void UAudioBusSubsystem::AddPendingConnection(uint64 SoundInstanceID, FPendingConnection&& PendingConnection)
{
	FScopeLock ScopeLock(&Mutex);
	FSoundInstanceConnections& SoundInstanceConnections = SoundInstanceConnectionMap.FindOrAdd(SoundInstanceID);
	SoundInstanceConnections.PendingConnections.Add(MoveTemp(PendingConnection));
}

void UAudioBusSubsystem::ConnectPatches(uint64 SoundInstanceID)
{
	TArray<FPendingConnection> PendingConnections = ExtractPendingConnectionsIfReady(SoundInstanceID);
	if (!PendingConnections.IsEmpty())
	{
		Audio::FMixerSourceManager* SourceManager = GetMutableSourceManager();
		check(SourceManager);
		for (FPendingConnection& PendingConnection : PendingConnections)
		{
			switch (PendingConnection.PatchVariant.GetIndex())
			{
			case FPendingConnection::FPatchVariant::IndexOfType<Audio::FPatchInput>():
				SourceManager->AddPendingAudioBusConnection(MoveTemp(PendingConnection.AudioBusKey)
															, PendingConnection.NumChannels
															, PendingConnection.bIsAutomatic
															, MoveTemp(PendingConnection.PatchVariant.Get<Audio::FPatchInput>())
															, INDEX_NONE
#if UE_AUDIO_PROFILERTRACE_ENABLED
															, PendingConnection.InstanceID
#endif // UE_AUDIO_PROFILERTRACE_ENABLED
				);

				break;
			case FPendingConnection::FPatchVariant::IndexOfType<Audio::FPatchOutputStrongPtr>():
				SourceManager->AddPendingAudioBusConnection(MoveTemp(PendingConnection.AudioBusKey)
															, PendingConnection.NumChannels
															, PendingConnection.bIsAutomatic
															, MoveTemp(PendingConnection.PatchVariant.Get<Audio::FPatchOutputStrongPtr>())
															, INDEX_NONE
#if UE_AUDIO_PROFILERTRACE_ENABLED
															, PendingConnection.InstanceID
#endif // UE_AUDIO_PROFILERTRACE_ENABLED
				);

				break;
			}
		}
	}
}

void UAudioBusSubsystem::RemoveSound(uint64 SoundInstanceID)
{
	FScopeLock ScopeLock(&Mutex);
	SoundInstanceConnectionMap.Remove(SoundInstanceID);
}

TArray<UAudioBusSubsystem::FPendingConnection> UAudioBusSubsystem::ExtractPendingConnectionsIfReady(uint64 SoundInstanceID)
{
	FScopeLock ScopeLock(&Mutex);
	if (FSoundInstanceConnections* SoundInstanceConnections = SoundInstanceConnectionMap.Find(SoundInstanceID))
	{
		TArray<FPendingConnection> PendingConnections = MoveTemp(SoundInstanceConnections->PendingConnections);
		SoundInstanceConnections->PendingConnections.Empty();
		return PendingConnections;
	}
	return {};
}

void UAudioBusSubsystem::InitDefaultAudioBuses()
{
	if (!ensure(IsInGameThread()))
	{
		return;
	}

	if (const UAudioSettings* AudioSettings = GetDefault<UAudioSettings>())
	{
		TArray<TStrongObjectPtr<UAudioBus>> StaleBuses = DefaultAudioBuses;
		DefaultAudioBuses.Reset();

		for (const FDefaultAudioBusSettings& BusSettings : AudioSettings->DefaultAudioBuses)
		{
			if (UObject* BusObject = BusSettings.AudioBus.TryLoad())
			{
				if (UAudioBus* AudioBus = Cast<UAudioBus>(BusObject))
				{
					const int32 NumChannels = static_cast<int32>(AudioBus->AudioBusChannels) + 1;
					StartAudioBus(Audio::FAudioBusKey(AudioBus->GetUniqueID()), AudioBus->GetPathName(), NumChannels, false /* bInIsAutomatic */);

					TStrongObjectPtr<UAudioBus>AddedBus(AudioBus);
					DefaultAudioBuses.AddUnique(AddedBus);
					StaleBuses.Remove(AddedBus);
				}
			}
		}

		for (TStrongObjectPtr<UAudioBus>& Bus : StaleBuses)
		{
			if (Bus.IsValid())
			{
				StopAudioBus(Audio::FAudioBusKey(Bus->GetUniqueID()));
			}
		}
	}
	else
	{
		UE_LOGF(LogAudioMixer, Error, "Failed to initialize Default Audio Buses. Audio Settings not found.");
	}
}

void UAudioBusSubsystem::ShutdownDefaultAudioBuses()
{
	if (!ensure(IsInGameThread()))
	{
		return;
	}

	for (TObjectIterator<UAudioBus> It; It; ++It)
	{
		UAudioBus* AudioBus = *It;
		if (AudioBus)
		{
			StopAudioBus(Audio::FAudioBusKey(AudioBus->GetUniqueID()));
		}
	}

	DefaultAudioBuses.Reset();
}
