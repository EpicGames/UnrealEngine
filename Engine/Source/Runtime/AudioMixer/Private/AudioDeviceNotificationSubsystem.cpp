// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioDeviceNotificationSubsystem.h"
#include "CoreGlobals.h"
#include "AudioThread.h"
#include "Async/Async.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioDeviceNotificationSubsystem)

void UAudioDeviceNotificationSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
}

void UAudioDeviceNotificationSubsystem::Deinitialize()
{
	DefaultCaptureDeviceChanged.Clear();
	DefaultRenderDeviceChanged.Clear();
	DeviceAdded.Clear();
	DeviceRemoved.Clear();
	DeviceStateChanged.Clear();

	DefaultCaptureDeviceChangedNative.Clear();
	DefaultRenderDeviceChangedNative.Clear();
	DeviceAddedNative.Clear();
	DeviceRemovedNative.Clear();
	DeviceStateChangedNative.Clear();
}

void UAudioDeviceNotificationSubsystem::OnDefaultCaptureDeviceChanged(const Audio::EAudioDeviceRole InAudioDeviceRole, const FString& DeviceId)
{
	TWeakObjectPtr<UAudioDeviceNotificationSubsystem> WeakThis(this);
	AsyncTask(ENamedThreads::GameThread, [WeakThis, InAudioDeviceRole, DeviceId]()
	{
		if (WeakThis.IsValid())
		{
			EAudioDeviceChangedRole NewRole = WeakThis->GetDeviceChangedRole(InAudioDeviceRole);
			WeakThis->DefaultCaptureDeviceChanged.Broadcast(NewRole, DeviceId);
			WeakThis->DefaultCaptureDeviceChangedNative.Broadcast(NewRole, DeviceId);
		}
	});
}

void UAudioDeviceNotificationSubsystem::OnDefaultRenderDeviceChanged(const Audio::EAudioDeviceRole InAudioDeviceRole, const FString& DeviceId)
{
	TWeakObjectPtr<UAudioDeviceNotificationSubsystem> WeakThis(this);
	AsyncTask(ENamedThreads::GameThread, [WeakThis, InAudioDeviceRole, DeviceId]()
	{
		if (WeakThis.IsValid())
		{
			EAudioDeviceChangedRole NewRole = WeakThis->GetDeviceChangedRole(InAudioDeviceRole);
			WeakThis->DefaultRenderDeviceChanged.Broadcast(NewRole, DeviceId);
			WeakThis->DefaultRenderDeviceChangedNative.Broadcast(NewRole, DeviceId);
		}
	});
}

void UAudioDeviceNotificationSubsystem::OnDeviceAdded(const FString& DeviceId, bool bIsRenderDevice)
{
	// We currently ignore changes in non-render devices
	if (!bIsRenderDevice)
	{
		return;
	}

	TWeakObjectPtr<UAudioDeviceNotificationSubsystem> WeakThis(this);
	AsyncTask(ENamedThreads::GameThread, [WeakThis, DeviceId]()
	{
		if (WeakThis.IsValid())
		{
			WeakThis->DeviceAdded.Broadcast(DeviceId);
			WeakThis->DeviceAddedNative.Broadcast(DeviceId);
		}
	});
}

void UAudioDeviceNotificationSubsystem::OnDeviceRemoved(const FString& DeviceId, bool bIsRenderDevice)
{
	// We currently ignore changes in non-render devices
	if (!bIsRenderDevice)
	{
		return;
	}
	
	TWeakObjectPtr<UAudioDeviceNotificationSubsystem> WeakThis(this);
	AsyncTask(ENamedThreads::GameThread, [WeakThis, DeviceId]()
	{
		if (WeakThis.IsValid())
		{
			WeakThis->DeviceRemoved.Broadcast(DeviceId);
			WeakThis->DeviceRemovedNative.Broadcast(DeviceId);
		}
	});
}

void UAudioDeviceNotificationSubsystem::OnDeviceStateChanged(const FString& DeviceId, const Audio::EAudioDeviceState InState, bool bIsRenderDevice)
{
	// We currently ignore changes in non-render devices
	if (!bIsRenderDevice)
	{
		return;
	}
	
	TWeakObjectPtr<UAudioDeviceNotificationSubsystem> WeakThis(this);
	AsyncTask(ENamedThreads::GameThread, [WeakThis, DeviceId, InState]()
	{
		if (WeakThis.IsValid())
		{
			EAudioDeviceChangedState NewState = WeakThis->GetDeviceChangedState(InState);
			WeakThis->DeviceStateChanged.Broadcast(DeviceId, NewState);
			WeakThis->DeviceStateChangedNative.Broadcast(DeviceId, NewState);
		}
	});
}

void UAudioDeviceNotificationSubsystem::OnDeviceSwitched(const FString& DeviceId)
{
	TWeakObjectPtr<UAudioDeviceNotificationSubsystem> WeakThis(this);
	AsyncTask(ENamedThreads::GameThread, [WeakThis, DeviceId]()
	{
		if (WeakThis.IsValid())
		{
			WeakThis->DeviceSwitched.Broadcast(DeviceId);
			WeakThis->DeviceSwitchedNative.Broadcast(DeviceId);
		}
	});
}

EAudioDeviceChangedRole UAudioDeviceNotificationSubsystem::GetDeviceChangedRole(Audio::EAudioDeviceRole InRole) const
{
	EAudioDeviceChangedRole Role;
	switch (InRole)
	{
		case Audio::EAudioDeviceRole::Console:
			Role = EAudioDeviceChangedRole::Console;
			break;

		case Audio::EAudioDeviceRole::Multimedia:
			Role = EAudioDeviceChangedRole::Multimedia;
			break;

		case Audio::EAudioDeviceRole::Communications:
			Role = EAudioDeviceChangedRole::Communications;
			break;
			
		default:
			Role = EAudioDeviceChangedRole::Invalid;
			break;
	}

	return Role;
}

EAudioDeviceChangedState UAudioDeviceNotificationSubsystem::GetDeviceChangedState(Audio::EAudioDeviceState InState) const
{
	EAudioDeviceChangedState OutState;
	switch (InState)
	{
	case Audio::EAudioDeviceState::Active:
		OutState = EAudioDeviceChangedState::Active;
		break;

	case Audio::EAudioDeviceState::Disabled:
		OutState = EAudioDeviceChangedState::Disabled;
		break;

	case Audio::EAudioDeviceState::NotPresent:
		OutState = EAudioDeviceChangedState::NotPresent;
		break;

	case Audio::EAudioDeviceState::Unplugged:
		OutState = EAudioDeviceChangedState::Unplugged;
		break;

	default:
		OutState = EAudioDeviceChangedState::Invalid;
		break;
	}

	return OutState;
}

