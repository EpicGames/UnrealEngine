// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioMixerBlueprintLibrary.h"

#include "Algo/Transform.h"
#include "Async/Async.h"
#include "AudioBusSubsystem.h"
#include "AudioCompressionSettingsUtils.h"
#include "AudioDevice.h"
#include "AudioDeviceManager.h"
#include "AudioMixerDevice.h"
#include "AudioMixerSubmix.h"
#include "ContentStreaming.h"
#include "CoreMinimal.h"
#include "DSP/ConstantQ.h"
#include "DSP/SpectrumAnalyzer.h"
#include "Engine/World.h"
#include "Sound/SoundEffectPreset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioMixerBlueprintLibrary)

// This is our global recording task:
static TUniquePtr<Audio::FAudioRecordingData> RecordingData;

FAudioOutputDeviceInfo::FAudioOutputDeviceInfo(const Audio::FAudioPlatformDeviceInfo& InDeviceInfo)
	: Name(InDeviceInfo.Name)
	, DeviceId(InDeviceInfo.DeviceId)
	, NumChannels(InDeviceInfo.NumChannels)
	, SampleRate(InDeviceInfo.SampleRate)
	, Format(EAudioMixerStreamDataFormatType(InDeviceInfo.Format))
	, bIsSystemDefault(InDeviceInfo.bIsSystemDefault)
	, bIsCurrentDevice(false)
{
	for (EAudioMixerChannel::Type i : InDeviceInfo.OutputChannelArray)
	{
		OutputChannelArray.Emplace(EAudioMixerChannelType(i));
	}

}

FString UAudioMixerBlueprintLibrary::Conv_AudioOutputDeviceInfoToString(const FAudioOutputDeviceInfo& InDeviceInfo)
{
	FString output = FString::Printf(TEXT("Device Name: %s, \nDevice Id: %s, \nNum Channels: %u, \nSample Rate: %u, \nFormat: %s,  \nIs System Default: %u, \n"),
		*InDeviceInfo.Name, *InDeviceInfo.DeviceId, InDeviceInfo.NumChannels, InDeviceInfo.SampleRate,
		*DataFormatAsString(EAudioMixerStreamDataFormatType(InDeviceInfo.Format)), InDeviceInfo.bIsSystemDefault);

	output.Append("Output Channel Array: \n");

	for (int32 i = 0; i < InDeviceInfo.NumChannels; ++i)
	{
		if (i < InDeviceInfo.OutputChannelArray.Num())
		{
			output += FString::Printf(TEXT("	%d: %s \n"), i, ToString(InDeviceInfo.OutputChannelArray[i]));
		}
	}

	return output;
}

FString DataFormatAsString(EAudioMixerStreamDataFormatType type)
{
	switch (type)
	{
	case EAudioMixerStreamDataFormatType::Unknown:
		return FString("Unknown");
	case EAudioMixerStreamDataFormatType::Float:
		return FString("Float");
	case EAudioMixerStreamDataFormatType::Int16:
		return FString("Int16");
	case EAudioMixerStreamDataFormatType::Unsupported:
		return FString("Unsupported");
	default:
		return FString("Invalid Format Type");
	}
}


void UAudioMixerBlueprintLibrary::AddMasterSubmixEffect(const UObject* WorldContextObject, USoundEffectSubmixPreset* SubmixEffectPreset)
{
	if (!SubmixEffectPreset)
	{
		UE_LOGF(LogAudioMixer, Warning, "AddMasterSubmixEffect was passed invalid submix effect preset");
		return;
	}

	if (Audio::FMixerDevice* MixerDevice = FAudioDeviceManager::GetAudioMixerDeviceFromWorldContext(WorldContextObject))
	{
		FSoundEffectSubmixInitData InitData;
		InitData.SampleRate = MixerDevice->GetSampleRate();
		InitData.DeviceID = MixerDevice->DeviceID;
		InitData.PresetSettings = nullptr;
		InitData.ParentPresetUniqueId = SubmixEffectPreset->GetUniqueID();

		// Immediately create a new sound effect base here before the object becomes potentially invalidated
		TSoundEffectSubmixPtr SoundEffectSubmix = USoundEffectPreset::CreateInstance<FSoundEffectSubmixInitData, FSoundEffectSubmix>(InitData, *SubmixEffectPreset);
		SoundEffectSubmix->SetEnabled(true);

		MixerDevice->AddMasterSubmixEffect(SoundEffectSubmix);
	}
}

void UAudioMixerBlueprintLibrary::RemoveMasterSubmixEffect(const UObject* WorldContextObject, USoundEffectSubmixPreset* SubmixEffectPreset)
{
	if (!SubmixEffectPreset)
	{
		UE_LOGF(LogAudioMixer, Warning, "RemoveMasterSubmixEffect was passed invalid submix effect preset");
		return;
	}

	if (Audio::FMixerDevice* MixerDevice = FAudioDeviceManager::GetAudioMixerDeviceFromWorldContext(WorldContextObject))
	{
		// Get the unique id for the preset object on the game thread. Used to refer to the object on audio render thread.
		uint32 SubmixPresetUniqueId = SubmixEffectPreset->GetUniqueID();

		MixerDevice->RemoveMasterSubmixEffect(SubmixPresetUniqueId);
	}
}

void UAudioMixerBlueprintLibrary::ClearMasterSubmixEffects(const UObject* WorldContextObject)
{
	if (Audio::FMixerDevice* MixerDevice = FAudioDeviceManager::GetAudioMixerDeviceFromWorldContext(WorldContextObject))
	{
		MixerDevice->ClearMasterSubmixEffects();
	}
}

int32 UAudioMixerBlueprintLibrary::AddSubmixEffect(const UObject* WorldContextObject, USoundSubmix* InSoundSubmix, USoundEffectSubmixPreset* SubmixEffectPreset)
{
	if (!SubmixEffectPreset || !InSoundSubmix)
	{
		return 0;
	}

	if (Audio::FMixerDevice* MixerDevice = FAudioDeviceManager::GetAudioMixerDeviceFromWorldContext(WorldContextObject))
	{
		FSoundEffectSubmixInitData InitData;
		InitData.DeviceID = MixerDevice->DeviceID;
		InitData.SampleRate = MixerDevice->GetSampleRate();
		InitData.ParentPresetUniqueId = SubmixEffectPreset->GetUniqueID();

		TSoundEffectSubmixPtr SoundEffectSubmix = USoundEffectPreset::CreateInstance<FSoundEffectSubmixInitData, FSoundEffectSubmix>(InitData, *SubmixEffectPreset);
		SoundEffectSubmix->SetEnabled(true);

		return MixerDevice->AddSubmixEffect(InSoundSubmix, SoundEffectSubmix);
	}

	return 0;
}

void UAudioMixerBlueprintLibrary::RemoveSubmixEffectPreset(const UObject* WorldContextObject, USoundSubmix* InSoundSubmix, USoundEffectSubmixPreset* InSubmixEffectPreset)
{
	RemoveSubmixEffect(WorldContextObject, InSoundSubmix, InSubmixEffectPreset);
}

void UAudioMixerBlueprintLibrary::RemoveSubmixEffect(const UObject* WorldContextObject, USoundSubmix* InSoundSubmix, USoundEffectSubmixPreset* InSubmixEffectPreset)
{
	if (Audio::FMixerDevice* MixerDevice = FAudioDeviceManager::GetAudioMixerDeviceFromWorldContext(WorldContextObject))
	{
		uint32 SubmixPresetUniqueId = InSubmixEffectPreset->GetUniqueID();
		MixerDevice->RemoveSubmixEffect(InSoundSubmix, SubmixPresetUniqueId);
	}
}

void UAudioMixerBlueprintLibrary::RemoveSubmixEffectPresetAtIndex(const UObject* WorldContextObject, USoundSubmix* InSoundSubmix, int32 SubmixChainIndex)
{
	RemoveSubmixEffectAtIndex(WorldContextObject, InSoundSubmix, SubmixChainIndex);
}

void UAudioMixerBlueprintLibrary::RemoveSubmixEffectAtIndex(const UObject* WorldContextObject, USoundSubmix* InSoundSubmix, int32 SubmixChainIndex)
{
	if (Audio::FMixerDevice* MixerDevice = FAudioDeviceManager::GetAudioMixerDeviceFromWorldContext(WorldContextObject))
	{
		MixerDevice->RemoveSubmixEffectAtIndex(InSoundSubmix, SubmixChainIndex);
	}
}

void UAudioMixerBlueprintLibrary::ReplaceSoundEffectSubmix(const UObject* WorldContextObject, USoundSubmix* InSoundSubmix, int32 SubmixChainIndex, USoundEffectSubmixPreset* SubmixEffectPreset)
{
	ReplaceSubmixEffect(WorldContextObject, InSoundSubmix, SubmixChainIndex, SubmixEffectPreset);
}

void UAudioMixerBlueprintLibrary::ReplaceSubmixEffect(const UObject* WorldContextObject, USoundSubmix* InSoundSubmix, int32 SubmixChainIndex, USoundEffectSubmixPreset* SubmixEffectPreset)
{
	if (!SubmixEffectPreset || !InSoundSubmix)
	{
		return;
	}

	if (Audio::FMixerDevice* MixerDevice = FAudioDeviceManager::GetAudioMixerDeviceFromWorldContext(WorldContextObject))
	{
		FSoundEffectSubmixInitData InitData;
		InitData.SampleRate = MixerDevice->GetSampleRate();

		TSoundEffectSubmixPtr SoundEffectSubmix = USoundEffectPreset::CreateInstance<FSoundEffectSubmixInitData, FSoundEffectSubmix>(InitData, *SubmixEffectPreset);
		SoundEffectSubmix->SetEnabled(true);

		MixerDevice->ReplaceSoundEffectSubmix(InSoundSubmix, SubmixChainIndex, SoundEffectSubmix);
	}
}

void UAudioMixerBlueprintLibrary::ClearSubmixEffects(const UObject* WorldContextObject, USoundSubmix* InSoundSubmix)
{
	if (Audio::FMixerDevice* MixerDevice = FAudioDeviceManager::GetAudioMixerDeviceFromWorldContext(WorldContextObject))
	{
		MixerDevice->ClearSubmixEffects(InSoundSubmix);
	}
}

void UAudioMixerBlueprintLibrary::SetSubmixEffectChainOverride(const UObject* WorldContextObject, USoundSubmix* InSoundSubmix, TArray<USoundEffectSubmixPreset*> InSubmixEffectPresetChain, float InFadeTimeSec)
{
	if (Audio::FMixerDevice* MixerDevice = FAudioDeviceManager::GetAudioMixerDeviceFromWorldContext(WorldContextObject))
	{
		TArray<FSoundEffectSubmixPtr> NewSubmixEffectPresetChain;

		for (USoundEffectSubmixPreset* SubmixEffectPreset : InSubmixEffectPresetChain)
		{
			if (SubmixEffectPreset)
			{
				FSoundEffectSubmixInitData InitData;
				InitData.DeviceID = MixerDevice->DeviceID;
				InitData.SampleRate = MixerDevice->GetSampleRate();
				InitData.ParentPresetUniqueId = SubmixEffectPreset->GetUniqueID();

				TSoundEffectSubmixPtr SoundEffectSubmix = USoundEffectPreset::CreateInstance<FSoundEffectSubmixInitData, FSoundEffectSubmix>(InitData, *SubmixEffectPreset);
				SoundEffectSubmix->SetEnabled(true);

				NewSubmixEffectPresetChain.Add(SoundEffectSubmix);
			}
		}
		
		if (NewSubmixEffectPresetChain.Num() > 0)
		{
			MixerDevice->SetSubmixEffectChainOverride(InSoundSubmix, NewSubmixEffectPresetChain, InFadeTimeSec);
		}
	}
}

void UAudioMixerBlueprintLibrary::ClearSubmixEffectChainOverride(const UObject* WorldContextObject, USoundSubmix* InSoundSubmix, float InFadeTimeSec)
{
	if (Audio::FMixerDevice* MixerDevice = FAudioDeviceManager::GetAudioMixerDeviceFromWorldContext(WorldContextObject))
	{
		MixerDevice->ClearSubmixEffectChainOverride(InSoundSubmix, InFadeTimeSec);
	}
}

void UAudioMixerBlueprintLibrary::StartRecordingOutput(const UObject* WorldContextObject, float ExpectedDuration, USoundSubmix* SubmixToRecord)
{
	if (Audio::FMixerDevice* MixerDevice = FAudioDeviceManager::GetAudioMixerDeviceFromWorldContext(WorldContextObject))
	{
		MixerDevice->StartRecording(SubmixToRecord, ExpectedDuration);
	}
	else
	{
		UE_LOGF(LogAudioMixer, Error, "Output recording is an audio mixer only feature.");
	}
}

USoundWave* UAudioMixerBlueprintLibrary::StopRecordingOutput(const UObject* WorldContextObject, EAudioRecordingExportType ExportType, const FString& Name, FString Path, USoundSubmix* SubmixToRecord, USoundWave* ExistingSoundWaveToOverwrite)
{
	if (RecordingData.IsValid())
	{
		UE_LOGF(LogAudioMixer, Warning, "Abandoning existing write operation. If you'd like to export multiple submix recordings at the same time, use Start/Finish Recording Submix Output instead.");
	}

	if (Audio::FMixerDevice* MixerDevice = FAudioDeviceManager::GetAudioMixerDeviceFromWorldContext(WorldContextObject))
	{
		float SampleRate;
		float ChannelCount;

		// call the thing here.
		Audio::FAlignedFloatBuffer& RecordedBuffer = MixerDevice->StopRecording(SubmixToRecord, ChannelCount, SampleRate);

		if (RecordedBuffer.Num() == 0)
		{
			UE_LOGF(LogAudioMixer, Warning, "No audio data. Did you call Start Recording Output?");
			return nullptr;
		}

		// Pack output data into a TSampleBuffer and record out:
		RecordingData.Reset(new Audio::FAudioRecordingData());
		RecordingData->InputBuffer = Audio::TSampleBuffer<int16>(RecordedBuffer, ChannelCount, SampleRate);

		switch (ExportType)
		{
		case EAudioRecordingExportType::SoundWave:
		{
			USoundWave* ResultingSoundWave = RecordingData->Writer.SynchronouslyWriteSoundWave(RecordingData->InputBuffer, &Name, &Path);
			RecordingData.Reset();
			return ResultingSoundWave;
		}
		case EAudioRecordingExportType::WavFile:
		{
			RecordingData->Writer.BeginWriteToWavFile(RecordingData->InputBuffer, Name, Path, [SubmixToRecord]()
			{
				if (SubmixToRecord && SubmixToRecord->OnSubmixRecordedFileDone.IsBound())
				{
					SubmixToRecord->OnSubmixRecordedFileDone.Broadcast(nullptr);
				}

				// I'm gonna try this, but I do not feel great about it.
				RecordingData.Reset();
			});
			break;
		}
		default:
			break;
		}	
	}
	else
	{
		UE_LOGF(LogAudioMixer, Error, "Output recording is an audio mixer only feature.");
	}

	return nullptr;
}

void UAudioMixerBlueprintLibrary::PauseRecordingOutput(const UObject* WorldContextObject, USoundSubmix* SubmixToPause)
{
	if (Audio::FMixerDevice* MixerDevice = FAudioDeviceManager::GetAudioMixerDeviceFromWorldContext(WorldContextObject))
	{
		MixerDevice->PauseRecording(SubmixToPause);
	}
	else
	{
		UE_LOGF(LogAudioMixer, Error, "Output recording is an audio mixer only feature.");
	}
}

void UAudioMixerBlueprintLibrary::ResumeRecordingOutput(const UObject* WorldContextObject, USoundSubmix* SubmixToResume)
{
	if (Audio::FMixerDevice* MixerDevice = FAudioDeviceManager::GetAudioMixerDeviceFromWorldContext(WorldContextObject))
	{
		MixerDevice->ResumeRecording(SubmixToResume);
	}
	else
	{
		UE_LOGF(LogAudioMixer, Error, "Output recording is an audio mixer only feature.");
	}
}

void UAudioMixerBlueprintLibrary::StartAnalyzingOutput(const UObject* WorldContextObject, USoundSubmix* SubmixToAnalyze, EFFTSize FFTSize, EFFTPeakInterpolationMethod InterpolationMethod, EFFTWindowType WindowType, float HopSize, EAudioSpectrumType AudioSpectrumType)
{
	if (Audio::FMixerDevice* MixerDevice = FAudioDeviceManager::GetAudioMixerDeviceFromWorldContext(WorldContextObject))
	{
		FSoundSpectrumAnalyzerSettings Settings = USoundSubmix::GetSpectrumAnalyzerSettings(FFTSize, InterpolationMethod, WindowType, HopSize, AudioSpectrumType);
		MixerDevice->StartSpectrumAnalysis(SubmixToAnalyze, Settings);
	}
	else
	{
		UE_LOGF(LogAudioMixer, Error, "Spectrum Analysis is an audio mixer only feature.");
	}
}

void UAudioMixerBlueprintLibrary::StopAnalyzingOutput(const UObject* WorldContextObject, USoundSubmix* SubmixToStopAnalyzing)
{
	if (Audio::FMixerDevice* MixerDevice = FAudioDeviceManager::GetAudioMixerDeviceFromWorldContext(WorldContextObject))
	{
		MixerDevice->StopSpectrumAnalysis(SubmixToStopAnalyzing);
	}
	else
	{
		UE_LOGF(LogAudioMixer, Error, "Spectrum Analysis is an audio mixer only feature.");
	}
}

TArray<FSoundSubmixSpectralAnalysisBandSettings> UAudioMixerBlueprintLibrary::MakeMusicalSpectralAnalysisBandSettings(int32 InNumNotes, EMusicalNoteName InStartingMusicalNote, int32 InStartingOctave, int32 InAttackTimeMsec, int32 InReleaseTimeMsec)
{
	// Make values sane.
	InNumNotes = FMath::Clamp(InNumNotes, 0, 10000);
	InStartingOctave = FMath::Clamp(InStartingOctave, -1, 10);
	InAttackTimeMsec = FMath::Clamp(InAttackTimeMsec, 0, 10000);
	InReleaseTimeMsec = FMath::Clamp(InReleaseTimeMsec, 0, 10000);

	// Some assumptions here on what constitutes "music". 12 notes, equal temperament.
	const float BandsPerOctave = 12.f;
	// This QFactor makes the bandwidth equal to the difference in frequency between adjacent notes.
	const float QFactor = 1.f / (FMath::Pow(2.f, 1.f / BandsPerOctave) - 1.f);

	// Base note index off of A4 which we know to be 440 hz.
	// Make note relative to A
	int32 NoteIndex = static_cast<int32>(InStartingMusicalNote) - static_cast<int32>(EMusicalNoteName::A);
	// Make relative to 4th octave of A
	NoteIndex += 12 * (InStartingOctave - 4);

	const float StartingFrequency = 440.f * FMath::Pow(2.f, static_cast<float>(NoteIndex) / 12.f);


	TArray<FSoundSubmixSpectralAnalysisBandSettings> BandSettingsArray;
	for (int32 i = 0; i < InNumNotes; i++)
	{
		FSoundSubmixSpectralAnalysisBandSettings BandSettings;

		BandSettings.BandFrequency = Audio::FPseudoConstantQ::GetConstantQCenterFrequency(i, StartingFrequency, BandsPerOctave);

		BandSettings.QFactor = QFactor;
		BandSettings.AttackTimeMsec = InAttackTimeMsec;
		BandSettings.ReleaseTimeMsec = InReleaseTimeMsec;

		BandSettingsArray.Add(BandSettings);
	}

	return BandSettingsArray;
}

TArray<FSoundSubmixSpectralAnalysisBandSettings> UAudioMixerBlueprintLibrary::MakeFullSpectrumSpectralAnalysisBandSettings(int32 InNumBands, float InMinimumFrequency, float InMaximumFrequency, int32 InAttackTimeMsec, int32 InReleaseTimeMsec)
{
	// Make inputs sane.
	InNumBands = FMath::Clamp(InNumBands, 0, 10000);
	InMinimumFrequency = FMath::Clamp(InMinimumFrequency, 20.0f, 20000.0f);
	InMaximumFrequency = FMath::Clamp(InMaximumFrequency, InMinimumFrequency, 20000.0f);
	InAttackTimeMsec = FMath::Clamp(InAttackTimeMsec, 0, 10000);
	InReleaseTimeMsec = FMath::Clamp(InReleaseTimeMsec, 0, 10000);

	// Calculate CQT settings needed to space bands.
	const float NumOctaves = FMath::Loge(InMaximumFrequency / InMinimumFrequency) / FMath::Loge(2.f);
	const float BandsPerOctave = static_cast<float>(InNumBands) / FMath::Max(NumOctaves, 0.01f);
	const float QFactor = 1.f / (FMath::Pow(2.f, 1.f / FMath::Max(BandsPerOctave, 0.01f)) - 1.f);

	TArray<FSoundSubmixSpectralAnalysisBandSettings> BandSettingsArray;
	for (int32 i = 0; i < InNumBands; i++)
	{
		FSoundSubmixSpectralAnalysisBandSettings BandSettings;

		BandSettings.BandFrequency = Audio::FPseudoConstantQ::GetConstantQCenterFrequency(i, InMinimumFrequency, BandsPerOctave);

		BandSettings.QFactor = QFactor;
		BandSettings.AttackTimeMsec = InAttackTimeMsec;
		BandSettings.ReleaseTimeMsec = InReleaseTimeMsec;

		BandSettingsArray.Add(BandSettings);
	}

	return BandSettingsArray;
}

TArray<FSoundSubmixSpectralAnalysisBandSettings> UAudioMixerBlueprintLibrary::MakePresetSpectralAnalysisBandSettings(EAudioSpectrumBandPresetType InBandPresetType, int32 InNumBands, int32 InAttackTimeMsec, int32 InReleaseTimeMsec)
{
	float MinimumFrequency = 20.f;
	float MaximumFrequency = 20000.f;

	// Likely all these are debatable. What we are shooting for is the most active frequency
	// ranges, so when an instrument plays a significant amount of spectral energy from that
	// instrument will show up in the frequency range. 
	switch (InBandPresetType)
	{
		case EAudioSpectrumBandPresetType::KickDrum:
			MinimumFrequency = 40.f;
			MaximumFrequency = 100.f;
			break;

		case EAudioSpectrumBandPresetType::SnareDrum:
			MinimumFrequency = 150.f;
			MaximumFrequency = 4500.f;
			break;

		case EAudioSpectrumBandPresetType::Voice:
			MinimumFrequency = 300.f;
			MaximumFrequency = 3000.f;
			break;

		case EAudioSpectrumBandPresetType::Cymbals:
			MinimumFrequency = 6000.f;
			MaximumFrequency = 16000.f;
			break;

		// More presets can be added. The possibilities are endless.
	}

	return MakeFullSpectrumSpectralAnalysisBandSettings(InNumBands, MinimumFrequency, MaximumFrequency, InAttackTimeMsec, InReleaseTimeMsec);
}

void UAudioMixerBlueprintLibrary::GetMagnitudeForFrequencies(const UObject* WorldContextObject, const TArray<float>& Frequencies, TArray<float>& Magnitudes, USoundSubmix* SubmixToAnalyze /*= nullptr*/)
{
	if (Audio::FMixerDevice* MixerDevice = FAudioDeviceManager::GetAudioMixerDeviceFromWorldContext(WorldContextObject))
	{
		MixerDevice->GetMagnitudesForFrequencies(SubmixToAnalyze, Frequencies, Magnitudes);
	}
	else
	{
		UE_LOGF(LogAudioMixer, Error, "Getting magnitude for frequencies is an audio mixer only feature.");
	}
}

void UAudioMixerBlueprintLibrary::GetPhaseForFrequencies(const UObject* WorldContextObject, const TArray<float>& Frequencies, TArray<float>& Phases, USoundSubmix* SubmixToAnalyze /*= nullptr*/)
{
	if (Audio::FMixerDevice* MixerDevice = FAudioDeviceManager::GetAudioMixerDeviceFromWorldContext(WorldContextObject))
	{
		MixerDevice->GetPhasesForFrequencies(SubmixToAnalyze, Frequencies, Phases);
	}
	else
	{
		UE_LOGF(LogAudioMixer, Error, "Output recording is an audio mixer only feature.");
	}
}

void UAudioMixerBlueprintLibrary::AddSourceEffectToPresetChain(const UObject* WorldContextObject, USoundEffectSourcePresetChain* PresetChain, FSourceEffectChainEntry Entry)
{
	if (!PresetChain)
	{
		UE_LOGF(LogAudioMixer, Warning, "AddSourceEffectToPresetChain was passed invalid preset chain");
		return;
	}

	if (Audio::FMixerDevice* MixerDevice = FAudioDeviceManager::GetAudioMixerDeviceFromWorldContext(WorldContextObject))
	{
		TArray<FSourceEffectChainEntry> Chain;

		uint32 PresetChainId = PresetChain->GetUniqueID();

		if (!MixerDevice->GetCurrentSourceEffectChain(PresetChainId, Chain))
		{
			Chain = PresetChain->Chain;
		}

		Chain.Add(Entry);
		MixerDevice->UpdateSourceEffectChain(PresetChainId, Chain, PresetChain->bPlayEffectChainTails);
	}
}

void UAudioMixerBlueprintLibrary::RemoveSourceEffectFromPresetChain(const UObject* WorldContextObject, USoundEffectSourcePresetChain* PresetChain, int32 EntryIndex)
{
	if (!PresetChain)
	{
		UE_LOGF(LogAudioMixer, Warning, "RemoveSourceEffectFromPresetChain was passed invalid preset chain");
		return;
	}

	if (Audio::FMixerDevice* MixerDevice = FAudioDeviceManager::GetAudioMixerDeviceFromWorldContext(WorldContextObject))
	{
		TArray<FSourceEffectChainEntry> Chain;

		uint32 PresetChainId = PresetChain->GetUniqueID();

		if (!MixerDevice->GetCurrentSourceEffectChain(PresetChainId, Chain))
		{
			Chain = PresetChain->Chain;
		}

		if (EntryIndex >= 0 && EntryIndex < Chain.Num())
		{
			Chain.RemoveAt(EntryIndex);
			MixerDevice->UpdateSourceEffectChain(PresetChainId, Chain, PresetChain->bPlayEffectChainTails);
		}
	}

}

void UAudioMixerBlueprintLibrary::SetBypassSourceEffectChainEntry(const UObject* WorldContextObject, USoundEffectSourcePresetChain* PresetChain, int32 EntryIndex, bool bBypassed)
{
	if (!PresetChain)
	{
		UE_LOGF(LogAudioMixer, Warning, "SetBypassSourceEffectChainEntry was passed invalid preset chain");
		return;
	}

	if (Audio::FMixerDevice* MixerDevice = FAudioDeviceManager::GetAudioMixerDeviceFromWorldContext(WorldContextObject))
	{
		TArray<FSourceEffectChainEntry> Chain;

		uint32 PresetChainId = PresetChain->GetUniqueID();

		if (!MixerDevice->GetCurrentSourceEffectChain(PresetChainId, Chain))
		{
			Chain = PresetChain->Chain;
		}

		if (EntryIndex >= 0 && EntryIndex < Chain.Num())
		{
			Chain[EntryIndex].bBypass = bBypassed;
			MixerDevice->UpdateSourceEffectChain(PresetChainId, Chain, PresetChain->bPlayEffectChainTails);
		}
	}
}

int32 UAudioMixerBlueprintLibrary::GetNumberOfEntriesInSourceEffectChain(const UObject* WorldContextObject, USoundEffectSourcePresetChain* PresetChain)
{
	if (!PresetChain)
	{
		UE_LOGF(LogAudioMixer, Warning, "GetNumberOfEntriesInSourceEffectChain was passed invalid preset chain");
		return 0;
	}

	if (Audio::FMixerDevice* MixerDevice = FAudioDeviceManager::GetAudioMixerDeviceFromWorldContext(WorldContextObject))
	{
		TArray<FSourceEffectChainEntry> Chain;

		uint32 PresetChainId = PresetChain->GetUniqueID();

		if (!MixerDevice->GetCurrentSourceEffectChain(PresetChainId, Chain))
		{
			return PresetChain->Chain.Num();
		}

		return Chain.Num();
	}

	return 0;
}

void UAudioMixerBlueprintLibrary::PrimeSoundForPlayback(USoundWave* SoundWave, const FOnSoundLoadComplete OnLoadCompletion)
{
	if (!SoundWave)
	{
		UE_LOGF(LogAudioMixer, Warning, "Prime Sound For Playback called with a null SoundWave pointer.");
		OnLoadCompletion.ExecuteIfBound(SoundWave, true);

		return;
	}

	TSharedRef<const FSoundWaveProxy> SoundWaveProxy = SoundWave->GetSoundWaveProxy();

	IStreamingManager::Get().GetAudioStreamingManager().RequestChunk(SoundWaveProxy->GetSoundWaveDataRef(), 1, [OnLoadCompletion, SoundWave](EAudioChunkLoadResult InResult)
	{
		AsyncTask(ENamedThreads::GameThread, [OnLoadCompletion, SoundWave, InResult]() {
			if (InResult == EAudioChunkLoadResult::Completed || InResult == EAudioChunkLoadResult::AlreadyLoaded)
			{
				OnLoadCompletion.ExecuteIfBound(SoundWave, false);
			}
			else
			{
				OnLoadCompletion.ExecuteIfBound(SoundWave, true);
			}
		});
	});
}

void UAudioMixerBlueprintLibrary::PrimeSoundCueForPlayback(USoundCue* SoundCue)
{
	if (SoundCue)
	{
		SoundCue->PrimeSoundCue();
	}
}

float UAudioMixerBlueprintLibrary::TrimAudioCache(float InMegabytesToFree)
{
	uint64 NumBytesToFree = (uint64) (((double)InMegabytesToFree) * 1024.0 * 1024.0);
	uint64 NumBytesFreed = IStreamingManager::Get().GetAudioStreamingManager().TrimMemory(NumBytesToFree);
	return (float)(((double) NumBytesFreed / 1024) / 1024.0);
}

void UAudioMixerBlueprintLibrary::StartAudioBus(const UObject* WorldContextObject, UAudioBus* AudioBus)
{
	if (!AudioBus)
	{
		UE_LOGF(LogAudioMixer, Error, "Start Audio Bus called with an invalid Audio Bus.");
		return;
	}
	if (Audio::FMixerDevice* MixerDevice = FAudioDeviceManager::GetAudioMixerDeviceFromWorldContext(WorldContextObject))
	{
		uint32 AudioBusId = AudioBus->GetUniqueID();
		int32 NumChannels = (int32)AudioBus->AudioBusChannels + 1;

		UAudioBusSubsystem* AudioBusSubsystem = MixerDevice->GetSubsystem<UAudioBusSubsystem>();
		check(AudioBusSubsystem);
		AudioBusSubsystem->StartAudioBus(Audio::FAudioBusKey(AudioBusId), AudioBus->GetPathName(), NumChannels, false);
	}
	else
	{
		UE_LOGF(LogAudioMixer, Error, "Audio buses are an audio mixer only feature. Please run the game with audio mixer enabled for this feature.");
	}
}

void UAudioMixerBlueprintLibrary::StopAudioBus(const UObject* WorldContextObject, UAudioBus* AudioBus)
{
	if (!AudioBus)
	{
		UE_LOGF(LogAudioMixer, Error, "Stop Audio Bus called with an invalid Audio Bus.");
		return;
	}
	if (Audio::FMixerDevice* MixerDevice = FAudioDeviceManager::GetAudioMixerDeviceFromWorldContext(WorldContextObject))
	{
		uint32 AudioBusId = AudioBus->GetUniqueID();
		UAudioBusSubsystem* AudioBusSubsystem = MixerDevice->GetSubsystem<UAudioBusSubsystem>();
		check(AudioBusSubsystem);
		AudioBusSubsystem->StopAudioBus(Audio::FAudioBusKey(AudioBusId));
	}
	else
	{
		UE_LOGF(LogAudioMixer, Error, "Audio buses are an audio mixer only feature. Please run the game with audio mixer enabled for this feature.");
	}
}

bool UAudioMixerBlueprintLibrary::IsAudioBusActive(const UObject* WorldContextObject, UAudioBus* AudioBus)
{
	if (!AudioBus)
	{
		UE_LOGF(LogAudioMixer, Error, "Is Audio Bus Active called with an invalid Audio Bus.");
		return false;
	}
	if (Audio::FMixerDevice* MixerDevice = FAudioDeviceManager::GetAudioMixerDeviceFromWorldContext(WorldContextObject))
	{
		uint32 AudioBusId = AudioBus->GetUniqueID(); 
		UAudioBusSubsystem* AudioBusSubsystem = MixerDevice->GetSubsystem<UAudioBusSubsystem>();
		check(AudioBusSubsystem);
		return AudioBusSubsystem->IsAudioBusActive(Audio::FAudioBusKey(AudioBusId));
	}
	else
	{
		UE_LOGF(LogAudioMixer, Error, "Audio buses are an audio mixer only feature. Please run the game with audio mixer enabled for this feature.");
		return false;
	}
}

void UAudioMixerBlueprintLibrary::RegisterAudioBusToSubmix(const UObject* WorldContextObject, USoundSubmix* SoundSubmix, UAudioBus* AudioBus)
{
	if (!SoundSubmix)
	{
		UE_LOGF(LogAudioMixer, Error, "RegisterAudioBusToSubmix called with an invalid Submix.");
		return;
	}

	if (!AudioBus)
	{
		UE_LOGF(LogAudioMixer, Error, "RegisterAudioBusToSubmix called with an invalid Audio Bus.");
		return;
	}

	if (!IsInAudioThread())
	{
		//Send this over to the audio thread, with the same settings
		FAudioThread::RunCommandOnAudioThread([WorldContextObject, SoundSubmix, AudioBus]()
		{
			RegisterAudioBusToSubmix(WorldContextObject, SoundSubmix, AudioBus);
		});

		return;
	}

	if (Audio::FMixerDevice* MixerDevice = FAudioDeviceManager::GetAudioMixerDeviceFromWorldContext(WorldContextObject))
	{
		Audio::FMixerSubmixPtr MixerSubmixPtr = MixerDevice->GetSubmixInstance(SoundSubmix).Pin();

		if (MixerSubmixPtr.IsValid())
		{
			UAudioBusSubsystem* AudioBusSubsystem = MixerDevice->GetSubsystem<UAudioBusSubsystem>();
			check(AudioBusSubsystem);

			const Audio::FAudioBusKey AudioBusKey = Audio::FAudioBusKey(AudioBus->GetUniqueID());
			MixerSubmixPtr->RegisterAudioBus(AudioBusKey, AudioBusSubsystem->AddPatchInputForAudioBus(AudioBusKey, MixerDevice->GetNumOutputFrames(), AudioBus->GetNumChannels()));
		}
		else
		{
			UE_LOGF(LogAudioMixer, Error, "Submix not found in audio mixer.");
		}
	}
	else
	{
		UE_LOGF(LogAudioMixer, Error, "Audio mixer device not found.");
	}
}

void UAudioMixerBlueprintLibrary::UnregisterAudioBusFromSubmix(const UObject* WorldContextObject, USoundSubmix* SoundSubmix, UAudioBus* AudioBus)
{
	if (!SoundSubmix)
	{
		UE_LOGF(LogAudioMixer, Error, "UnregisterAudioBusToSubmix called with an invalid Submix.");
		return;
	}

	if (!AudioBus)
	{
		UE_LOGF(LogAudioMixer, Error, "UnregisterAudioBusToSubmix called with an invalid Audio Bus.");
		return;
	}

	if (!IsInAudioThread())
	{
		//Send this over to the audio thread, with the same settings
		FAudioThread::RunCommandOnAudioThread([WorldContextObject, SoundSubmix, AudioBus]()
		{
			UnregisterAudioBusFromSubmix(WorldContextObject, SoundSubmix, AudioBus);
		});

		return;
	}

	if (Audio::FMixerDevice* MixerDevice = FAudioDeviceManager::GetAudioMixerDeviceFromWorldContext(WorldContextObject))
	{
		Audio::FMixerSubmixPtr MixerSubmixPtr = MixerDevice->GetSubmixInstance(SoundSubmix).Pin();

		if (MixerSubmixPtr.IsValid())
		{
			const Audio::FAudioBusKey AudioBusKey(AudioBus->GetUniqueID());
			MixerSubmixPtr->UnregisterAudioBus(AudioBusKey);
		}
		else
		{
			UE_LOGF(LogAudioMixer, Error, "Submix not found in audio mixer.");
		}
	}
	else
	{
		UE_LOGF(LogAudioMixer, Error, "Audio mixer device not found.");
	}
}

void UAudioMixerBlueprintLibrary::GetAvailableAudioOutputDevices(const UObject* WorldContextObject, const FOnAudioOutputDevicesObtained& OnObtainDevicesEvent)
{
	if (!OnObtainDevicesEvent.IsBound())
	{
		return;
	}

	if (!IsInAudioThread())
	{
		//Send this over to the audio thread, with the same settings
		FAudioThread::RunCommandOnAudioThread([WorldContextObject, OnObtainDevicesEvent]()
		{
			GetAvailableAudioOutputDevices(WorldContextObject, OnObtainDevicesEvent);
		}); 

		return;
	}

	TArray<FAudioOutputDeviceInfo> OutputDeviceInfos; //The array of audio device info to return

	//Verifies its safe to access the audio mixer device interface
	Audio::FMixerDevice* AudioMixerDevice = FAudioDeviceManager::GetAudioMixerDeviceFromWorldContext(WorldContextObject);
	if (AudioMixerDevice)
	{
		if (Audio::IAudioMixerPlatformInterface* MixerPlatform = AudioMixerDevice->GetAudioMixerPlatform())
		{
			if (Audio::IAudioPlatformDeviceInfoCache* DeviceInfoCache = MixerPlatform->GetDeviceInfoCache())
			{
				TArray<Audio::FAudioPlatformDeviceInfo> AllDevices = DeviceInfoCache->GetAllActiveOutputDevices();
				Algo::Transform(AllDevices, OutputDeviceInfos, [](auto& i) -> FAudioOutputDeviceInfo { return { i }; });
			}
			else 
			{
				uint32 NumOutputDevices = 0;
				MixerPlatform->GetNumOutputDevices(NumOutputDevices);
				OutputDeviceInfos.Reserve(NumOutputDevices);
				FAudioOutputDeviceInfo CurrentOutputDevice = MixerPlatform->GetPlatformDeviceInfo();

				for (uint32 i = 0; i < NumOutputDevices; ++i)
				{
					Audio::FAudioPlatformDeviceInfo DeviceInfo;
					MixerPlatform->GetOutputDeviceInfo(i, DeviceInfo);

					FAudioOutputDeviceInfo NewInfo(DeviceInfo);
					NewInfo.bIsCurrentDevice = (NewInfo.DeviceId == CurrentOutputDevice.DeviceId);

					OutputDeviceInfos.Emplace(MoveTemp(NewInfo));
				}
			}
		}
	}

	//Send data through delegate on game thread
	FAudioThread::RunCommandOnGameThread([OnObtainDevicesEvent, OutputDeviceInfos]()
	{
		OnObtainDevicesEvent.ExecuteIfBound(OutputDeviceInfos);
	});
}

void UAudioMixerBlueprintLibrary::GetCurrentAudioOutputDeviceName(const UObject* WorldContextObject, const FOnMainAudioOutputDeviceObtained& OnObtainCurrentDeviceEvent)
{
	if (!OnObtainCurrentDeviceEvent.IsBound())
	{
		return;
	}

	if (!IsInAudioThread())
	{
		//Send this over to the audio thread, with the same settings
		FAudioThread::RunCommandOnAudioThread([WorldContextObject, OnObtainCurrentDeviceEvent]()
		{
			GetCurrentAudioOutputDeviceName(WorldContextObject, OnObtainCurrentDeviceEvent);
		});

		return;
	}

	TArray<FAudioOutputDeviceInfo> toReturn; //The array of audio device info to return

	//Verifies its safe to access the audio mixer device interface
	Audio::FMixerDevice* AudioMixerDevice = FAudioDeviceManager::GetAudioMixerDeviceFromWorldContext(WorldContextObject);
	if (AudioMixerDevice)
	{
		Audio::IAudioMixerPlatformInterface* MixerPlatform = AudioMixerDevice->GetAudioMixerPlatform();
		if (MixerPlatform)
		{
			//Send data through delegate on game thread
			FString CurrentDeviceName = MixerPlatform->GetCurrentDeviceName();
			FAudioThread::RunCommandOnGameThread([OnObtainCurrentDeviceEvent, CurrentDeviceName]()
			{
				OnObtainCurrentDeviceEvent.ExecuteIfBound(CurrentDeviceName);
			});
		}
	}

}

void UAudioMixerBlueprintLibrary::SwapAudioOutputDevice(const UObject* WorldContextObject, const FString& NewDeviceId, const FOnCompletedDeviceSwap& OnCompletedDeviceSwap)
{
	if (!OnCompletedDeviceSwap.IsBound())
	{
		return;
	}

	if (!IsInAudioThread())
	{
		//Send this over to the audio thread, with the same settings
		FAudioThread::RunCommandOnAudioThread([WorldContextObject, NewDeviceId, OnCompletedDeviceSwap]()
		{
			SwapAudioOutputDevice(WorldContextObject, NewDeviceId, OnCompletedDeviceSwap);
		});

		return;
	}

	//Verifies its safe to access the audio mixer device interface
	Audio::FMixerDevice* AudioMixerDevice = FAudioDeviceManager::GetAudioMixerDeviceFromWorldContext(WorldContextObject);
	if (AudioMixerDevice)
	{
		Audio::IAudioMixerPlatformInterface* MixerPlatform = AudioMixerDevice->GetAudioMixerPlatform();

		//Send message to swap device
		if (MixerPlatform)
		{
			bool result = MixerPlatform->RequestDeviceSwap(NewDeviceId, /*force*/ false, TEXT("UAudioMixerBlueprintLibrary::SwapAudioOutputDevice"));
			FAudioOutputDeviceInfo CurrentOutputDevice = MixerPlatform->GetPlatformDeviceInfo();

			//Send data through delegate on game thread
			FSwapAudioOutputResult SwapResult;
			SwapResult.CurrentDeviceId = CurrentOutputDevice.DeviceId;
			SwapResult.RequestedDeviceId = NewDeviceId;
			SwapResult.Result = result ? ESwapAudioOutputDeviceResultState::Success : ESwapAudioOutputDeviceResultState::Failure;
			FAudioThread::RunCommandOnGameThread([OnCompletedDeviceSwap, SwapResult]()
			{
				OnCompletedDeviceSwap.ExecuteIfBound(SwapResult);
			});
		}
	}
}
