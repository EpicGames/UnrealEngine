// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioMixerBuffer.h"
#include "AudioMixerDevice.h"
#include "AudioDecompress.h"
#include "Interfaces/IAudioFormat.h"
#include "AudioMixerSourceDecode.h"
#include "AudioMixerTrace.h"
#include "AudioStreaming.h"

namespace Audio
{

	FMixerBuffer::FMixerBuffer(FAudioDevice* InAudioDevice, USoundWave* InWave, EBufferType::Type InBufferType)
		: FSoundBuffer(InAudioDevice)
		, RealtimeAsyncHeaderParseTask(nullptr)
		, DecompressionState(nullptr)
		, BufferType(InBufferType)
		, SampleRate(InWave->GetSampleRateForCurrentPlatform())
		, NumFrames(0)
		, BitsPerSample(16) // TODO: support more bits, currently hard-coded to 16
		, Data(nullptr)
		, DataSize(0)
		, bIsRealTimeSourceReady(false)
		, bIsDynamicResource(false)
	{
		// Set the base-class NumChannels to wave's NumChannels
		NumChannels = InWave->NumChannels;
	}

	FMixerBuffer::~FMixerBuffer()
	{
		if (bAllocationInPermanentPool)
		{
			UE_LOGF(LogAudioMixer, Fatal, "Can't free resource '%ls' as it was allocated in permanent pool.", *ResourceName);
		}

		if (DecompressionState)
		{
			delete DecompressionState;
			DecompressionState = nullptr;
		}

		switch (BufferType)
		{
			case EBufferType::PCM:
			{
				if (Data)
				{
					FMemory::Free((void*)Data);
				}
			}
			break;

			case EBufferType::PCMPreview:
			{
				if (bIsDynamicResource && Data)
				{
					FMemory::Free((void*)Data);
				}
			}
			break;

			case EBufferType::PCMRealTime:
			case EBufferType::Streaming:
			// Buffers are freed as part of the ~FSoundSource
			break;

			case EBufferType::Invalid:
			// nothing
			break;
		}
	}

	int32 FMixerBuffer::GetSize()
	{
		switch (BufferType)
		{
			case EBufferType::PCM:
			case EBufferType::PCMPreview:
				return DataSize;

			case EBufferType::PCMRealTime:
				return (DecompressionState ? DecompressionState->GetSourceBufferSize() : 0) + (MONO_PCM_BUFFER_SIZE * NumChannels);

			case EBufferType::Streaming:
				return MONO_PCM_BUFFER_SIZE * NumChannels;

			case EBufferType::Invalid:
			break;
		}

		return 0;
	}

	int32 FMixerBuffer::GetCurrentChunkIndex() const
	{
		if (DecompressionState)
		{
			return DecompressionState->GetCurrentChunkIndex();
		}

		return 0;
	}

	int32 FMixerBuffer::GetCurrentChunkOffset() const
	{
		if (DecompressionState)
		{
			return DecompressionState->GetCurrentChunkOffset();
		}
		return 0;
	}

	bool FMixerBuffer::IsRealTimeSourceReady()
	{
		// If we have a realtime async header parse task, then we check if its done
		if (RealtimeAsyncHeaderParseTask)
		{
			bool bIsDone = RealtimeAsyncHeaderParseTask->IsDone();
			if (bIsDone)
			{
				delete RealtimeAsyncHeaderParseTask;
				RealtimeAsyncHeaderParseTask = nullptr;
			}
			return bIsDone;
		}

		// Otherwise, we weren't a real time decoding sound buffer (or we've already asked and it was ready)
		return true;
	}

	bool FMixerBuffer::ReadCompressedInfo(USoundWave* SoundWave)
	{
		if (!DecompressionState)
		{
			UE_LOGF(LogAudioMixer, Warning, "Attempting to read compressed info without a compression state instance for resource '%ls'", *ResourceName);
			return false;
		}

		AUDIO_MIXER_TRACE_CPUPROFILER_EVENT_SCOPE(FMixerBuffer::ReadCompressedInfo);

		FSoundQualityInfo QualityInfo;

		if (!SoundWave->GetResourceData() || !SoundWave->GetResourceSize())
		{
			UE_LOGF(LogAudioMixer, Warning, "Failed to read compressed info of '%ls' because there was no resource data or invalid resource size.", *ResourceName);
			return false;
		}

		if (DecompressionState->ReadCompressedInfo(SoundWave->GetResourceData(), SoundWave->GetResourceSize(), &QualityInfo))
		{
			NumFrames = QualityInfo.SampleDataSize / (QualityInfo.NumChannels * sizeof(int16));
			return true;
		}
		else
		{
			UE_LOGF(LogAudioMixer, Warning, "Failed to read compressed info of '%ls'.", *ResourceName);
		}
		return false;
	}

	void FMixerBuffer::Seek(const float SeekTime)
	{
		AUDIO_MIXER_TRACE_CPUPROFILER_EVENT_SCOPE(FMixerBuffer::Seek);

		if (ensure(DecompressionState))
		{
			DecompressionState->SeekToTime(SeekTime);
		}
	}

	FMixerBuffer* FMixerBuffer::Init(FAudioDevice* InAudioDevice, USoundWave* InWave, bool bForceRealtime)
	{
		// Can't create a buffer without any source data
		if (InWave == nullptr || InWave->NumChannels == 0)
		{
			return nullptr;
		}

		AUDIO_MIXER_TRACE_CPUPROFILER_EVENT_SCOPE(FMixerBuffer::Init);

#if WITH_EDITOR
		InWave->InvalidateSoundWaveIfNeccessary();
#endif // WITH_EDITOR

		FAudioDeviceManager* AudioDeviceManager = FAudioDevice::GetAudioDeviceManager();

		FMixerBuffer* Buffer = nullptr;

		EDecompressionType DecompressionType = InWave->DecompressionType;

		if (bForceRealtime  && DecompressionType != DTYPE_Setup && DecompressionType != DTYPE_Streaming && DecompressionType != DTYPE_Procedural)
		{
			DecompressionType = DTYPE_RealTime;
		}

		switch (DecompressionType)
		{
			case DTYPE_Setup:
			{
				// We've circumvented the level-load precache mechanism, precache synchronously // TODO: support async loading here?
				const bool bSynchronous = true;
				InAudioDevice->Precache(InWave, bSynchronous, false);
				check(InWave->DecompressionType != DTYPE_Setup);
				Buffer = Init(InAudioDevice, InWave, bForceRealtime);
			}
			break;
		
			case DTYPE_Procedural:
			{
				// Always create a new buffer for procedural or bus buffers
				Buffer = FMixerBuffer::CreateProceduralBuffer(InAudioDevice, InWave);
			}
			break;

			case DTYPE_RealTime:
			{
				// Always create a new buffer for real-time buffers
				Buffer = FMixerBuffer::CreateRealTimeBuffer(InAudioDevice, InWave);
			}
			break;
			
			case DTYPE_Streaming:
			{
				Buffer = FMixerBuffer::CreateStreamingBuffer(InAudioDevice, InWave);
			}
			break;

			case DTYPE_Invalid:
			default:
			{
				// Invalid will be set if the wave cannot be played.
			}
			break;
		}

		return Buffer;
	}

	FMixerBuffer* FMixerBuffer::CreatePreviewBuffer(FAudioDevice* AudioDevice, USoundWave* InWave)
	{
		// Create a new buffer
		FMixerBuffer* Buffer = new FMixerBuffer(AudioDevice, InWave, EBufferType::PCMPreview);

		Buffer->bIsDynamicResource = InWave->bDynamicResource;
		return Buffer;
	}

	FMixerBuffer* FMixerBuffer::CreateProceduralBuffer(FAudioDevice* AudioDevice, USoundWave* InWave)
	{
		FMixerBuffer* Buffer = new FMixerBuffer(AudioDevice, InWave, EBufferType::PCMRealTime);

		// No tracking of this resource needed
		Buffer->ResourceID = 0;
		InWave->ResourceID = 0;

		return Buffer;
	}

	FMixerBuffer* FMixerBuffer::CreateNativeBuffer(FAudioDevice* AudioDevice, USoundWave* InWave)
	{
		check(InWave->GetPrecacheState() == ESoundWavePrecacheState::Done);

		FMixerBuffer* Buffer = new FMixerBuffer(AudioDevice, InWave, EBufferType::PCM);
		return Buffer;
	}

	FMixerBuffer* FMixerBuffer::CreateStreamingBuffer(FAudioDevice* AudioDevice, USoundWave* InWave)
	{
		if (!InWave)
		{
			return nullptr;
		}
		
		// Ignore attempts to create if this wave has been flagged as containing errors
		if (InWave->HasError())
		{
			UE_LOGF(LogAudioMixer, VeryVerbose, "FMixerBuffer::CreateStreamingBuffer, ignoring '%ls' as it contains previously seen errors", *InWave->GetName() );
			return nullptr;
		}

		AUDIO_MIXER_TRACE_CPUPROFILER_EVENT_SCOPE(FMixerBuffer::CreateStreamingBuffer);

		FMixerBuffer* Buffer = new FMixerBuffer(AudioDevice, InWave, EBufferType::Streaming);

		FSoundQualityInfo QualityInfo = { 0 };
		
		Buffer->DecompressionState = IAudioInfoFactoryRegistry::Get().Create(InWave->GetRuntimeFormat(), InWave->GetFName());

		// Get the header information of our compressed format
		if (Buffer->DecompressionState && Buffer->DecompressionState->StreamCompressedInfo(InWave, &QualityInfo))
		{
			// Refresh the wave data
			if (QualityInfo.SampleRate != 0)
			{
				InWave->SetSampleRate(QualityInfo.SampleRate, /* bFromDecoders */ true);
			}
			if (QualityInfo.NumChannels != 0)
			{
				ensure(QualityInfo.NumChannels <= AUDIO_MIXER_MAX_OUTPUT_CHANNELS);
				InWave->NumChannels = QualityInfo.NumChannels;
			}
			if (QualityInfo.SampleDataSize != 0)
			{
				if (QualityInfo.NumChannels > 0)
				{
					// Update the NumFrames *if* the decoders have returned how big the sample data is.
					// Some decoder implementations don't do this because of how they organise the bit-stream for streaming.
					const int64 NumFrames = QualityInfo.SampleDataSize / sizeof(int16) / QualityInfo.NumChannels;
					InWave->SetNumFrames(IntCastChecked<uint32>(NumFrames));
				}
				InWave->RawPCMDataSize = QualityInfo.SampleDataSize;
			}
			if (QualityInfo.Duration != 0.0f)
			{
				ensure(QualityInfo.Duration > 0.0f);
				InWave->Duration = QualityInfo.Duration;
			}
		}
		else
		{
			// Failed to stream in compressed info, so mark the wave as having an error.
			if (Buffer->DecompressionState)
			{
				InWave->SetError(TEXT("ICompressedAudioInfo::StreamCompressedInfo failed"));
			}

			// When set to seekable streaming, missing the first chunk is possible and
			// does not signify any issue with the asset itself, so don't mark it as invalid.
			if (InWave && !InWave->IsSeekable())
			{
				UE_LOGF(LogAudioMixer, Warning,
					"FMixerBuffer::CreateStreamingBuffer failed to StreamCompressedInfo on SoundWave '%ls'.  Invalidating wave resource data (asset now requires re-cook).",
					*InWave->GetName());

				InWave->DecompressionType = DTYPE_Invalid;
				InWave->NumChannels = 0;
				InWave->RemoveAudioResource();
			}

			delete Buffer;
			Buffer = nullptr;
		}

		return Buffer;
	}

	FMixerBuffer* FMixerBuffer::CreateRealTimeBuffer(FAudioDevice* AudioDevice, USoundWave* InWave)
	{
		check(AudioDevice);
		check(InWave);
		check(InWave->GetPrecacheState() == ESoundWavePrecacheState::Done);

		// Create a new buffer for real-time sounds
		FMixerBuffer* Buffer = new FMixerBuffer(AudioDevice, InWave, EBufferType::PCMRealTime);

		FName FormatName = InWave->GetRuntimeFormat();
		if (InWave->GetResourceData() == nullptr)
		{
			InWave->InitAudioResource(FormatName);
		}

		Buffer->DecompressionState = IAudioInfoFactoryRegistry::Get().Create(FormatName);
		check(Buffer->DecompressionState);

		if (Buffer->DecompressionState)
		{
			FHeaderParseAudioTaskData NewTaskData;
			NewTaskData.MixerBuffer = Buffer;
			NewTaskData.SoundWave = InWave;

			check(Buffer->RealtimeAsyncHeaderParseTask == nullptr);
			Buffer->RealtimeAsyncHeaderParseTask = CreateAudioTask(AudioDevice->DeviceID, NewTaskData);

			Buffer->NumChannels = InWave->NumChannels;
		}
		else
		{
			InWave->DecompressionType = DTYPE_Invalid;
			InWave->NumChannels = 0;

			InWave->RemoveAudioResource();

			delete Buffer;
			Buffer = nullptr;
		}

		return Buffer;
	}

	EBufferType::Type FMixerBuffer::GetType() const
	{
		return BufferType;
	}

	bool FMixerBuffer::IsRealTimeBuffer() const
	{
		return BufferType == EBufferType::PCMRealTime || BufferType == EBufferType::Streaming;
	}

	ICompressedAudioInfo* FMixerBuffer::GetDecompressionState(bool bTakesOwnership)
	{
		ICompressedAudioInfo* Output = DecompressionState;
		if (bTakesOwnership)
		{
			DecompressionState = nullptr;
		}
		return Output;
	}

	void FMixerBuffer::GetPCMData(uint8** OutData, uint32* OutDataSize)
	{
		*OutData = Data;
		*OutDataSize = DataSize;
	}

	void FMixerBuffer::EnsureHeaderParseTaskFinished()
	{
		if (RealtimeAsyncHeaderParseTask)
		{
			RealtimeAsyncHeaderParseTask->EnsureCompletion();
			delete RealtimeAsyncHeaderParseTask;
			RealtimeAsyncHeaderParseTask = nullptr;
		}
	}
}
