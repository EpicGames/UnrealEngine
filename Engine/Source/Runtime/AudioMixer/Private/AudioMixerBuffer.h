// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AudioMixer.h"
#include "Audio.h"

namespace Audio
{
	namespace EBufferType
	{
		enum Type
		{
			PCM,
			PCMPreview,
			PCMRealTime,
			Streaming,
			Invalid,
		};
	}

	class FMixerDevice;
	class FMixerBuffer;
	class IAudioTask;

	class FMixerBuffer : public FSoundBuffer
	{
	public:
		FMixerBuffer(FAudioDevice* InAudioDevice, USoundWave* InWave, EBufferType::Type InBufferType);
		~FMixerBuffer();

		//~ Begin FSoundBuffer Interface
		int32 GetSize() override;
		int32 GetCurrentChunkIndex() const override;
		int32 GetCurrentChunkOffset() const override;
		bool IsRealTimeSourceReady() override;
		bool ReadCompressedInfo(USoundWave* SoundWave) override;
		void Seek(const float SeekTime) override;
		//~ End FSoundBuffer Interface

		static FMixerBuffer* Init(FAudioDevice* AudioDevice, USoundWave* InWave, bool bForceRealtime);
		static FMixerBuffer* CreatePreviewBuffer(FAudioDevice* AudioDevice, USoundWave* InWave);
		static FMixerBuffer* CreateProceduralBuffer(FAudioDevice* AudioDevice, USoundWave* InWave);
		static FMixerBuffer* CreateNativeBuffer(FAudioDevice* AudioDevice, USoundWave* InWave);
		static FMixerBuffer* CreateStreamingBuffer(FAudioDevice* AudioDevice, USoundWave* InWave);
		static FMixerBuffer* CreateRealTimeBuffer(FAudioDevice* AudioDevice, USoundWave* InWave);

		/** Returns the buffer's format */
		EBufferType::Type GetType() const;
		bool IsRealTimeBuffer() const;

		/** Retrieve the compressed audio info pointer. This transfers ownership of the decompression state. */
		ICompressedAudioInfo* GetDecompressionState(bool bTakesOwnership = false);

		/** Returns the contained raw PCM data and data size */
		void GetPCMData(uint8** OutData, uint32* OutDataSize);

		void EnsureHeaderParseTaskFinished();

		float GetSampleRate() const { return SampleRate; }
		int32 GetNumChannels() const { return NumChannels; }
		uint32 GetNumFrames() const { return NumFrames; }
		void InitSampleRate(const float InSampleRate) { SampleRate = InSampleRate; }
		void InitNumFrames(const uint32 InNumFrames) { NumFrames = InNumFrames; }

	private:

		/** Async task for parsing real-time decompressed compressed info headers. */
		IAudioTask* RealtimeAsyncHeaderParseTask;

		/** Wrapper to handle the decompression of audio codecs. */
		ICompressedAudioInfo* DecompressionState;

		/** Format of the sound referenced by this buffer */
		EBufferType::Type BufferType;

		/** Sample rate of the audio buffer. */
		int32 SampleRate;

		/** Number of frames of the audio. */
		uint32 NumFrames;

		/** Number of bits per sample. */
		int16 BitsPerSample;

		/** Ptr to raw PCM data. */
		uint8* Data;

		/** The raw PCM data size. */
		uint32 DataSize;

		/** Bool indicating the that the real-time source is ready for real-time decoding. */
		FThreadSafeBool bIsRealTimeSourceReady;

		/** Set to true when the PCM data should be freed when the buffer is destroyed */
		bool bIsDynamicResource;
	};
}

inline const TCHAR* LexToString(const Audio::EBufferType::Type In)
{
#define CASE_TO_STRING(X) case X: return TEXT(#X);
	using namespace Audio::EBufferType;
	switch(In)
	{
		CASE_TO_STRING(Invalid);
		CASE_TO_STRING(PCMPreview)
		CASE_TO_STRING(PCMRealTime)
		CASE_TO_STRING(Streaming)
		default: return TEXT("Unknown");
	}
#undef CASE_TO_STRING
}