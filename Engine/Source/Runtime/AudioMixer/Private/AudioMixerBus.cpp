// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioMixerBus.h"

#include "Algo/ForEach.h"
#include "AudioMixerCVars.h"
#include "AudioMixerSourceManager.h"
#include "AudioRenderScheduler.h"
#include "DSP/AlignedBuffer.h"
#include "DSP/FloatArrayMath.h"

namespace Audio
{
	FMixerAudioBus::FMixerAudioBus(FMixerSourceManager* InSourceManager, bool bInIsAutomatic, int32 InNumChannels, const FAudioBusKey InBusKey)
		: CurrentBufferIndex(1)
		, NumChannels(InNumChannels)
		, NumFrames(InSourceManager->GetNumOutputFrames())
		, SourceManager(InSourceManager)
		, BusKey(InBusKey)
		, bIsAutomatic(bInIsAutomatic)
	{
		SetNumOutputChannels(NumChannels);

		if (SourceManager->UseRenderScheduler())
		{
			SourceManager->GetRenderScheduler().AddStep(FAudioRenderStepId::FromAudioBusKey(BusKey), this);
		}
	}

	FMixerAudioBus::~FMixerAudioBus()
	{
		if (SourceManager->UseRenderScheduler())
		{
			SourceManager->GetRenderScheduler().RemoveStep(FAudioRenderStepId::FromAudioBusKey(BusKey));
		}
	}

	void FMixerAudioBus::SetNumOutputChannels(int32 InNumOutputChannels)
	{
		NumChannels = InNumOutputChannels;
		const int32 NumSamples = NumChannels * NumFrames;
		for (int32 i = 0; i < 2; ++i)
		{
			MixedSourceData[i].Reset();
			MixedSourceData[i].AddZeroed(NumSamples);
		}
	}

	void FMixerAudioBus::Update()
	{
		// When using the scheduler we flip buffers at the beginning of MixBuffer() instead.
		check(!SourceManager->UseRenderScheduler());
		CurrentBufferIndex = 1 - CurrentBufferIndex;
	}

	void FMixerAudioBus::AddInstanceId(const int32 InSourceId, const uint32 WaveInstancePlayOrder, int32 InNumOutputChannels)
	{
		InstanceIds.Add(InSourceId);
		if (SourceManager->UseRenderScheduler())
		{
			// Make sure this bus gets mixed before the source from it gets rendered
			const FAudioRenderStepId SourceStepId = FAudioRenderStepId::FromWaveInstancePlayOrder(WaveInstancePlayOrder);
			const FAudioRenderStepId BusStepId = FAudioRenderStepId::FromAudioBusKey(BusKey);
			SourceManager->GetRenderScheduler().AddDependency(BusStepId, SourceStepId);
		}
	}

	bool FMixerAudioBus::RemoveInstanceId(const int32 InSourceId, const uint32 WaveInstancePlayOrder)
	{
		if (InstanceIds.Remove(InSourceId) > 0 && SourceManager->UseRenderScheduler())
		{
			const FAudioRenderStepId SourceStepId = FAudioRenderStepId::FromWaveInstancePlayOrder(WaveInstancePlayOrder);
			const FAudioRenderStepId BusStepId = FAudioRenderStepId::FromAudioBusKey(BusKey);
			SourceManager->GetRenderScheduler().RemoveDependency(BusStepId, SourceStepId);
		}

		// Return true if there is no more instances or sends
		return bIsAutomatic && !InstanceIds.Num() && !AudioBusSends[(int32)EBusSendType::PreEffect].Num() && !AudioBusSends[(int32)EBusSendType::PostEffect].Num() && !AudioBusSends[(int32)EBusSendType::PostAttenuation].Num();
	}

	void FMixerAudioBus::AddSend(EBusSendType BusSendType, const FAudioBusSend& InAudioBusSend)
	{
		// Make sure we don't have duplicates in the bus sends
		for (FAudioBusSend& BusSend : AudioBusSends[(int32)BusSendType])
		{
			// If it's already added, update send level and filter state
			if (BusSend.SourceId == InAudioBusSend.SourceId)
			{
				BusSend.SendLevel = InAudioBusSend.SendLevel;
					if (InAudioBusSend.Filter.bEnableLPF)
					{
						BusSend.Filter.LpfCutoff = InAudioBusSend.Filter.LpfCutoff;
						// If LPF was previously disabled, the Lpf object was never Init()'d; copy it in.
						if (!BusSend.Filter.bEnableLPF)
						{
							BusSend.Lpf = InAudioBusSend.Lpf;
						}
					}
					if (InAudioBusSend.Filter.bEnableHPF)
					{
						BusSend.Filter.HpfCutoff = InAudioBusSend.Filter.HpfCutoff;
						// If HPF was previously disabled, the Hpf object was never Init()'d; copy it in.
						if (!BusSend.Filter.bEnableHPF)
						{
							BusSend.Hpf = InAudioBusSend.Hpf;
						}
					}
					BusSend.Filter.bEnableLPF = InAudioBusSend.Filter.bEnableLPF;
					BusSend.Filter.bEnableHPF = InAudioBusSend.Filter.bEnableHPF;
				return;
			}
		}

		// It's a new source id so just add it
		AudioBusSends[(int32)BusSendType].Add(InAudioBusSend);

		if (SourceManager->UseRenderScheduler())
		{
			// Make sure the source sending to the bus gets rendered before the bus is mixed
			const FAudioRenderStepId SourceStepId = FAudioRenderStepId::FromWaveInstancePlayOrder(InAudioBusSend.WaveInstancePlayOrder);
			const FAudioRenderStepId BusStepId = FAudioRenderStepId::FromAudioBusKey(BusKey);
			SourceManager->GetRenderScheduler().AddDependency(SourceStepId, BusStepId);
		}
	}

	bool FMixerAudioBus::RemoveSend(EBusSendType BusSendType, const int32 InSourceId)
	{
		TArray<FAudioBusSend>& Sends = AudioBusSends[(int32)BusSendType];

		for (int32 i = Sends.Num() - 1; i >= 0; --i)
		{
			// Remove this source id's send
			if (Sends[i].SourceId == InSourceId)
			{
				if (SourceManager->UseRenderScheduler())
				{
					const FAudioRenderStepId SourceStepId = FAudioRenderStepId::FromWaveInstancePlayOrder(Sends[i].WaveInstancePlayOrder);
					const FAudioRenderStepId BusStepId = FAudioRenderStepId::FromAudioBusKey(BusKey);
					SourceManager->GetRenderScheduler().RemoveDependency(SourceStepId, BusStepId);
				}

				Sends.RemoveAtSwap(i, EAllowShrinking::No);

				// There will only be one entry
				break;
			}
		}

		// Return true if there is no more instances or sends and this is an automatic audio bus
		return bIsAutomatic && !InstanceIds.Num() && !AudioBusSends[(int32)EBusSendType::PreEffect].Num() && !AudioBusSends[(int32)EBusSendType::PostEffect].Num();
	}

	void FMixerAudioBus::MixBuffer()
	{
		// Mix the patch mixer's inputs into the source data
		const int32 NumSamples = NumFrames * NumChannels;
		const int32 NumOutputFrames = SourceManager->GetNumOutputFrames();

		if (SourceManager->UseRenderScheduler())
		{
			CurrentBufferIndex = 1 - CurrentBufferIndex;
		}

		FAlignedFloatBuffer& MixBuffer = MixedSourceData[CurrentBufferIndex];
		float* BusDataBufferPtr = MixBuffer.GetData();

		PatchMixer.PopAudio(BusDataBufferPtr, NumSamples, false);

		// FilterScratchBuffer is a member so its capacity persists across render callbacks,
		// avoiding per-frame heap reallocation once the working size is established.
		// Reset element count; actual size is set on demand inside the send loop.
		FilterScratchBuffer.Reset();

		for (int32 BusSendType = 0; BusSendType < (int32)EBusSendType::Count; ++BusSendType)
		{
			// Loop through the send list for this bus
				for (FAudioBusSend& AudioBusSend : AudioBusSends[BusSendType])
			{
				const float* SourceBufferPtr = nullptr;

				// If the audio source mixing to this audio bus is itself a source bus, we need to use the previous renderer buffer to avoid infinite recursion.
				// With the render scheduler we don't need to treat source buses any differently from other sources -- the source bus should already have been
				//  rendered.
				if (!SourceManager->UseRenderScheduler() && SourceManager->IsSourceBus(AudioBusSend.SourceId))
				{
					SourceBufferPtr = SourceManager->GetPreviousSourceBusBuffer(AudioBusSend.SourceId);
				}
				// If the source mixing into this is not itself a bus, then simply mix the pre/post-attenuation audio of the source into the bus
				// The source will have already computed its buffers for this frame
				else if (BusSendType == (int32)EBusSendType::PostEffect)
				{
					SourceBufferPtr = SourceManager->GetPreDistanceAttenuationBuffer(AudioBusSend.SourceId);
				}
				else if (BusSendType == (int32)EBusSendType::PreEffect)
				{
					SourceBufferPtr = SourceManager->GetPreEffectBuffer(AudioBusSend.SourceId);
				}
				else
				{
					SourceBufferPtr = SourceManager->GetPostDistanceAttenuationBuffer(AudioBusSend.SourceId);
				}

				// It's possible we may not have a source buffer ptr here if the sound is not playing
				if (SourceBufferPtr)
				{
					const int32 NumSourceChannels = SourceManager->GetNumChannels(AudioBusSend.SourceId);
					const int32 NumSourceSamples = NumSourceChannels * NumOutputFrames;

					// Up-mix or down-mix if source channels differ from bus channels
					if (NumSourceChannels != NumChannels)
					{
							const float* FilteredSourcePtr = SourceBufferPtr;

							// Apply per-send filters to the source buffer before downmixing.
							// When both LPF and HPF are active, the LPF writes into FilterScratchBuffer and
							// the HPF processes it via ProcessBufferInPlace -- avoiding aliased pointers into
							// ProcessAudioBuffer (whose RESTRICT arguments must not alias).
							if (AudioBusSend.Filter.IsFiltering())
							{
								const bool bBypassLPF = !AudioBusSend.Filter.bEnableLPF;
								const bool bBypassHPF = !AudioBusSend.Filter.bEnableHPF;

								FilterScratchBuffer.SetNumUninitialized(NumSourceSamples);

								if (!bBypassLPF)
								{
									AudioBusSend.Lpf.StartFrequencyInterpolation(AudioBusSend.Filter.LpfCutoff, NumOutputFrames);
									AudioBusSend.Lpf.ProcessAudioBuffer(SourceBufferPtr, FilterScratchBuffer.GetData(), NumSourceSamples);
									AudioBusSend.Lpf.StopFrequencyInterpolation();
									FilteredSourcePtr = FilterScratchBuffer.GetData();
								}

								if (!bBypassHPF)
								{
									AudioBusSend.Hpf.StartFrequencyInterpolation(AudioBusSend.Filter.HpfCutoff, NumOutputFrames);
									if (FilteredSourcePtr == FilterScratchBuffer.GetData())
									{
										// LPF already wrote into scratch; process HPF in-place using the dedicated API.
										AudioBusSend.Hpf.ProcessBufferInPlace(FilterScratchBuffer.GetData(), NumSourceSamples);
									}
									else
									{
										AudioBusSend.Hpf.ProcessAudioBuffer(FilteredSourcePtr, FilterScratchBuffer.GetData(), NumSourceSamples);
									}
									AudioBusSend.Hpf.StopFrequencyInterpolation();
									FilteredSourcePtr = FilterScratchBuffer.GetData();
								}
							}

						FAlignedFloatBuffer ChannelMap;
						SourceManager->Get2DChannelMap(AudioBusSend.SourceId, NumChannels, ChannelMap);
						Algo::ForEach(ChannelMap, [SendLevel = AudioBusSend.SendLevel](float& ChannelValue) { ChannelValue *= SendLevel; });
							DownmixAndSumIntoBuffer(NumSourceChannels, NumChannels, FilteredSourcePtr, BusDataBufferPtr, NumOutputFrames, ChannelMap.GetData());
					}
					else
					{
						TArrayView<const float> SourceBufferView(SourceBufferPtr, NumOutputFrames * NumChannels);
						TArrayView<float> BusDataBufferView(BusDataBufferPtr, NumOutputFrames * NumChannels);

						if (AudioBusSend.Filter.IsFiltering())
						{
							ArrayFilterAndMixIn(SourceBufferView, BusDataBufferView, AudioBusSend.SendLevel, NumChannels, AudioBusSend.Filter.bEnableLPF, AudioBusSend.Filter.LpfCutoff, AudioBusSend.Filter.bEnableHPF, AudioBusSend.Filter.HpfCutoff, AudioBusSend.Lpf, AudioBusSend.Hpf, FilterScratchBuffer);
						}
						else
						{
							ArrayMixIn(SourceBufferView, BusDataBufferView, AudioBusSend.SendLevel);
						}
					}
				}
			}
		}

		// Send the mix to the patch splitter's outputs
		PatchSplitter.PushAudio(BusDataBufferPtr, NumSamples);

#if UE_AUDIO_PROFILERTRACE_ENABLED
		if (bIsEnvelopeFollowing)
		{
			ProcessEnvelopeFollower(BusDataBufferPtr);
		}
#endif // UE_AUDIO_PROFILERTRACE_ENABLED
	}

	void FMixerAudioBus::CopyCurrentBuffer(Audio::FAlignedFloatBuffer& InChannelMap, int32 InNumOutputChannels, FAlignedFloatBuffer& OutBuffer, int32 NumOutputFrames) const
	{
		check(NumChannels != InNumOutputChannels);
		DownmixAndSumIntoBuffer(NumChannels, InNumOutputChannels, MixedSourceData[CurrentBufferIndex], OutBuffer, InChannelMap.GetData());
	}

	void FMixerAudioBus::CopyCurrentBuffer(int32 InNumOutputChannels, FAlignedFloatBuffer& OutBuffer, int32 NumOutputFrames) const
	{
		const float* RESTRICT CurrentBuffer = GetCurrentBusBuffer();

		check(NumChannels == InNumOutputChannels);

		FMemory::Memcpy(OutBuffer.GetData(), CurrentBuffer, sizeof(float) * NumOutputFrames * InNumOutputChannels);
	}

	const float* FMixerAudioBus::GetCurrentBusBuffer() const
	{
		return MixedSourceData[CurrentBufferIndex].GetData();
	}

	const float* FMixerAudioBus::GetPreviousBusBuffer() const
	{
		return MixedSourceData[1 - CurrentBufferIndex].GetData();
	}

	void FMixerAudioBus::AddNewPatchOutput(const FPatchOutputStrongPtr& InPatchOutputStrongPtr)
	{
		PatchSplitter.AddNewPatch(InPatchOutputStrongPtr);
	}

	void FMixerAudioBus::AddNewPatchInput(const FPatchInput& InPatchInput)
	{
		return PatchMixer.AddNewInput(InPatchInput);
	}

	void FMixerAudioBus::RemovePatchInput(const FPatchInput& PatchInput)
	{
		return PatchMixer.RemovePatch(PatchInput);
	}


	void FMixerAudioBus::DoRenderStep()
	{
		MixBuffer();
	}

	const TCHAR* FMixerAudioBus::GetRenderStepName()
	{
#if ENABLE_AUDIO_DEBUG
		return *BusName;
#else
		return TEXT("FMixerAudioBus mixing");
#endif
	}

#if UE_AUDIO_PROFILERTRACE_ENABLED
	bool FMixerAudioBus::StartEnvelopeFollower(const float InAttackTime, const float InReleaseTime, const float InSampleRate)
	{
		++EnvelopeFollowerRequestsRefCount;

		if (!bIsEnvelopeFollowing)
		{
			FEnvelopeFollowerInitParams EnvelopeFollowerInitParams;

			EnvelopeFollowerInitParams.SampleRate = InSampleRate;
			EnvelopeFollowerInitParams.NumChannels = NumChannels;
			EnvelopeFollowerInitParams.AttackTimeMsec = InAttackTime;
			EnvelopeFollowerInitParams.ReleaseTimeMsec = InReleaseTime;

			EnvelopeFollower.Init(EnvelopeFollowerInitParams);

			// Zero out any previous envelope values which may have been in the array before starting up
			for (int32 ChannelIndex = 0; ChannelIndex < AUDIO_MIXER_MAX_OUTPUT_CHANNELS; ++ChannelIndex)
			{
				EnvelopeValues[ChannelIndex] = 0.0f;
			}

			bIsEnvelopeFollowing = true;
			return true;
		}

		return false;
	}

	bool FMixerAudioBus::StopEnvelopeFollower()
	{
		if (!bIsEnvelopeFollowing)
		{
			return false;
		}

		--EnvelopeFollowerRequestsRefCount;

		if (EnvelopeFollowerRequestsRefCount <= 0)
		{
			EnvelopeFollowerRequestsRefCount = 0;
			bIsEnvelopeFollowing = false;
			return true;
		}

		return false;
	}

	void FMixerAudioBus::ProcessEnvelopeFollower(const float* InBuffer)
	{
		FMemory::Memset(EnvelopeValues, sizeof(float) * AUDIO_MIXER_MAX_OUTPUT_CHANNELS);

		if (NumChannels > 0)
		{
			if (EnvelopeFollower.GetNumChannels() != NumChannels)
			{
				EnvelopeFollower.SetNumChannels(NumChannels);
			}

			EnvelopeFollower.ProcessAudio(InBuffer, NumFrames);

			const TArray<float>& EnvValues = EnvelopeFollower.GetEnvelopeValues();

			check(EnvValues.Num() == NumChannels);

			FMemory::Memcpy(EnvelopeValues, EnvValues.GetData(), sizeof(float) * NumChannels);

			Audio::ArrayClampInPlace(MakeArrayView(EnvelopeValues, NumChannels), 0.0f, 1.0f);
		}

		EnvelopeNumChannels = NumChannels;
	}
#endif // UE_AUDIO_PROFILERTRACE_ENABLED
}
