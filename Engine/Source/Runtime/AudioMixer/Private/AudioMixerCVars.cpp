// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioMixerCVars.h"

#include "AudioMixerDevice.h"
#include "AudioRenderScheduler.h"
#include "Engine/Engine.h"
#include "HAL/IConsoleManager.h"

namespace AudioMixerCVars
{
	int32 UseRenderScheduler = 1;
	FAutoConsoleVariableRef CVarUseAudioRenderScheduler(
		TEXT("au.UseRenderScheduler"),
		UseRenderScheduler,
		TEXT("Use audio rendering scheduler to handle render dependencies. Should be set in an ini file or on the command line, as changing this after audio mixer initialization has no effect.\n")
		TEXT("0: Disabled, 1: Enabled"),
		ECVF_SaveForNextBoot);

#if ENABLE_AUDIO_DEBUG
	FAutoConsoleCommand AudioRenderSchedulerDumpCmd(
		TEXT("au.RenderScheduler.dump"),
		TEXT("Dump the internal state of audio render scheduler to the log."),
		FConsoleCommandDelegate::CreateStatic(
			[]()
			{
				using namespace Audio;
				FAudioDevice* AudioDevice = GEngine != nullptr ? GEngine->GetMainAudioDeviceRaw() : nullptr;
				if (AudioDevice != nullptr)
				{
					FMixerDevice* MixerDevice = static_cast<FMixerDevice*>(AudioDevice);
					MixerDevice->GetRenderScheduler().DumpSteps();
				}
			}
		)
	);
#endif
}
