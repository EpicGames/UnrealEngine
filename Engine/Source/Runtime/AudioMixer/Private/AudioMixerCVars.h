// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

namespace AudioMixerCVars
{
	// This is used to decide whether or not to create a render scheduler when the mixer device is created.
	//  Most code should test FMixerDevice::UseRenderScheduler() instead of using this.
	extern int32 UseRenderScheduler;
}
