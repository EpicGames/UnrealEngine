// Copyright Epic Games, Inc. All Rights Reserved.

#include "Audio.h"
#include "AudioMixer.h"
#include "AudioMixerDevice.h"
#include "DSP/ChannelMap.h"
#include "Misc/ConfigCacheIni.h"

namespace Audio
{
	// Make a channel map cache
	static TArray<TArray<float>> ChannelMapCache;
	static TArray<TArray<float>> VorbisChannelMapCache;

	int32 FMixerDevice::GetChannelMapCacheId(const int32 NumSourceChannels, const int32 NumOutputChannels, const bool bIsCenterChannelOnly)
	{
		if (ensure(NumSourceChannels > 0) && 
			ensure(NumOutputChannels > 0) &&
			ensure(NumSourceChannels <= AUDIO_MIXER_MAX_OUTPUT_CHANNELS) &&
			ensure(NumOutputChannels <= AUDIO_MIXER_MAX_OUTPUT_CHANNELS) )
		{
			int32 Index = (NumSourceChannels - 1) + AUDIO_MIXER_MAX_OUTPUT_CHANNELS * (NumOutputChannels - 1);
			if (bIsCenterChannelOnly)
			{
				Index += AUDIO_MIXER_MAX_OUTPUT_CHANNELS * AUDIO_MIXER_MAX_OUTPUT_CHANNELS;
			}			
			return Index;
		}			   		 
		return 0;	
	}

	void FMixerDevice::Get2DChannelMap(bool bIsVorbis, const int32 NumSourceChannels, const bool bIsCenterChannelOnly, Audio::FAlignedFloatBuffer& OutChannelMap) const
	{
		Get2DChannelMap(bIsVorbis, NumSourceChannels, PlatformInfo.NumChannels, bIsCenterChannelOnly, OutChannelMap);
	}

	void FMixerDevice::Get2DChannelMap(bool bIsVorbis, const int32 NumSourceChannels, const int32 NumOutputChannels, const bool bIsCenterChannelOnly, Audio::FAlignedFloatBuffer& OutChannelMap)
	{
		if (NumSourceChannels <= 0 ||
			NumOutputChannels <= 0 ||
			NumSourceChannels > AUDIO_MIXER_MAX_OUTPUT_CHANNELS || 
			NumOutputChannels > AUDIO_MIXER_MAX_OUTPUT_CHANNELS
			)
		{
			// Return a zero'd channel map buffer in the case of an unsupported channel configuration
			OutChannelMap.AddZeroed(AUDIO_MIXER_MAX_OUTPUT_CHANNELS * AUDIO_MIXER_MAX_OUTPUT_CHANNELS);

#if !NO_LOGGING			
			// Anti-Spam warning.
			static uint64 TimeOfLastLogMsgInCycles = 0;
			constexpr double MinTimeBetweenWarningsInMs = 5000.f; // 5 Secs.
			double ElapsedTimeInMs = FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64() - TimeOfLastLogMsgInCycles);

			if (ElapsedTimeInMs > MinTimeBetweenWarningsInMs)
			{
				TimeOfLastLogMsgInCycles = FPlatformTime::Cycles64();
				UE_LOGF(LogAudioMixer, Warning, "Unsupported source channel (%d) count or output channels (%d)", NumSourceChannels, NumOutputChannels);
			}	
#endif //!NO_LOGGING
			
			// Bail.
			return;
		}

		// 5.1 Vorbis files have a non-standard channel order so pick a channel map from the 5.1 vorbis channel maps based on the output channels
		if (bIsVorbis && NumSourceChannels == 6)
		{
			OutChannelMap = VorbisChannelMapCache[NumOutputChannels - 1];
		}
		else
		{
			const int32 CacheID = GetChannelMapCacheId(NumSourceChannels, NumOutputChannels, bIsCenterChannelOnly);
			OutChannelMap = ChannelMapCache[CacheID];
		}
	}

	void FMixerDevice::CacheChannelMap(const int32 NumSourceChannels, const int32 NumOutputChannels, const bool bIsCenterChannelOnly)
	{
		if (NumSourceChannels <= 0 ||
			NumOutputChannels <= 0 ||
			NumSourceChannels > AUDIO_MIXER_MAX_OUTPUT_CHANNELS || 
			NumOutputChannels > AUDIO_MIXER_MAX_OUTPUT_CHANNELS
			)
		{
			return;
		}
		// Generate the unique cache ID for the channel count configuration
		const int32 CacheID = GetChannelMapCacheId(NumSourceChannels, NumOutputChannels, bIsCenterChannelOnly);

		// Setup parameters for generating channel maps. 
		FChannelMapParams Params;
		Params.NumInputChannels = NumSourceChannels;
		Params.NumOutputChannels = NumOutputChannels;
		Params.Order = EChannelMapOrder::OutputMajorOrder; // Downmix code expects OutputMajorOrder
		Params.bIsCenterChannelOnly = bIsCenterChannelOnly;

		switch (MonoChannelUpmixMethod)
		{
			case EMonoChannelUpmixMethod::Linear:
				Params.MonoUpmixMethod = EChannelMapMonoUpmixMethod::Linear;
				break;

			case EMonoChannelUpmixMethod::EqualPower:
				Params.MonoUpmixMethod = EChannelMapMonoUpmixMethod::EqualPower;
				break;

			case EMonoChannelUpmixMethod::FullVolume:
				Params.MonoUpmixMethod = EChannelMapMonoUpmixMethod::FullVolume;
				break;

			default:
				Params.MonoUpmixMethod = EChannelMapMonoUpmixMethod::EqualPower;
				checkNoEntry();
		}

		bool bSuccess = Create2DChannelMap(Params, ChannelMapCache[CacheID]);
		check(bSuccess);
	}

	void FMixerDevice::InitializeChannelMaps()
	{	
		// If we haven't yet created the static channel map cache
		if (!ChannelMapCache.Num())
		{
			// Make a matrix big enough for every possible configuration, double it to account for center channel only 
			ChannelMapCache.AddZeroed(AUDIO_MIXER_MAX_OUTPUT_CHANNELS * AUDIO_MIXER_MAX_OUTPUT_CHANNELS * 2);

			// Create a vorbis channel map cache
			VorbisChannelMapCache.AddZeroed(AUDIO_MIXER_MAX_OUTPUT_CHANNELS);

			// Loop through all input to output channel map configurations and cache them
			for (int32 OutputChannelCount = 1; OutputChannelCount < AUDIO_MIXER_MAX_OUTPUT_CHANNELS + 1; ++OutputChannelCount)
			{
				for (int32 InputChannelCount = 1; InputChannelCount < AUDIO_MIXER_MAX_OUTPUT_CHANNELS + 1; ++InputChannelCount)
				{
					// Cache non-vorbis channel maps
					CacheChannelMap(InputChannelCount, OutputChannelCount, true /* bIsCenterChannelOnly */);
					CacheChannelMap(InputChannelCount, OutputChannelCount, false /* bIsCenterChannelOnly */);
				}
				// Cache vorbis channel maps. 
				bool bSuccess = CreateVorbis2DChannelMap(OutputChannelCount, EChannelMapOrder::OutputMajorOrder, VorbisChannelMapCache[OutputChannelCount - 1]);
				check(bSuccess);
			}
		}
	}

	void FMixerDevice::InitializeChannelAzimuthMap(const int32 NumChannels)
	{
		// Initialize and cache 2D channel maps
		InitializeChannelMaps();

		// Now setup the hard-coded values
		if (NumChannels == 2)
		{
			DefaultChannelAzimuthPositions[EAudioMixerChannel::FrontLeft] = { EAudioMixerChannel::FrontLeft, 270 };
			DefaultChannelAzimuthPositions[EAudioMixerChannel::FrontRight] = { EAudioMixerChannel::FrontRight, 90 };
		}
		else
		{
			DefaultChannelAzimuthPositions[EAudioMixerChannel::FrontLeft] = { EAudioMixerChannel::FrontLeft, 330 };
			DefaultChannelAzimuthPositions[EAudioMixerChannel::FrontRight] = { EAudioMixerChannel::FrontRight, 30 };
		}

		if (bAllowCenterChannel3DPanning)
		{
			// Allow center channel for azimuth computations
			DefaultChannelAzimuthPositions[EAudioMixerChannel::FrontCenter] = { EAudioMixerChannel::FrontCenter, 0 };
		}
		else
		{
			// Ignore front center for azimuth computations. 
			DefaultChannelAzimuthPositions[EAudioMixerChannel::FrontCenter] = { EAudioMixerChannel::FrontCenter, INDEX_NONE };
		}

		// Always ignore low frequency channel for azimuth computations. 
		DefaultChannelAzimuthPositions[EAudioMixerChannel::LowFrequency] = { EAudioMixerChannel::LowFrequency, INDEX_NONE };

		DefaultChannelAzimuthPositions[EAudioMixerChannel::BackLeft] = { EAudioMixerChannel::BackLeft, 210 };
		DefaultChannelAzimuthPositions[EAudioMixerChannel::BackRight] = { EAudioMixerChannel::BackRight, 150 };
		DefaultChannelAzimuthPositions[EAudioMixerChannel::FrontLeftOfCenter] = { EAudioMixerChannel::FrontLeftOfCenter, 15 };
		DefaultChannelAzimuthPositions[EAudioMixerChannel::FrontRightOfCenter] = { EAudioMixerChannel::FrontRightOfCenter, 345 };
		DefaultChannelAzimuthPositions[EAudioMixerChannel::BackCenter] = { EAudioMixerChannel::BackCenter, 180 };
		DefaultChannelAzimuthPositions[EAudioMixerChannel::SideLeft] = { EAudioMixerChannel::SideLeft, 250 };
		DefaultChannelAzimuthPositions[EAudioMixerChannel::SideRight] = { EAudioMixerChannel::SideRight, 110 };

		// Check any engine ini overrides for these default positions
		if (NumChannels != 2)
		{
			int32 AzimuthPositionOverride = 0;
			for (int32 ChannelOverrideIndex = 0; ChannelOverrideIndex < EAudioMixerChannel::MaxSupportedChannel; ++ChannelOverrideIndex)
			{
				EAudioMixerChannel::Type MixerChannelType = EAudioMixerChannel::Type(ChannelOverrideIndex);
				
				// Don't allow overriding the center channel if its not allowed to spatialize.
				if (MixerChannelType != EAudioMixerChannel::FrontCenter || bAllowCenterChannel3DPanning)
				{
					const TCHAR* ChannelName = EAudioMixerChannel::ToString(MixerChannelType);
					if (GConfig->GetInt(TEXT("AudioChannelAzimuthMap"), ChannelName, AzimuthPositionOverride, GEngineIni))
					{
						if (AzimuthPositionOverride >= 0 && AzimuthPositionOverride < 360)
						{
							// Make sure no channels have this azimuth angle first, otherwise we'll get some bad math later
							bool bIsUnique = true;
							for (int32 ExistingChannelIndex = 0; ExistingChannelIndex < EAudioMixerChannel::MaxSupportedChannel; ++ExistingChannelIndex)
							{
								if (DefaultChannelAzimuthPositions[ExistingChannelIndex].Azimuth == AzimuthPositionOverride)
								{
									bIsUnique = false;

									// If the override is setting the same value as our default, don't print a warning
									if (ExistingChannelIndex != ChannelOverrideIndex)
									{
										const TCHAR* ExistingChannelName = EAudioMixerChannel::ToString(EAudioMixerChannel::Type(ExistingChannelIndex));
										UE_LOGF(LogAudioMixer, Warning, "Azimuth value '%d' for audio mixer channel '%ls' is already used by '%ls'. Azimuth values must be unique.",
											AzimuthPositionOverride, ChannelName, ExistingChannelName);
									}
									break;
								}
							}

							if (bIsUnique)
							{
								DefaultChannelAzimuthPositions[MixerChannelType].Azimuth = AzimuthPositionOverride;
							}
						}
						else
						{
							UE_LOGF(LogAudioMixer, Warning, "Azimuth value, %d, for audio mixer channel %ls out of range. Must be [0, 360).", AzimuthPositionOverride, ChannelName);
						}
					}
				}
			}
		}

		// Sort the current mapping by azimuth
		struct FCompareByAzimuth
		{
			FORCEINLINE bool operator()(const FChannelPositionInfo& A, const FChannelPositionInfo& B) const
			{
				return A.Azimuth < B.Azimuth;
			}
		};

		// Build a array of azimuth positions of only the current audio device's output channels
		DeviceChannelAzimuthPositions.Reset();

		// Setup the default channel azimuth positions
		TArray<FChannelPositionInfo> DevicePositions;
		for (EAudioMixerChannel::Type Channel : PlatformInfo.OutputChannelArray)
		{
			// Only track non-LFE and non-Center channel azimuths for use with 3d channel mappings
			if (Channel != EAudioMixerChannel::LowFrequency && DefaultChannelAzimuthPositions[Channel].Azimuth >= 0)
			{
				DeviceChannelAzimuthPositions.Add(DefaultChannelAzimuthPositions[Channel]);
			}
		}
		DeviceChannelAzimuthPositions.Sort(FCompareByAzimuth());
	}

	const TArray<EAudioMixerChannel::Type>& FMixerDevice::GetChannelArray() const
	{
		return PlatformInfo.OutputChannelArray;
	}
	const FChannelPositionInfo* FMixerDevice::GetDefaultChannelPositions() const
	{
		return DefaultChannelAzimuthPositions;
	}
}
