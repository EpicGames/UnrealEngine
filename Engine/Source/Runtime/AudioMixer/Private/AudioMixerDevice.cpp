// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioMixerDevice.h"

#include "AssetRegistry/IAssetRegistry.h"
#include "Async/Async.h"
#include "Audio/AudioDebug.h"
#include "AudioAnalytics.h"
#include "AudioBusSubsystem.h"
#include "AudioDeviceNotificationSubsystem.h"
#include "AudioMixerGeneratorSource.h"
#include "AudioMixerSource.h"
#include "AudioMixerSourceManager.h"
#include "AudioMixerSubmix.h"
#include "AudioMixerSourceVoice.h"
#include "AudioPluginUtilities.h"
#include "AudioMixerCVars.h"
#include "AudioMixerEffectsManager.h"
#include "AudioRenderScheduler.h"
#include "DSP/Noise.h"
#include "DSP/SinOsc.h"
#include "Logging/StructuredLog.h"
#include "Sound/AudioSettings.h"
#include "Sound/SoundSubmix.h"
#include "Sound/SoundSubmixSend.h"
#include "SubmixEffects/AudioMixerSubmixEffectEQ.h"
#include "SubmixEffects/AudioMixerSubmixEffectDynamicsProcessor.h"
#include "UObject/StrongObjectPtr.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "IHeadMountedDisplayModule.h"
#include "ISubmixBufferListener.h"
#include "Misc/App.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Async/Async.h"
#include "AudioDeviceNotificationSubsystem.h"
#include "Sound/AudioFormatSettings.h"
#include "HAL/PlatformMisc.h"

#if WITH_EDITOR
#include "AudioEditorModule.h"
#include "AudioEditorSettings.h"
#endif // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioMixerDevice)

#ifndef CASE_ENUM_TO_TEXT
#define CASE_ENUM_TO_TEXT(X) case X: return TEXT(#X);
#endif

const TCHAR* LexToString(const ERequiredSubmixes InType)
{
	switch (InType)
	{
		FOREACH_ENUM_EREQUIREDSUBMIXES(CASE_ENUM_TO_TEXT)
	}
	return TEXT("Unknown");
}

static int32 DisableSubmixEffectEQCvar = 1;
FAutoConsoleVariableRef CVarDisableSubmixEQ(
	TEXT("au.DisableSubmixEffectEQ"),
	DisableSubmixEffectEQCvar,
	TEXT("Disables the eq submix (true by default as of 5.0).\n")
	TEXT("0: Not Disabled, 1: Disabled"),
	ECVF_Default);

static int32 DisableSubmixMutationLockCVar = 0;
FAutoConsoleVariableRef CVarDisableSubmixMutationLock(
	TEXT("au.DisableSubmixMutationLock"),
	DisableSubmixMutationLockCVar,
	TEXT("Disables the submix mutation lock.\n")
	TEXT("0: Not Disabled (Default), 1: Disabled"),
	ECVF_Default);

static int32 EnableAudibleDefaultEndpointSubmixesCVar = 0;
FAutoConsoleVariableRef CVarEnableAudibleDefaultEndpointSubmixes(
	TEXT("au.submix.audibledefaultendpoints"),
	EnableAudibleDefaultEndpointSubmixesCVar,
	TEXT("Allows audio sent to defaulted (typically silent) endpoint submixes to be audible via master. (useful for debugging)\n")
	TEXT("0: Disabled (Default), 1: Enabled"),
	ECVF_Default);

static int32 DebugGeneratorEnableCVar = 0;
FAutoConsoleVariableRef CVarDebugGeneratorEnable(
	TEXT("au.Debug.Generator"),
	DebugGeneratorEnableCVar,
	TEXT("Enables/disables debug sound generation.\n")
	TEXT("0: Disabled, 1: SinTone, 2: WhiteNoise"),
	ECVF_Default);

static float DebugGeneratorAmpCVar = 0.2f;
FAutoConsoleVariableRef CVarDebugGeneratorAmp(
	TEXT("au.Debug.Generator.Amp"),
	DebugGeneratorAmpCVar,
	TEXT("Sets.\n")
	TEXT("Default: 0.2f"),
	ECVF_Default);

static int32 DebugGeneratorChannelCVar = 0;
FAutoConsoleVariableRef CVarDebugGeneratorChannel(
	TEXT("au.Debug.Generator.Channel"),
	DebugGeneratorChannelCVar,
	TEXT("Sets channel output index of debug audio.  If number provided is above supported number, uses left.\n")
	TEXT("0: Left, 1: Right, etc."),
	ECVF_Default);

static float DebugGeneratorFreqCVar = 440.0f;
FAutoConsoleVariableRef CVarDebugGeneratorFreq(
	TEXT("au.Debug.Generator.Freq"),
	DebugGeneratorFreqCVar,
	TEXT("Sets debug sound generation frequency.\n")
	TEXT("0: Not Disabled, 1: SinTone, 2: WhiteNoise"),
	ECVF_Default);

static int32 AudioMixerPatchBufferBlocks = 3;
FAutoConsoleVariableRef CVarAudioMixerPatchBufferBlocks(
	TEXT("au.PatchBufferBlocks"),
	AudioMixerPatchBufferBlocks,
	TEXT("Determines the number of blocks that fit in a patch buffer."),
	ECVF_Default);

static int32 bMuteAudioCVar = 0;
FAutoConsoleVariableRef CVarMuteAudio(
	TEXT("au.MuteAudio"),
	bMuteAudioCVar,
	TEXT("Mutes the audio output in the final output without effecting audio engine behavior.\n")
	TEXT("1: Mutes the final submix. 0: Does not mute the final submix."),
	ECVF_Cheat);

static int32 SubmixBufferListenerForceRegistrationCVar = 1;
FAutoConsoleVariableRef CVarSubmixBufferListenerForceRegistration(
	TEXT("au.SubmixBufferListenerForceRegistration"),
	SubmixBufferListenerForceRegistrationCVar,
	TEXT("If true, will force register the referenced submix when Registering a Submix Buffer Listener."),
	ECVF_Default);

static int32 AudioMixerGeneratorSourcesCVar = 0;
FAutoConsoleVariableRef CVarAudioMixerGeneratorSources(
	TEXT("au.NumGeneratorSources"),
	AudioMixerGeneratorSourcesCVar,
	TEXT("Sets number of generator sources available in the AudioMixer."),
	ECVF_Default);

#if UE_AUDIO_PROFILERTRACE_ENABLED

UE_TRACE_EVENT_BEGIN(Audio, MainSubmixLoaded)
	UE_TRACE_EVENT_FIELD(uint32, DeviceId)
	UE_TRACE_EVENT_FIELD(uint32, SubmixId)
	UE_TRACE_EVENT_FIELD(uint64, Timestamp)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Audio, MainSubmixAlivePing)
	UE_TRACE_EVENT_FIELD(uint32, DeviceId)
	UE_TRACE_EVENT_FIELD(uint32, SubmixId)
	UE_TRACE_EVENT_FIELD(uint64, Timestamp)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Audio, MainSubmixTruePeakMaxValue)
	UE_TRACE_EVENT_FIELD(uint32, DeviceId)
	UE_TRACE_EVENT_FIELD(uint32, SubmixId)
	UE_TRACE_EVENT_FIELD(uint64, Timestamp)
	UE_TRACE_EVENT_FIELD(float, TruePeakMaxValueDb)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Audio, MainSubmixLKFSValues)
	UE_TRACE_EVENT_FIELD(uint32, DeviceId)
	UE_TRACE_EVENT_FIELD(uint32, SubmixId)
	UE_TRACE_EVENT_FIELD(uint64, Timestamp)
	UE_TRACE_EVENT_FIELD(float, LongTermLoudness)
	UE_TRACE_EVENT_FIELD(float, ShortTermLoudness)
	UE_TRACE_EVENT_FIELD(float, MomentaryLoudness)
	UE_TRACE_EVENT_FIELD(float, LoudnessRangeLowerBound)
	UE_TRACE_EVENT_FIELD(float, LoudnessRangeUpperBound)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Audio, SubmixAlivePing)
	UE_TRACE_EVENT_FIELD(uint32, DeviceId)
	UE_TRACE_EVENT_FIELD(uint32, SubmixId)
	UE_TRACE_EVENT_FIELD(uint64, Timestamp)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Name)
	UE_TRACE_EVENT_FIELD(int32, NumChannels)
	UE_TRACE_EVENT_FIELD(bool, IsMainSubmix)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Audio, SubmixToSubmixSend)
	UE_TRACE_EVENT_FIELD(uint32, DeviceId)
	UE_TRACE_EVENT_FIELD(uint64, Timestamp)
	UE_TRACE_EVENT_FIELD(uint32, SenderID)
	UE_TRACE_EVENT_FIELD(uint32, ReceiverID)
	UE_TRACE_EVENT_FIELD(float, SendLevel)
	UE_TRACE_EVENT_FIELD(bool, IsFinalStage)
UE_TRACE_EVENT_END()

#endif // UE_AUDIO_PROFILERTRACE_ENABLED

// Link to "Audio" profiling category
CSV_DECLARE_CATEGORY_MODULE_EXTERN(AUDIOMIXERCORE_API, Audio);

namespace Audio
{
	int32 GRemoveSubmixesFromRoot = 1;
	FAutoConsoleVariableRef CVarRemoveSubmixesFromRoot(
		TEXT("au.RemoveSubmixesFromRoot"),
		GRemoveSubmixesFromRoot,
		TEXT("Submixes that are registered will use a ref counter. When unregistering, the ref counter is decremented. When no longer referenced, the submixes are removed from root.\n")
		TEXT("1: Submixes are removed from root when no longer referenced. 0: Submixes remain rooted."),
		ECVF_Default);
	
	void FSubmixMap::Add(const FSubmixMap::FObjectId InObjectId, FMixerSubmixPtr InMixerSubmix)
	{
		if (DisableSubmixMutationLockCVar)
		{
			SubmixMap.Add(InObjectId, InMixerSubmix);
		}
		else
		{
			FScopeLock ScopeLock(&MutationLock);
			SubmixMap.Add(InObjectId, InMixerSubmix);
		}
	}

	void FSubmixMap::Iterate(FIterFunc InFunction)
	{
		if (DisableSubmixMutationLockCVar)
		{
			for (const FPair& Pair : SubmixMap)
			{
				InFunction(Pair);
			}
		}
		else
		{
			FScopeLock ScopeLock(&MutationLock);
			for (const FPair& Pair : SubmixMap)
			{
				InFunction(Pair);
			}
		}
	}

	FMixerSubmixPtr FSubmixMap::FindRef(const FSubmixMap::FObjectId InObjectId) const
	{
		if (DisableSubmixMutationLockCVar)
		{
			return SubmixMap.FindRef(InObjectId);
		}
		else
		{
			FScopeLock ScopeLock(&MutationLock);
			return SubmixMap.FindRef(InObjectId);
		}
	}

	int32 FSubmixMap::Remove(const FSubmixMap::FObjectId InObjectId)
	{
		if (DisableSubmixMutationLockCVar)
		{
			return SubmixMap.Remove(InObjectId);
		}
		else
		{
			FScopeLock ScopeLock(&MutationLock);
			return SubmixMap.Remove(InObjectId);
		}
	}

	void FSubmixMap::Reset()
	{
		if (DisableSubmixMutationLockCVar)
		{
			SubmixMap.Reset();
		}
		else
		{
			FScopeLock ScopeLock(&MutationLock);
			SubmixMap.Reset();
		}
	}

	TSet<FSubmixMap::FObjectId> FSubmixMap::GetKeys() const
	{
		FScopeLock ScopeLock(&MutationLock);
		TArray<FObjectId> Keys;
		SubmixMap.GenerateKeyArray(Keys);
		return TSet<FObjectId>(Keys);
	}

	static FAutoConsoleCommand DumpSubmixCmd(
		TEXT("au.submix.drawgraph"),
		TEXT("Draws the submix hierarchy for this world to the debug output"),
		FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateLambda([](const TArray<FString>& InArgs, UWorld* InWorld, FOutputDevice& OutLog)
		{
			if (InWorld)
			{
				if (const FMixerDevice* MixerDevice = static_cast<FMixerDevice*>(InWorld->GetAudioDeviceRaw()))
				{
					MixerDevice->DrawSubmixes(OutLog, InArgs);
				}
			}
		})
	);
	
	static void DrawSubmixHierarchy(USoundSubmixBase* InSubmix, const TSharedPtr<Audio::FMixerSubmix, ESPMode::ThreadSafe> InInstance, const FMixerDevice* InDevice, int32 InIdent, FOutputDevice& Ar, const TCHAR* GroupingText)
	{
		if (!InSubmix)
		{
			return;
		}
		
		FString Indent = FCString::Spc(InIdent*3);
		FString FxChain;
		
		// Show what FX the mixer is actually using: the live FMixerSubmix's current chain
		// This reflects any active override 
		// Fall back to the static USoundSubmix asset chain only when no runtime instance exists yet.
		if (InInstance.IsValid())
		{
			const int32 NumEffects = InInstance->GetNumChainEffects();
			for (int32 EffectIdx = 0; EffectIdx < NumEffects; ++EffectIdx)
			{
				FSoundEffectSubmixPtr EffectPtr = InInstance->GetSubmixEffect(EffectIdx);
				const USoundEffectPreset* Preset = EffectPtr.IsValid() ? EffectPtr->GetPreset() : nullptr;
				FxChain += FString::Printf(TEXT("[%s]"), *GetNameSafe(Preset));
			}
		}
		else if (USoundSubmix* Submix = Cast<USoundSubmix>(InSubmix))
		{	
			for (const TObjectPtr<USoundEffectSubmixPreset>& i: Submix->SubmixEffectChain)
			{
				FxChain += FString::Printf(TEXT("[%s]"), *GetNameSafe(i));
			}
		}

		//Get Mute/Solo state
		FString MuteSoloStateString = FCString::Spc(3);

#if ENABLE_AUDIO_DEBUG
		if (FAudioDeviceManager* Manager = FAudioDeviceManager::Get())
		{
			FAudioDebugger& Debugger = Manager->GetDebugger();
			bool bIsSoloed = false;
			bool bIsMuted = false;
			FString SoloMuteStateReason;
			Debugger.QuerySoloMuteSubmix(InSubmix->GetName(), bIsSoloed, bIsMuted, SoloMuteStateReason);

			if  (bIsSoloed)
			{
				MuteSoloStateString = "[S]";
			}
			else if (bIsMuted)
			{
				MuteSoloStateString = "[M]";
			}
		}
#endif
		
		Ar.Logf(TEXT("%s%sName=%s,Instance=0x%p,Id=%u,Fx=%s,[%s]"), *MuteSoloStateString, *Indent, *InSubmix->GetName(), InInstance.Get(), InInstance ? InInstance->GetId() : 0, *FxChain, GroupingText);
		for (const auto& i : InSubmix->ChildSubmixes)
		{
			DrawSubmixHierarchy(i, InDevice->GetSubmixInstance(i).Pin(), InDevice, InIdent+1, Ar, TEXT("Static"));
		}
		const auto& DynamicSubmixes = InSubmix->DynamicChildSubmixes.FindOrAdd(InDevice->DeviceID);
		for (const auto& i : DynamicSubmixes.ChildSubmixes)
		{
			DrawSubmixHierarchy(i, InDevice->GetSubmixInstance(i).Pin(), InDevice, InIdent+1, Ar, TEXT("Dynamic"));
		}
	}

	static void DrawSubmixInstances(const FMixerSubmix* InRoot, const int32 InIdent, FOutputDevice& InOutput)
	{
		if (!InRoot)
		{
			return;
		}

		const FString Indent = FCString::Spc(InIdent*3);
		InOutput.Logf(TEXT("%sName=%s,Instance=0x%p,Id=%u"),
			*Indent, *InRoot->GetName(), InRoot, InRoot->GetId());

		// Go downwards.
		const TMap<uint32, FChildSubmixInfo>& Children = InRoot->GetChildren();
		for (const auto& i : Children)
		{
			if (FMixerSubmixPtr Child =  i.Value.SubmixPtr.Pin())
			{
				DrawSubmixInstances(Child.Get(), InIdent+1, InOutput);
			}
		}
	}
	
	void FMixerDevice::DrawSubmixes(FOutputDevice& InOutput, const TArray<FString>& InArgs) const
	{
		InOutput.Logf(TEXT("AudioDevice=%d, Device Instance=0x%p"), DeviceID, this);

		// Params.
		if (Algo::FindByPredicate(InArgs, [](const FString& InStr){ return FParse::Param(*InStr, TEXT("Instances")); }) != nullptr)
		{
			InOutput.Logf(TEXT("[Instance Hierarchy]"));
			for (int32 i = 0; i < RequiredSubmixes.Num(); i++)
			{
				InOutput.Logf(TEXT("SlotName=[%s]"), ToCStr(LexToString(static_cast<ERequiredSubmixes>(i))));
				DrawSubmixInstances(RequiredSubmixInstances[i].Get(), 1, InOutput);
			}
		}
		if (Algo::FindByPredicate(InArgs, [](const FString& InStr){ return FParse::Param(*InStr, TEXT("Map")); }) != nullptr)
		{
			InOutput.Logf(TEXT("[Map of UObject -> SubmixPtrs]"));

			// Map loop of unique ids from UObjects.
			TMap<uint32, USoundSubmixBase*> AllSubmixes;
			for (TObjectIterator<USoundSubmixBase> It; It; ++It)
			{
				AllSubmixes.Add(It->GetUniqueID(),*It);
			}
			for (const auto i : Submixes.GetKeys())
			{
				const auto pFound = AllSubmixes.Find(i);
				InOutput.Logf(TEXT("%u -> %s"), i, pFound ? *GetNameSafe(*pFound) : TEXT("Not found"));
			}
		}

		// USubmixMap hierarchy from slots downwards.
		InOutput.Logf(TEXT("[Map of UObject Hierarchy]"));
		for (int32 i = 0; i < RequiredSubmixes.Num(); i++)
		{
			InOutput.Logf(TEXT("SlotName=[%s]"), ToCStr(LexToString(static_cast<ERequiredSubmixes>(i))));
			DrawSubmixHierarchy(RequiredSubmixes[i], RequiredSubmixInstances[i], this, 1, InOutput, TEXT("In Slot"));
		}
	}

#if ENABLE_AUDIO_DEBUG
	void FMixerDevice::UpdateSubmixDebugVolumes()
	{
		Submixes.Iterate([](const FSubmixMap::FPair& SubmixPair)
		{
			const FMixerSubmixPtr& Submix = SubmixPair.Value;
			if (Submix.IsValid())
			{
				Submix->UpdateDebugVolume();
			}
		});

		for (const FMixerSubmixPtr& RequiredSubmix : RequiredSubmixInstances)
		{
			if (RequiredSubmix.IsValid())
			{
				RequiredSubmix->UpdateDebugVolume();
			}
		}
	}

	static FAutoConsoleCommand SubmixSoloCmd(
		TEXT("au.submix.solo"),
		TEXT("Toggle solo on a submix by name. Soloed submix is audible, all others are silenced. Usage: au.submix.solo <SubmixName>"),
		FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& InArgs)
		{
			FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get();
			if (AudioDeviceManager == nullptr)
			{
				return;
			}

			Audio::FAudioDebugger& Debugger = AudioDeviceManager->GetDebugger();

			if (InArgs.Num() < 1 || InArgs[0].IsEmpty())
			{
				Debugger.ClearSubmixMuteSoloState();
				return;
			}

			Debugger.ToggleSoloSubmixChain(FName(InArgs[0]));
		}));

	static FAutoConsoleCommand SubmixMuteCmd(
		TEXT("au.submix.mute"),
		TEXT("Toggle mute on a submix by name. Usage: au.submix.mute <SubmixName>"),
		FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& InArgs)
		{
			FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get();
			if (AudioDeviceManager == nullptr)
			{
				return;
			}

			Audio::FAudioDebugger& Debugger = AudioDeviceManager->GetDebugger();

			if (InArgs.Num() < 1 || InArgs[0].IsEmpty())
			{
				Debugger.ClearSubmixMuteSoloState();
				return;
			}

			Debugger.ToggleMuteSubmix(FName(InArgs[0]));
		}));
        
#endif // ENABLE_AUDIO_DEBUG

	FMixerDevice::FMixerDevice(IAudioMixerPlatformInterface* InAudioMixerPlatform)
		: QuantizedEventClockManager(this)
		, AudioMixerPlatform(InAudioMixerPlatform)
		, AudioClockDelta(0.0)
		, PreviousPrimaryVolume((float)INDEX_NONE)
		, GameOrAudioThreadId(INDEX_NONE)
		, AudioPlatformThreadId(INDEX_NONE)
		, bDebugOutputEnabled(false)
		, bSubmixRegistrationDisabled(true)
	{
		// This audio device is the audio mixer
		bAudioMixerModuleLoaded = true;

		SourceManager = MakeUnique<FMixerSourceManager>(this);

		RenderScheduler = MakeUnique<FAudioRenderScheduler>(this);
		bUseRenderScheduler = AudioMixerCVars::UseRenderScheduler != 0;
		UE_LOGFMT(LogAudioMixer, Display, "bUseRenderScheduler set to {UseRenderScheduler}.", ("UseRenderScheduler", bUseRenderScheduler));

		// Register AudioLink Factory. 	
		TArray<FName> Factories = IAudioLinkFactory::GetAllRegisteredFactoryNames();
		if(Factories.Num() > 0)
		{
			// Allow only a single registered factory instance for now.
			check(Factories.Num()==1);
			AudioLinkFactory=IAudioLinkFactory::FindFactory(Factories[0]);
		}

#if UE_AUDIO_PROFILERTRACE_ENABLED
		FTraceAuxiliary::OnTraceStarted.AddRaw(this, &FMixerDevice::OnTraceStarted);
		FTraceAuxiliary::OnTraceStopped.AddRaw(this, &FMixerDevice::OnTraceStopped);
#endif // UE_AUDIO_PROFILERTRACE_ENABLED
	}

	FMixerDevice::~FMixerDevice()
	{
		AUDIO_MIXER_CHECK_GAME_THREAD(this);

#if UE_AUDIO_PROFILERTRACE_ENABLED
		FTraceAuxiliary::OnTraceStarted.RemoveAll(this);
		FTraceAuxiliary::OnTraceStopped.RemoveAll(this);
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

		// Shutdown all pending clock events, as they may have references to 
		// the FMixerSourceManager that is about to be destroyed
		QuantizedEventClockManager.Shutdown();

		if (AudioMixerPlatform != nullptr)
		{
			delete AudioMixerPlatform;
		}

		// The rendering thread has been stopped now, so it is safe to run any cleanup code that
		//  would normally be run only on the rendering thread.
		ResetAudioRenderingThreadId();
	}

#if UE_AUDIO_PROFILERTRACE_ENABLED
	void FMixerDevice::OnTraceStarted(FTraceAuxiliary::EConnectionType TraceType, const FString& TraceDestination)
	{
		SendSubmixesAlivePing();
		SendMainSubmixAlivePingMessage();
	}

	void FMixerDevice::OnTraceStopped(FTraceAuxiliary::EConnectionType TraceType, const FString& TraceDestination)
	{
		// Auto-Disable output metering
		Submixes.Iterate([this](const FSubmixMap::FPair& SubmixPair)
		{
			const FMixerSubmixPtr& Submix = SubmixPair.Value;
			if (Submix.IsValid() && Submix->bIsEnvelopeFollowing && SubmixesWithAutoEnvelopeFollowers.Contains(Submix->GetId()))
			{
				FAudioThread::RunCommandOnAudioThread([this, Submix]()
				{
					StopSubmixEnvelopeFollowing(Submix);
				});
			}
		});

		for (const FMixerSubmixPtr& RequiredSubmix : RequiredSubmixInstances)
		{
			if (RequiredSubmix.IsValid() && RequiredSubmix->bIsEnvelopeFollowing && SubmixesWithAutoEnvelopeFollowers.Contains(RequiredSubmix->GetId()))
			{
				FAudioThread::RunCommandOnAudioThread([this, RequiredSubmix]()
				{
					StopSubmixEnvelopeFollowing(RequiredSubmix);
				});
			}
		}

		SubmixesWithAutoEnvelopeFollowers.Reset();
	}

	void FMixerDevice::InitializeMainSubmixTruePeakEnvelopeFollower(int32 InSampleRate, int32 InNumChannels, int32 InNumSamples)
	{
		MainSubmixTruePeakEnvelopeFollower = MakeUnique<FEnvelopeFollower>(
			FEnvelopeFollowerInitParams{
				.SampleRate = static_cast<float>(InSampleRate),
				.NumChannels = InNumChannels,
				.AttackTimeMsec = 0.0f,
				.ReleaseTimeMsec = 0.0f,
				.Mode = EPeakMode::TruePeak,
				.bIsAnalog = false,
			});
		
		MainSubmixTruePeakOutputEnvelope.SetNumUninitialized(InNumSamples);
	}

	void FMixerDevice::InitializeMainSubmixLKFSAnalyzer(int32 InSampleRate, int32 InNumChannels)
	{
		FLKFSAnalyzerSettings LKFSAnalyzerSettings;
		LKFSAnalyzerSettings.IntegratedLoudnessDuration = 1800.0f; // 30 minutes
		LKFSAnalyzerSettings.bCalculateOverallLoudnessRange = true;

		MainSubmixLKFSAnalyzer = MakeUnique<FLKFSAnalyzer>(InNumChannels, static_cast<float>(InSampleRate), LKFSAnalyzerSettings);
	}

	void FMixerDevice::SendMainSubmixLoadedMessage()
	{
		const bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AudioMixerChannel);
		if (!bChannelEnabled)
		{
			return;
		}

		const FMixerSubmixPtr MainSubmixPtr = GetMainSubmix().Pin();
		if (!MainSubmixPtr.IsValid())
		{
			return;
		}

		UE_TRACE_LOG(Audio, MainSubmixLoaded, AudioMixerChannel)
			<< MainSubmixLoaded.DeviceId(DeviceID)
			<< MainSubmixLoaded.SubmixId(GetTypeHash(MainSubmixPtr->GetPathName()))
			<< MainSubmixLoaded.Timestamp(FPlatformTime::Cycles64());
	}

	void FMixerDevice::SendMainSubmixAlivePingMessage()
	{
		const bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AudioMixerChannel);
		if (!bChannelEnabled)
		{
			return;
		}

		const FMixerSubmixPtr MainSubmixPtr = GetMainSubmix().Pin();
		if (!MainSubmixPtr.IsValid())
		{
			return;
		}

		UE_TRACE_LOG(Audio, MainSubmixAlivePing, AudioMixerChannel)
			<< MainSubmixAlivePing.DeviceId(DeviceID)
			<< MainSubmixAlivePing.SubmixId(GetTypeHash(MainSubmixPtr->GetPathName()))
			<< MainSubmixAlivePing.Timestamp(FPlatformTime::Cycles64());
	}

	void FMixerDevice::UpdateAndSendMainSubmixTruePeakMaxValueMessage(const FAlignedFloatBuffer& InMainSubmixAudioBuffer)
	{
		const bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AudioMixerChannel);
		if (!bChannelEnabled)
		{
			return;
		}

		const FMixerSubmixPtr MainSubmixPtr = GetMainSubmix().Pin();
		if (!MainSubmixPtr.IsValid())
		{
			return;
		}

		if (!MainSubmixTruePeakEnvelopeFollower.IsValid())
		{
			return;
		}

		MainSubmixTruePeakEnvelopeFollower->ProcessAudio(InMainSubmixAudioBuffer.GetData(), PlatformSettings.CallbackBufferFrameSize, MainSubmixTruePeakOutputEnvelope.GetData());
		
		const float* MainSubmixTruePeakMaxValuePtr = Algo::MaxElement(MainSubmixTruePeakOutputEnvelope);
		const float MainSubmixTruePeakMaxValueDb   = MainSubmixTruePeakMaxValuePtr ? ConvertToDecibels(*MainSubmixTruePeakMaxValuePtr) : MIN_VOLUME_DECIBELS;

		UE_TRACE_LOG(Audio, MainSubmixTruePeakMaxValue, AudioMixerChannel)
			<< MainSubmixTruePeakMaxValue.DeviceId(DeviceID)
			<< MainSubmixTruePeakMaxValue.SubmixId(GetTypeHash(MainSubmixPtr->GetPathName()))
			<< MainSubmixTruePeakMaxValue.Timestamp(FPlatformTime::Cycles64())
			<< MainSubmixTruePeakMaxValue.TruePeakMaxValueDb(MainSubmixTruePeakMaxValueDb);
	}

	void FMixerDevice::UpdateAndSendMainSubmixLKFSValuesMessage(const FAlignedFloatBuffer& InMainSubmixAudioBuffer)
	{
		const bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AudioMixerChannel);
		if (!bChannelEnabled)
		{
			return;
		}

		const FMixerSubmixPtr MainSubmixPtr = GetMainSubmix().Pin();
		if (!MainSubmixPtr.IsValid())
		{
			return;
		}

		if (!MainSubmixLKFSAnalyzer.IsValid())
		{
			return;
		}

		FLKFSAnalyzerResults MainSubmixLKFSAnalyzerResults;
		MainSubmixLKFSAnalyzer->Analyze(InMainSubmixAudioBuffer, MainSubmixLKFSAnalyzerResults);

		const TArray<FLKFSResult>& OverallLKFSArray = MainSubmixLKFSAnalyzerResults.GetLoudnessResults();
		if (OverallLKFSArray.Num() > 0)
		{
			const FLKFSResult& LatestLKFSResult = OverallLKFSArray[OverallLKFSArray.Num() - 1];

			float LoudnessRangeLowerBound = 0.0f;
			float LoudnessRangeUpperBound = 0.0f;

			if (LatestLKFSResult.LoudnessRange.IsSet())
			{
				LoudnessRangeLowerBound = LatestLKFSResult.LoudnessRange.GetValue().Min;
				LoudnessRangeUpperBound = LatestLKFSResult.LoudnessRange.GetValue().Max;
			}

			UE_TRACE_LOG(Audio, MainSubmixLKFSValues, AudioMixerChannel)
				<< MainSubmixLKFSValues.DeviceId(DeviceID)
				<< MainSubmixLKFSValues.SubmixId(GetTypeHash(MainSubmixPtr->GetPathName()))
				<< MainSubmixLKFSValues.Timestamp(FPlatformTime::Cycles64())
				<< MainSubmixLKFSValues.LongTermLoudness(LatestLKFSResult.GatedLoudness)
				<< MainSubmixLKFSValues.ShortTermLoudness(LatestLKFSResult.ShortTermLoudness)
				<< MainSubmixLKFSValues.MomentaryLoudness(LatestLKFSResult.Loudness)
				<< MainSubmixLKFSValues.LoudnessRangeLowerBound(LoudnessRangeLowerBound)
				<< MainSubmixLKFSValues.LoudnessRangeUpperBound(LoudnessRangeUpperBound);
		}
	}

	void FMixerDevice::SendSubmixesAlivePing()
	{
		const bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AudioChannel);
		if (bChannelEnabled)
		{
			auto TraceSubmixLoadedMessage = [this](const FMixerSubmixPtr& InSubmix)
			{
				UE_TRACE_LOG(Audio, SubmixAlivePing, AudioChannel)
					<< SubmixAlivePing.DeviceId(DeviceID)
					<< SubmixAlivePing.SubmixId(GetTypeHash(InSubmix->GetPathName()))
					<< SubmixAlivePing.Timestamp(FPlatformTime::Cycles64())
					<< SubmixAlivePing.Name(*InSubmix->GetPathName())
					<< SubmixAlivePing.NumChannels(InSubmix->NumChannels)
					<< SubmixAlivePing.IsMainSubmix(InSubmix == GetMainSubmix());
			};

			Submixes.Iterate([this, &TraceSubmixLoadedMessage](const FSubmixMap::FPair& SubmixPair)
			{
				const FMixerSubmixPtr& Submix = SubmixPair.Value;
				if (Submix.IsValid())
				{
					TraceSubmixLoadedMessage(Submix);
				}
			});

			for (const FMixerSubmixPtr& RequiredSubmix : RequiredSubmixInstances)
			{
				if (RequiredSubmix.IsValid())
				{
					TraceSubmixLoadedMessage(RequiredSubmix);
				}
			}
		}

		AutomaticallyStartSubmixEnvelopeFollowers();
	}

	void FMixerDevice::UpdateSubmixesAlivePingTime()
	{
		SubmixesAlivePingTimeSec += AudioClockDelta;

		if (SubmixesAlivePingTimeSec > SubmixesAlivePingWaitTimeSec)
		{
			SendSubmixesAlivePing();
			SendMainSubmixAlivePingMessage();

			SubmixesAlivePingTimeSec = 0.0;
		}
	}

	void FMixerDevice::SendSubmixesSendLevelTraceMessages()
	{
		const bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AudioMixerChannel);
		if (bChannelEnabled)
		{
			auto TraceSubmixToSubmixSendMessage = [this](const FMixerSubmixPtr& InSubmix)
			{
				if (!InSubmix.IsValid())
				{
					return;
				}

				const Audio::FMixerSubmixPtr OutputSubmix = InSubmix->GetParentSubmix().Pin();
				
				// If this submix has no parent, send the Audio Device as the receiver ID and mark as a final stage of the chain
				const bool bIsFinalStage = !OutputSubmix.IsValid();
				const uint32 ReceiverID = bIsFinalStage ? DeviceID : GetTypeHash(OutputSubmix->GetPathName());
				
				UE_TRACE_LOG(Audio, SubmixToSubmixSend, AudioMixerChannel)
					<< SubmixToSubmixSend.DeviceId(DeviceID)
					<< SubmixToSubmixSend.Timestamp(FPlatformTime::Cycles64())
					<< SubmixToSubmixSend.SenderID(GetTypeHash(InSubmix->GetPathName()))
					<< SubmixToSubmixSend.ReceiverID(ReceiverID)
					<< SubmixToSubmixSend.SendLevel(InSubmix->CurrentOutputVolume)
					<< SubmixToSubmixSend.IsFinalStage(bIsFinalStage);
			};

			Submixes.Iterate([&TraceSubmixToSubmixSendMessage](const FSubmixMap::FPair& SubmixPair)
			{
				TraceSubmixToSubmixSendMessage(SubmixPair.Value);
			});

			for (const FMixerSubmixPtr& RequiredSubmixInstance : RequiredSubmixInstances)
			{
				TraceSubmixToSubmixSendMessage(RequiredSubmixInstance);
			}
		}
	}

	void FMixerDevice::RegisterSubmixEnvelopeFollowerCVars()
	{
		// StartSubmixEnvelopeFollowerCVar
		StartSubmixEnvelopeFollowerCVar = IConsoleManager::Get().RegisterConsoleVariable(
			*FString::Printf(TEXT("au.StartSubmixEnvelopeFollower_AD%d"), DeviceID),
			FString(),
			TEXT("Starts a submix envelope follower (args= submix name)\n"),
			ECVF_Default
		);

		if (StartSubmixEnvelopeFollowerCVar)
		{
			StartSubmixEnvelopeFollowerCVar->SetOnChangedCallback(FConsoleVariableDelegate::CreateRaw(this, &FMixerDevice::OnStartSubmixEnvelopeFollowerCVarChanged));
		}

		// StopSubmixEnvelopeFollowerCVar
		StopSubmixEnvelopeFollowerCVar = IConsoleManager::Get().RegisterConsoleVariable(
			*FString::Printf(TEXT("au.StopSubmixEnvelopeFollower_AD%d"), DeviceID),
			FString(),
			TEXT("Stops a submix envelope follower (args= submix name)\n"),
			ECVF_Default
		);

		if (StopSubmixEnvelopeFollowerCVar)
		{
			StopSubmixEnvelopeFollowerCVar->SetOnChangedCallback(FConsoleVariableDelegate::CreateRaw(this, &FMixerDevice::OnStopSubmixEnvelopeFollowerCVarChanged));
		}
	}

	void FMixerDevice::UnregisterSubmixEnvelopeFollowerCVars()
	{
		IConsoleManager::Get().UnregisterConsoleObject(StartSubmixEnvelopeFollowerCVar);
		IConsoleManager::Get().UnregisterConsoleObject(StopSubmixEnvelopeFollowerCVar);

		StartSubmixEnvelopeFollowerCVar	= nullptr;
		StopSubmixEnvelopeFollowerCVar	= nullptr;
	}

	void FMixerDevice::OnStartSubmixEnvelopeFollowerCVarChanged(IConsoleVariable* InStartSubmixEnvelopeFollowerCVar)
	{
		ensure(IsInGameThread());

		if (InStartSubmixEnvelopeFollowerCVar)
		{
			const FString SubmixName = InStartSubmixEnvelopeFollowerCVar->GetString();
			if (!SubmixesWithCVarEnvelopeFollowers.Contains(SubmixName))
			{
				Submixes.Iterate([this, &SubmixName](const FSubmixMap::FPair& SubmixPair)
				{
					const FMixerSubmixPtr& Submix = SubmixPair.Value;

					if (Submix.IsValid() && Submix->GetName() == SubmixName)
					{
						FAudioThread::RunCommandOnAudioThread([this, Submix]()
						{
							StartSubmixEnvelopeFollowing(Submix);
						});

						SubmixesWithCVarEnvelopeFollowers.Add(SubmixName);
					}
				});

				for (const FMixerSubmixPtr& RequiredSubmix : RequiredSubmixInstances)
				{
					if (RequiredSubmix.IsValid() && RequiredSubmix->GetName() == SubmixName)
					{
						FAudioThread::RunCommandOnAudioThread([this, RequiredSubmix]()
						{
							StartSubmixEnvelopeFollowing(RequiredSubmix);
						});

						SubmixesWithCVarEnvelopeFollowers.Add(SubmixName);
					}
				}

				SubmixesWithCVarEnvelopeFollowers.Add(SubmixName);
			}
			
		}
	}

	void FMixerDevice::OnStopSubmixEnvelopeFollowerCVarChanged(IConsoleVariable* InStopSubmixEnvelopeFollowerCVar)
	{
		ensure(IsInGameThread());

		if (InStopSubmixEnvelopeFollowerCVar)
		{
			const FString SubmixName = InStopSubmixEnvelopeFollowerCVar->GetString();
			if (SubmixesWithCVarEnvelopeFollowers.Contains(SubmixName))
			{
				Submixes.Iterate([this, &SubmixName](const FSubmixMap::FPair& SubmixPair)
				{
					const FMixerSubmixPtr& Submix = SubmixPair.Value;

					if (Submix.IsValid() && Submix->GetName() == SubmixName)
					{
						FAudioThread::RunCommandOnAudioThread([this, Submix]()
						{
							StopSubmixEnvelopeFollowing(Submix);
						});

						SubmixesWithCVarEnvelopeFollowers.Remove(SubmixName);
					}
				});

				for (const FMixerSubmixPtr& RequiredSubmix : RequiredSubmixInstances)
				{
					if (RequiredSubmix.IsValid() && RequiredSubmix->GetName() == SubmixName)
					{
						FAudioThread::RunCommandOnAudioThread([this, RequiredSubmix]()
						{
							StopSubmixEnvelopeFollowing(RequiredSubmix);
						});

						SubmixesWithCVarEnvelopeFollowers.Remove(SubmixName);
					}
				}
			}
		}
	}

	void FMixerDevice::StartSubmixEnvelopeFollowing(const FMixerSubmixPtr& InSubmix)
	{
		ensure(IsInAudioThread());

		if (!InSubmix || !InSubmix->IsOwnerValid())
		{
			return;
		}

		// Attack/Release default times used in USoundSubmix constructor
		constexpr int32 AttackTime = 10;
		constexpr int32 ReleaseTime = 500;

		InSubmix->StartEnvelopeFollowing(AttackTime, ReleaseTime);
	}

	void FMixerDevice::StopSubmixEnvelopeFollowing(const FMixerSubmixPtr& InSubmix)
	{
		ensure(IsInAudioThread());

		if (!InSubmix || !InSubmix->IsOwnerValid() || !InSubmix->bIsEnvelopeFollowing)
		{
			return;
		}

		InSubmix->StopEnvelopeFollowing();
	}

	void FMixerDevice::AutomaticallyStartSubmixEnvelopeFollowers()
	{
		const bool bAudioMixerChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AudioMixerChannel);
		if (bAudioMixerChannelEnabled)
		{
			// Auto-Enable output metering
			Submixes.Iterate([this](const FSubmixMap::FPair& SubmixPair)
			{
				const FMixerSubmixPtr& Submix = SubmixPair.Value;
				if (Submix.IsValid())
				{
					const uint32 SubmixID = Submix->GetId();
					if (!SubmixesWithAutoEnvelopeFollowers.Contains(SubmixID))
					{
						SubmixesWithAutoEnvelopeFollowers.Add(SubmixID);
						
						FAudioThread::RunCommandOnAudioThread([this, Submix]()
						{
							StartSubmixEnvelopeFollowing(Submix);
						});
					}
				}
			});

			for (const FMixerSubmixPtr& RequiredSubmix : RequiredSubmixInstances)
			{
				if (RequiredSubmix.IsValid())
				{
					const uint32 SubmixID = RequiredSubmix->GetId();
					if (!SubmixesWithAutoEnvelopeFollowers.Contains(SubmixID))
					{
						SubmixesWithAutoEnvelopeFollowers.Add(SubmixID);
						FAudioThread::RunCommandOnAudioThread([this, RequiredSubmix]()
						{
							StartSubmixEnvelopeFollowing(RequiredSubmix);
						});
					}
				}
			}
		}
	}

	void FMixerDevice::RegisterResetMainSubmixAnalyzersCmd()
	{
		if (ResetMainSubmixAnalyzersCmd == nullptr)
		{
			ResetMainSubmixAnalyzersCmd = IConsoleManager::Get().RegisterConsoleCommand(
				*FString::Printf(TEXT("au.ResetMainSubmixAnalyzers_AD%d"), DeviceID),
				TEXT("Resets the main submix LKFS and TruePeak analyzers"),
				FConsoleCommandDelegate::CreateRaw(this, &FMixerDevice::OnResetMainSubmixAnalyzersCmdInvoked),
				ECVF_Default
			);
		}
	}

	void FMixerDevice::UnregisterResetMainSubmixAnalyzersCmd()
	{
		IConsoleManager::Get().UnregisterConsoleObject(ResetMainSubmixAnalyzersCmd);
		ResetMainSubmixAnalyzersCmd = nullptr;
	}

	void FMixerDevice::OnResetMainSubmixAnalyzersCmdInvoked()
	{
		ensure(IsInGameThread());

		FAudioThread::RunCommandOnAudioThread([this]()
		{
			ResetMainSubmixAnalyzers();
		});
	}

	void FMixerDevice::ResetMainSubmixAnalyzers()
	{
		ensure(IsInAudioThread());

		const int32 NumSamples = PlatformInfo.NumChannels * PlatformSettings.CallbackBufferFrameSize;

		MainSubmixLKFSAnalyzer.Reset();
		InitializeMainSubmixLKFSAnalyzer(PlatformSettings.SampleRate, PlatformInfo.NumChannels);

		MainSubmixTruePeakEnvelopeFollower.Reset();
		InitializeMainSubmixTruePeakEnvelopeFollower(PlatformSettings.SampleRate, PlatformInfo.NumChannels, NumSamples);
	}
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

	void FMixerDevice::AddReferencedObjects(FReferenceCollector& Collector)
	{
	}

	void FMixerDevice::CheckAudioThread() const
	{
#if AUDIO_MIXER_ENABLE_DEBUG_MODE
		// "Audio Thread" is the game/audio thread ID used above audio rendering thread.
		AUDIO_MIXER_CHECK(IsInAudioThread());
#endif
	}

	void FMixerDevice::OnListenerUpdated(const TArray<FListener>& InListeners)
	{
		LLM_SCOPE(ELLMTag::AudioMixer);

		ListenerTransforms.Reset(InListeners.Num());

		for (const FListener& Listener : InListeners)
		{
			ListenerTransforms.Add(Listener.Transform);
		}

		SourceManager->SetListenerTransforms(ListenerTransforms);
	}

	void FMixerDevice::ResetAudioRenderingThreadId()
	{
		AudioPlatformThreadId = INDEX_NONE;
		CheckAudioRenderingThread();
	}

	void FMixerDevice::CheckAudioRenderingThread() const
	{
		if (AudioPlatformThreadId == INDEX_NONE)
		{
			AudioPlatformThreadId = FPlatformTLS::GetCurrentThreadId();
		}
		int32 CurrentThreadId = FPlatformTLS::GetCurrentThreadId();
		AUDIO_MIXER_CHECK(CurrentThreadId == AudioPlatformThreadId);
	}

	bool FMixerDevice::IsAudioRenderingThread() const
	{
		int32 CurrentThreadId = FPlatformTLS::GetCurrentThreadId();
		return CurrentThreadId == AudioPlatformThreadId;
	}

	bool FMixerDevice::IsNonRealtime() const
	{
		return AudioMixerPlatform && AudioMixerPlatform->IsNonRealtime();
	}

	TArray<Audio::FChannelPositionInfo>* FMixerDevice::GetDefaultPositionMap(int32 NumChannels)
	{
		const Audio::FChannelPositionInfo* SpeakerPositions = GetDefaultChannelPositions();

		if (!SpeakerPositions) // speaker maps are not yet initialized
		{
			return nullptr;
		}

		switch (NumChannels)
		{
			// Mono speaker directly in front of listener:
			case 1:
			{
				// force angle on single channel if we are mono
				static TArray<Audio::FChannelPositionInfo> MonoMap = { {EAudioMixerChannel::FrontCenter, 0, 0} };
				return &MonoMap;
			}

			// Stereo speakers to front left and right of listener:
			case 2:
			{
				static TArray<Audio::FChannelPositionInfo> StereoMap = { SpeakerPositions[EAudioMixerChannel::FrontLeft], SpeakerPositions[EAudioMixerChannel::FrontRight] };
				return &StereoMap;
			}

			// Quadrophonic speakers at each corner.
			case 4:
			{
				static TArray<Audio::FChannelPositionInfo> QuadMap = {
														 SpeakerPositions[EAudioMixerChannel::FrontLeft] //left
														,SpeakerPositions[EAudioMixerChannel::FrontRight] // right
														,SpeakerPositions[EAudioMixerChannel::SideLeft] //Left Surround
														,SpeakerPositions[EAudioMixerChannel::SideRight] //Right Surround
				};
				return &QuadMap;
			}

			// 5.1 speakers.
			case 6:
			{
				static TArray<Audio::FChannelPositionInfo> FiveDotOneMap = {
														 SpeakerPositions[EAudioMixerChannel::FrontLeft] //left
														,SpeakerPositions[EAudioMixerChannel::FrontRight] // right
														,SpeakerPositions[EAudioMixerChannel::FrontCenter] //center
														,SpeakerPositions[EAudioMixerChannel::LowFrequency] //LFE
														,SpeakerPositions[EAudioMixerChannel::SideLeft] //Left Rear
														,SpeakerPositions[EAudioMixerChannel::SideRight] //Right Rear
				};
				return &FiveDotOneMap;
			}

			// 7.1 speakers.
			case 8:
			{
				static TArray<Audio::FChannelPositionInfo> SevenDotOneMap = {
														 SpeakerPositions[EAudioMixerChannel::FrontLeft] // left
														,SpeakerPositions[EAudioMixerChannel::FrontRight] // right
														,SpeakerPositions[EAudioMixerChannel::FrontCenter] //center
														,SpeakerPositions[EAudioMixerChannel::LowFrequency] //LFE
														,SpeakerPositions[EAudioMixerChannel::BackLeft] // Left Rear
														,SpeakerPositions[EAudioMixerChannel::BackRight] // Right Rear
														,SpeakerPositions[EAudioMixerChannel::SideLeft] // Left Surround
														,SpeakerPositions[EAudioMixerChannel::SideRight] // Right Surround
				};


				return &SevenDotOneMap;
			}

			case 0:
			default:
			{
				return nullptr;
			}
		}
	}

	bool FMixerDevice::IsEndpointSubmix(const USoundSubmixBase* InSubmix)
	{
		return InSubmix && (InSubmix->IsA<UEndpointSubmix>() || InSubmix->IsA<USoundfieldEndpointSubmix>());
	}

	void FMixerDevice::UpdateDeviceDeltaTime()
	{
		DeviceDeltaTime = GetGameDeltaTime();
	}

	void FMixerDevice::GetAudioDeviceList(TArray<FString>& OutAudioDeviceNames) const
	{
		if (AudioMixerPlatform && AudioMixerPlatform->IsInitialized())
		{
			uint32 NumOutputDevices;
			if (AudioMixerPlatform->GetNumOutputDevices(NumOutputDevices))
			{
				for (uint32 i = 0; i < NumOutputDevices; ++i)
				{
					FAudioPlatformDeviceInfo DeviceInfo;
					if (AudioMixerPlatform->GetOutputDeviceInfo(i, DeviceInfo))
					{
						OutAudioDeviceNames.Add(DeviceInfo.Name);
					}
				}
			}
		}
	}

	bool FMixerDevice::InitializeHardware()
	{
		ensure(IsInGameThread());
	
		LLM_SCOPE(ELLMTag::AudioMixer);


		if (AudioMixerPlatform && AudioMixerPlatform->InitializeHardware())
		{
			UE_LOGF(LogAudioMixer, Display, "Initializing audio mixer using platform API: '%ls'", *AudioMixerPlatform->GetPlatformApi());

			// Issue a log if the audio engine will be muted.
#if !UE_BUILD_SHIPPING
			if (bMuteAudioCVar || FApp::IsAudioMuted())
			{
				UE_LOGF(LogAudioMixer, Display, "Audio output will be muted. If this not intentional, check for au.MuteAudio cvar or -muteaudio command line.");
			}
#endif // !UE_BUILD_SHIPPING

			const UAudioSettings* AudioSettings = GetDefault<UAudioSettings>();
			MonoChannelUpmixMethod = AudioSettings->MonoChannelUpmixMethod;
			PanningMethod = AudioSettings->PanningMethod;

			// Set whether we're the main audio mixer
			bIsMainAudioMixer = IsMainAudioDevice();

			AUDIO_MIXER_CHECK(SampleRate != 0.0f);

			AudioMixerPlatform->RegisterDeviceChangedListener();

			// Allow platforms to override the platform settings callback buffer frame size (i.e. restrict to particular values, etc)
			PlatformSettings.CallbackBufferFrameSize = AudioMixerPlatform->GetNumFrames(PlatformSettings.CallbackBufferFrameSize);

			OpenStreamParams.NumBuffers = PlatformSettings.NumBuffers;
			OpenStreamParams.NumFrames = PlatformSettings.CallbackBufferFrameSize;
			OpenStreamParams.OutputDeviceIndex = AUDIO_MIXER_DEFAULT_DEVICE_INDEX; // TODO: Support overriding which audio device user wants to open, not necessarily default.
			OpenStreamParams.SampleRate = SampleRate;
			OpenStreamParams.AudioMixer = this;
			OpenStreamParams.MaxSources = GetMaxSources();
#if WITH_EDITOR
			{
				IAudioEditorModule& AudioEditorModule = FModuleManager::LoadModuleChecked<IAudioEditorModule>("AudioEditor");
				TOptional<FAudioEditorDeviceSettings> DeviceSettings = AudioEditorModule.GetAudioEditorDeviceSettings();
				if (DeviceSettings.IsSet())
				{
					OpenStreamParams.bUseSystemAudioDevice = DeviceSettings->bUseSystemDevice;
					OpenStreamParams.AudioDeviceId = DeviceSettings->DeviceId;
				}
			}
#endif //WITH_EDITOR

			FString DefaultDeviceName = AudioMixerPlatform->GetDefaultDeviceName();

			// Allow HMD to specify audio device, if one was not specified in settings
			if (DefaultDeviceName.IsEmpty() && FAudioDevice::CanUseVRAudioDevice() && IHeadMountedDisplayModule::IsAvailable())
			{
				DefaultDeviceName = IHeadMountedDisplayModule::Get().GetAudioOutputDevice();
			}

			FString DeviceToOpen = DefaultDeviceName;
			if (!OpenStreamParams.bUseSystemAudioDevice)
			{
				DeviceToOpen = OpenStreamParams.AudioDeviceId;
			}
			
			if (!DeviceToOpen.IsEmpty())
			{
				uint32 NumOutputDevices = 0;
				AudioMixerPlatform->GetNumOutputDevices(NumOutputDevices);

				for (uint32 i = 0; i < NumOutputDevices; ++i)
				{
					FAudioPlatformDeviceInfo DeviceInfo;
					AudioMixerPlatform->GetOutputDeviceInfo(i, DeviceInfo);

					if (DeviceInfo.Name == DeviceToOpen || DeviceInfo.DeviceId == DeviceToOpen)
					{
						OpenStreamParams.OutputDeviceIndex = i;

						// If we're intentionally selecting an audio device (and not just using the default device) then 
						// lets try to restore audio to that device if it's removed and then later is restored
						OpenStreamParams.bRestoreIfRemoved = true;
						break;
					}
				}
			}

			if (AudioMixerPlatform->OpenAudioStream(OpenStreamParams))
			{
				// Get the platform device info we're using
				PlatformInfo = AudioMixerPlatform->GetPlatformDeviceInfo();
				UE_LOGF(LogAudioMixer, Display, "Using Audio Hardware Device %ls", *PlatformInfo.Name);

				// Initialize some data that depends on speaker configuration, etc.
				InitializeChannelAzimuthMap(PlatformInfo.NumChannels);

				FSourceManagerInitParams SourceManagerInitParams;
				SourceManagerInitParams.NumSources = GetMaxSources();
				SourceManagerInitParams.NumGeneratorSources = AudioMixerGeneratorSourcesCVar;

				AudioClock = 0.0;
				AudioClockDelta = (double)OpenStreamParams.NumFrames / OpenStreamParams.SampleRate;
				AudioClockTimingData.UpdateTime = 0.0;

#if UE_AUDIO_PROFILERTRACE_ENABLED
				SubmixesAlivePingTimeSec = 0.0;
#endif //UE_AUDIO_PROFILERTRACE_ENABLED

				PluginInitializationParams.NumSources = SourceManagerInitParams.NumSources;
				PluginInitializationParams.SampleRate = SampleRate;
				PluginInitializationParams.BufferLength = OpenStreamParams.NumFrames;
				PluginInitializationParams.AudioDevicePtr = this;

				{
					LLM_SCOPE(ELLMTag::AudioMixerPlugins);

					// Initialize any plugins if they exist
					// spatialization
					SetCurrentSpatializationPlugin(AudioPluginUtilities::GetDesiredSpatializationPluginName());

					if (OcclusionInterface.IsValid())
					{
						OcclusionInterface->Initialize(PluginInitializationParams);
					}

					if (ReverbPluginInterface.IsValid())
					{
						ReverbPluginInterface->Initialize(PluginInitializationParams);
					}

					if (SourceDataOverridePluginInterface.IsValid())
 					{
 						SourceDataOverridePluginInterface->Initialize(PluginInitializationParams);
 					}
				}

				// initialize the source manager after our plugins are spun up (cached by sources)
				SourceManager->Init(SourceManagerInitParams);

				// Need to set these up before we start the audio stream.
				InitSoundSubmixes();

#if UE_AUDIO_PROFILERTRACE_ENABLED
				const int32 NumSamples = PlatformInfo.NumChannels * PlatformSettings.CallbackBufferFrameSize;

				InitializeMainSubmixTruePeakEnvelopeFollower(PlatformSettings.SampleRate, PlatformInfo.NumChannels, NumSamples);
				InitializeMainSubmixLKFSAnalyzer(PlatformSettings.SampleRate, PlatformInfo.NumChannels);

				RegisterSubmixEnvelopeFollowerCVars();
				RegisterResetMainSubmixAnalyzersCmd();
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

				AudioMixerPlatform->PostInitializeHardware();

				// Initialize the data used for audio thread sub-frame timing.
				AudioThreadTimingData.StartTime = FPlatformTime::Seconds();
				AudioThreadTimingData.AudioThreadTime = 0.0;
				AudioThreadTimingData.AudioRenderThreadTime = 0.0;

				Audio::Analytics::RecordEvent_Usage(TEXT("ProjectSettings"), MakeAnalyticsEventAttributeArray(
					TEXT("SampleRate"), PlatformSettings.SampleRate,
					TEXT("BufferSize"), PlatformSettings.CallbackBufferFrameSize,
					TEXT("NumBuffers"), PlatformSettings.NumBuffers,
					TEXT("NumSources"), PlatformSettings.MaxChannels,
					TEXT("NumOutputChannels"), PlatformInfo.NumChannels));

				// Start streaming audio
				return AudioMixerPlatform->StartAudioStream();
			}
		}
		else if (AudioMixerPlatform)
		{
			UE_LOGF(LogAudioMixer, Warning, "Failed to initialize audio mixer for platform API: '%ls'", *AudioMixerPlatform->GetPlatformApi());
		}

		return false;
	}

	void FMixerDevice::FadeIn()
	{
		AudioMixerPlatform->FadeIn();
	}

	void FMixerDevice::FadeOut()
	{
		// In editor builds, we aren't going to fade out the main audio device.
#if WITH_EDITOR
		if (!IsMainAudioDevice())
#endif
		{
			AudioMixerPlatform->FadeOut();
		}
	}

	void FMixerDevice::TeardownHardware()
	{
		ensure(IsInGameThread());

		if (IsInitialized())
		{
#if UE_AUDIO_PROFILERTRACE_ENABLED
			UnregisterSubmixEnvelopeFollowerCVars();
			UnregisterResetMainSubmixAnalyzersCmd();
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

			for (TObjectIterator<USoundSubmix> It; It; ++It)
			{
				UnregisterSoundSubmix(*It, true);
			}
			
		}
		
		// reset all the sound effect presets loaded
#if WITH_EDITOR
		for (TObjectIterator<USoundEffectPreset> It; It; ++It)
		{
			USoundEffectPreset* SoundEffectPreset = *It;
			SoundEffectPreset->Init();
		}
#endif

		if (AudioMixerPlatform)
		{
			SourceManager->Update();

			AudioMixerPlatform->UnregisterDeviceChangedListener();
			AudioMixerPlatform->StopAudioStream();
			AudioMixerPlatform->CloseAudioStream();
			AudioMixerPlatform->TeardownHardware();
		}

#if UE_AUDIO_PROFILERTRACE_ENABLED
		MainSubmixTruePeakEnvelopeFollower.Reset();
		MainSubmixLKFSAnalyzer.Reset();
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

		// Reset existing submixes if they exist
		RequiredSubmixInstances.Reset();
		Submixes.Reset();
	}

	void FMixerDevice::UpdateHardwareTiming()
	{
		// Get the relative audio thread time (from start of audio engine)
		// Add some jitter delta to account for any audio thread timing jitter.
		const double AudioThreadJitterDelta = AudioClockDelta;
		AudioThreadTimingData.AudioThreadTime = FPlatformTime::Seconds() - AudioThreadTimingData.StartTime + AudioThreadJitterDelta;
	}

	void FMixerDevice::UpdateGameThread()
	{
		LLM_SCOPE(ELLMTag::AudioMixer);

		// Pump our command queue sending commands to the game thread
		PumpGameThreadCommandQueue();
	}

	void FMixerDevice::UpdateHardware()
	{
		LLM_SCOPE(ELLMTag::AudioMixer);

		// If we're in editor, re-query these in case they changed. 
#if WITH_EDITOR
		{
			const UAudioSettings* AudioSettings = GetDefault<UAudioSettings>();
			PanningMethod = AudioSettings->PanningMethod;
		}
#endif

		SourceManager->Update();

		if (SourceManager->CheckAndResetHardwareCallbackStallFlag())
		{
			// This is a foreground thread check if we are not receiving hardware callbacks. 
			AudioMixerPlatform->RequestDeviceSwap(TEXT(""), /* force */true, TEXT("FMixerDevice::UpdateHardware(). Force device swap because hardware has stalled."));
		}

		AudioMixerPlatform->OnHardwareUpdate();

		// Device swaps span multiple calls beginning with CheckAudioDeviceChange() and ending with
		// ResumePlaybackOnNewDevice(). We protect the associated resources by locking and unlocking around
		// the entire call chain once the swap has begun.
		if (AudioMixerPlatform->CheckAudioDeviceChange())
		{
			FScopedDeviceSwapLock Lock(AudioMixerPlatform);

			// Get the platform device info we're using
			PlatformInfo = AudioMixerPlatform->GetPlatformDeviceInfo();

			// Initialize some data that depends on speaker configuration, etc.
			InitializeChannelAzimuthMap(PlatformInfo.NumChannels);

			// Update the channel device count in case it changed
			SourceManager->UpdateDeviceChannelCount(PlatformInfo.NumChannels);

			// Reset rendering thread ID to this thread ID so that commands can
			// be flushed. Audio Rendering Thread ID will be reset again in call
			// to FMixerDevice::OnProcessAudio
			ResetAudioRenderingThreadId();

			// Cache the audio platform thread id for debugging purposes. This is
			// an attempt to narrow down the causes of UE-209237. The theory is
			// that the there are multiple threads attempting to run audio rendering
			// commands. To catch the issue we attempt to cache the thread id before
			// running audio rendering commands. We then check that the thread id
			// has not changed for the duration that render thread commands have 
			// run.
			std::atomic<int32> CurrentAudioPlatformThreadId = AudioPlatformThreadId.load();

			// Force source manager to incorporate device channel count change.
			// The render thread is stopped at this point, so skip waiting on it
			// to avoid a ~1s timeout per Wait() call in FlushCommandQueue.
			FlushAudioRenderingCommands(true /* bPumpSynchronously */, true /* bRenderThreadStopped */);

			if (UAudioDeviceNotificationSubsystem* AudioDeviceNotifSubsystem = UAudioDeviceNotificationSubsystem::Get())
			{
				AudioDeviceNotifSubsystem->OnDeviceSwitched(PlatformInfo.DeviceId);
			}

			// Related to earlier mention of UE-209237
			UE_CLOGF(CurrentAudioPlatformThreadId != AudioPlatformThreadId, LogAudioMixer, Error, "Platform audio thread id changed while flushing render commands. Expected %d, found %d. May result in corrupt internal audio source state.", CurrentAudioPlatformThreadId.load(), AudioPlatformThreadId.load());

			// Audio rendering was suspended in CheckAudioDeviceChange if it changed.
			AudioMixerPlatform->ResumePlaybackOnNewDevice();
		}

		// Device must be initialized prior to call as submix graph may not be ready yet otherwise.
		if (IsInitialized())
		{
			// Loop through any envelope-following submixes and perform any broadcasting of envelope data if needed
			TArray<float> SubmixEnvelopeData;
			for (USoundSubmix* SoundSubmix : DelegateBoundSubmixes)
			{
				if (SoundSubmix)
				{
					// Retrieve the submix instance and the envelope data and broadcast on the audio thread.
					Audio::FMixerSubmixWeakPtr SubmixPtr = GetSubmixInstance(SoundSubmix);
					if (SubmixPtr.IsValid())
					{
						FAudioThread::RunCommandOnGameThread([this, SubmixPtr]()
							{
								Audio::FMixerSubmixPtr ThisSubmixPtr = SubmixPtr.Pin();
								if (ThisSubmixPtr.IsValid())
								{
									ThisSubmixPtr->BroadcastDelegates();
								}
							});
					}
				}
			}

			// Check if the background mute changed state and update the submixes which are enabled to do background muting.
			const float CurrentPrimaryVolume = GetPrimaryVolume();
			if (!FMath::IsNearlyEqual(PreviousPrimaryVolume, CurrentPrimaryVolume))
			{
				PreviousPrimaryVolume = CurrentPrimaryVolume;
				bool IsMuted = FMath::IsNearlyZero(CurrentPrimaryVolume);

				for (TObjectIterator<USoundSubmix> It; It; ++It)
				{
					if (It->bMuteWhenBackgrounded)
					{
						FMixerSubmixPtr SubmixInstance = GetSubmixInstance(*It).Pin();
						if (SubmixInstance.IsValid())
						{
							SubmixInstance->SetBackgroundMuted(IsMuted);
						}
					}
				}
			}
		}

#if ENABLE_AUDIO_DEBUG
		UpdateSubmixDebugVolumes();
#endif // ENABLE_AUDIO_DEBUG
	}

	double FMixerDevice::GetAudioTime() const
	{
		return AudioClock;
	}

	double FMixerDevice::GetInterpolatedAudioClock() const
	{
		return AudioClockTimingData.GetInterpolatedAudioClock(AudioClock, AudioClockDelta);
	}

	FAudioEffectsManager* FMixerDevice::CreateEffectsManager()
	{
		return new FAudioMixerEffectsManager(this);
	}

	FSoundSource* FMixerDevice::CreateSoundSource()
	{
		return new FMixerSource(this);
	}

	bool FMixerDevice::HasCompressedAudioInfoClass(USoundWave* InSoundWave)
	{
		check(InSoundWave);
		check(AudioMixerPlatform);
		// Every platform has compressed audio. 
		return true;
	}
	
	bool FMixerDevice::ValidateAPICall(const TCHAR* Function, uint32 ErrorCode)
	{
		return false;
	}

#if UE_ALLOW_EXEC_COMMANDS
	bool FMixerDevice::Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
	{
		if (FAudioDevice::Exec(InWorld, Cmd, Ar))
		{
			return true;
		}

		return false;
	}
#endif // UE_ALLOW_EXEC_COMMANDS

	void FMixerDevice::CountBytes(FArchive& InArchive)
	{
		FAudioDevice::CountBytes(InArchive);
	}

	bool FMixerDevice::IsExernalBackgroundSoundActive()
	{
		return false;
	}

	void FMixerDevice::ResumeContext()
	{
        AudioMixerPlatform->ResumeContext();
	}

	void FMixerDevice::SuspendContext()
	{
        AudioMixerPlatform->SuspendContext();
	}

	void FMixerDevice::EnableDebugAudioOutput()
	{
		bDebugOutputEnabled = true;
	}

	bool FMixerDevice::OnProcessAudioStream(FAlignedFloatBuffer& Output)
	{
		LLM_SCOPE(ELLMTag::AudioMixer);

		// This function could be called in a task manager, which means the thread ID may change between calls.
		ResetAudioRenderingThreadId();

		// Cache the audio platform thread id for debugging purposes. This is
		// an attempt to narrow down the causes of UE-209237. The theory is
		// that the there are multiple threads attempting to run audio rendering
		// commands. To catch the issue we attempt to cache the thread id before
		// running audio rendering commands. We then check that the thread id
		// has not changed for the duration that render thread commands have 
		// run.
		std::atomic<int32> CurrentAudioPlatformThreadId = AudioPlatformThreadId.load();

		// Update the audio render thread time at the head of the render
		AudioThreadTimingData.AudioRenderThreadTime = FPlatformTime::Seconds() - AudioThreadTimingData.StartTime;

		// notify interested parties
		FAudioDeviceRenderInfo RenderInfo;
		RenderInfo.NumFrames = SourceManager->GetNumOutputFrames();
		NotifyAudioDevicePreRender(RenderInfo);

		// Pump the command queue to the audio render thread
		PumpCommandQueue();

		// update the clock manager
		QuantizedEventClockManager.Update(SourceManager->GetNumOutputFrames());

		// Compute the next block of audio in the source manager
		SourceManager->ComputeNextBlockOfSamples();

		FMixerSubmixWeakPtr MainSubmix = GetMasterSubmix();
		{
			CSV_SCOPED_TIMING_STAT(Audio, Submixes);
			SCOPE_CYCLE_COUNTER(STAT_AudioMixerSubmixes);

			FMixerSubmixPtr MainSubmixPtr = MainSubmix.Pin();
			if (MainSubmixPtr.IsValid())
			{
				// Process the audio output from the master submix
				MainSubmixPtr->ProcessAudio(Output);

#if !UE_BUILD_SHIPPING
				// This is done at the very end here to avoid changing the behavior in any other way (analysis, profiling, etc)
				if ((bMuteAudioCVar || FApp::IsAudioMuted()))
				{
					FMemory::Memzero((void*)Output.GetData(), sizeof(float) * Output.Num());
				}
#endif // !UE_BUILD_SHIPPING
			}
		}

		// Only after submixes have finished rendering audio, update the source state in the source manager to reflect sound-doneness, etc
		SourceManager->UpdateSourceState();

		{
			CSV_SCOPED_TIMING_STAT(Audio, EndpointSubmixes);
			SCOPE_CYCLE_COUNTER(STAT_AudioMixerEndpointSubmixes);

			FScopeLock ScopeLock(&EndpointSubmixesMutationLock);
			if (EnableAudibleDefaultEndpointSubmixesCVar !=0 )
			{
				for (const FMixerSubmixPtr& Submix : DefaultEndpointSubmixes)
				{
					// If this hit, a submix was added to the default submix endpoint array
					// even though it's not an endpoint, or a parent was set on an endpoint submix
					// and it wasn't removed from DefaultEndpointSubmixes.
					ensure(Submix->IsDefaultEndpointSubmix());

					// Any endpoint submixes that don't specify an endpoint
					// are summed into our master output.
					Submix->ProcessAudio(Output);
				}
			}
			
			for (FMixerSubmixPtr& Submix : ExternalEndpointSubmixes)
			{
				// If this hit, a submix was added to the external submix endpoint array
				// even though it's not an endpoint, or a parent was set on an endpoint submix
				// and it wasn't removed from ExternalEndpointSubmixes.
				ensure(Submix->IsExternalEndpointSubmix());

				Submix->ProcessAudioAndSendToEndpoint();
			}
		}

		// Reset stopping sounds and clear their state after submixes have been mixed
		SourceManager->ClearStoppingSounds();

		// Do any debug output performing
		if (bDebugOutputEnabled || DebugGeneratorEnableCVar > 0)
		{
			if (DebugGeneratorEnableCVar < 2)
			{
				SineOscTest(Output);
			}
			else
			{
				WhiteNoiseTest(Output);
			}
		}

		// Update the audio clock
		UpdateAudioClock();

#if UE_AUDIO_PROFILERTRACE_ENABLED
		UpdateAndSendMainSubmixTruePeakMaxValueMessage(Output);
		UpdateAndSendMainSubmixLKFSValuesMessage(Output);

		UpdateSubmixesAlivePingTime();
		SendSubmixesSendLevelTraceMessages();
#endif //UE_AUDIO_PROFILERTRACE_ENABLED

		// notify interested parties
		NotifyAudioDevicePostRender(RenderInfo);

		// Related to earlier mention of UE-209237
		UE_CLOGF(CurrentAudioPlatformThreadId != AudioPlatformThreadId, LogAudioMixer, Error, "Platform audio thread id changed while flushing render commands. Expected %d, found %d. May result in corrupt internal audio source state.", CurrentAudioPlatformThreadId.load(), AudioPlatformThreadId.load());


		return true;
	}

	void FMixerDevice::UpdateAudioClock()
	{
		AudioClock += AudioClockDelta;
		AudioClockTimingData.UpdateTime = FPlatformTime::Seconds();
	}

	void FMixerDevice::OnAudioStreamShutdown()
	{
		// Make sure the source manager pumps any final commands on shutdown. These allow for cleaning up sources, interfacing with plugins, etc.
		// Because we double buffer our command queues, we call this function twice to ensure all commands are successfully pumped.
		SourceManager->PumpCommandQueue();
		SourceManager->PumpCommandQueue();

		// Make sure we force any pending release data to happen on shutdown
		SourceManager->UpdatePendingReleaseData(true);
	}

	void FMixerDevice::LoadRequiredSubmix(ERequiredSubmixes InType, const FString& InDefaultName, bool bInDefaultMuteWhenBackgrounded, FSoftObjectPath& InObjectPath) 
	{
		check(IsInGameThread());

		const int32 RequiredSubmixCount = static_cast<int32>(ERequiredSubmixes::Count);
		if(RequiredSubmixes.Num() < RequiredSubmixCount)
		{
			RequiredSubmixes.AddZeroed(RequiredSubmixCount - RequiredSubmixes.Num());
		}

		if (RequiredSubmixInstances.Num() < RequiredSubmixCount)
		{
			RequiredSubmixInstances.AddZeroed(RequiredSubmixCount - RequiredSubmixInstances.Num());
		}

		const int32 TypeIndex = static_cast<int32>(InType);
		if (USoundSubmix* OldSubmix = RequiredSubmixes[TypeIndex])
		{
			// Don't bother swapping if new path is invalid...
			if (!InObjectPath.IsValid())
			{
				return;
			}

			// or is same object already initialized.
			if (InObjectPath.GetAssetPathString() == OldSubmix->GetPathName())
			{
				return;
			}
			OldSubmix->RemoveFromRoot();
			FMixerSubmixPtr OldSubmixPtr = RequiredSubmixInstances[TypeIndex];
			if (OldSubmixPtr.IsValid())
			{
				FMixerSubmixPtr ParentSubmixPtr = RequiredSubmixInstances[TypeIndex]->GetParentSubmix().Pin();
				if (ParentSubmixPtr.IsValid())
				{
					ParentSubmixPtr->RemoveChildSubmix(RequiredSubmixInstances[TypeIndex]);
				}
			}
		}

		// 1. Try loading from Developer Audio Settings
		USoundSubmix* NewSubmix = Cast<USoundSubmix>(InObjectPath.TryLoad());

		// 2. If Unset or not found, fallback to engine asset
		if (!NewSubmix)
		{
			static const FString EngineSubmixDir = "/Engine/EngineSounds/Submixes";

			InObjectPath = FString::Printf(TEXT("%s/%s.%s"), *EngineSubmixDir, *InDefaultName, *InDefaultName);
			NewSubmix = Cast<USoundSubmix>(InObjectPath.TryLoad());
			UE_LOGF(LogAudioMixer, Display, "Submix unset or invalid in 'AudioSettings': Using engine asset '%ls'",
				*InObjectPath.GetAssetPathString());
		}

		// 3. If engine version not found, dynamically spawn and post error
		if (!NewSubmix)
		{
			UE_LOGF(LogAudioMixer, Error, "Failed to load submix from engine asset path '%ls'. Creating '%ls' as a stub.",
				*InObjectPath.GetAssetPathString(),
				*InDefaultName);

			NewSubmix = NewObject<USoundSubmix>(USoundSubmix::StaticClass(), *InDefaultName);
			// Make the master reverb mute when backgrounded
			NewSubmix->bMuteWhenBackgrounded = bInDefaultMuteWhenBackgrounded;
		}

		check(NewSubmix);
		NewSubmix->AddToRoot();

		// If sharing submix with other explicitly defined MainSubmix, create
		// shared pointer directed to already existing submix instance. Otherwise,
		// create a new version.
		FMixerSubmixPtr NewMixerSubmix = GetRequiredSubmixInstance(NewSubmix);
		if (!NewMixerSubmix.IsValid())
		{
			UE_LOGF(LogAudioMixer, Display, "Creating Master Submix '%ls'", *NewSubmix->GetName());
			NewMixerSubmix = MakeShared<FMixerSubmix, ESPMode::ThreadSafe>(this);
		}

		// Ensure that master submixes are ONLY tracked in master submix array.
		// RequiredSubmixes array can share instances, but should not be duplicated in Submixes Map.
		if (Submixes.Remove(NewSubmix->GetUniqueID()) > 0)
		{
			UE_LOGF(LogAudioMixer, Display, "Submix '%ls' has been promoted to master array.", *NewSubmix->GetName());
		}

		// Update/add new submix and instance to respective master arrays
		RequiredSubmixes[TypeIndex] = NewSubmix;
		RequiredSubmixInstances[TypeIndex] = NewMixerSubmix;

		//Note: If we support using endpoint/soundfield submixes as a master submix in the future, we will need to call NewMixerSubmix->SetSoundfieldFactory here.
		NewMixerSubmix->Init(NewSubmix, false /* bAllowReInit */);
	}

	void FMixerDevice::LoadPluginSoundSubmixes()
	{
		check(IsInGameThread());

		if (IsReverbPluginEnabled() && ReverbPluginInterface)
		{
			LLM_SCOPE(ELLMTag::AudioMixerPlugins);
			USoundSubmix* ReverbPluginSubmix = ReverbPluginInterface->LoadSubmix();
			check(ReverbPluginSubmix);
			
			// This AddToRoot is redundant since we AddToRoot in the LoadSoundSubmix stack
			if (!GRemoveSubmixesFromRoot)
			{
				ReverbPluginSubmix->AddToRoot();
			}

			LoadSoundSubmix(*ReverbPluginSubmix);
			InitSoundfieldAndEndpointDataForSubmix(*ReverbPluginSubmix, GetSubmixInstance(ReverbPluginSubmix).Pin(), false);

			// Plugin must provide valid effect to enable reverb
			FSoundEffectSubmixPtr ReverbPluginEffectSubmix = ReverbPluginInterface->GetEffectSubmix();
			

			if (ReverbPluginEffectSubmix.IsValid())
			{
				if (USoundEffectPreset* Preset = ReverbPluginEffectSubmix->GetPreset())
				{
					FMixerSubmixPtr ReverbPluginMixerSubmixPtr = GetSubmixInstance(ReverbPluginSubmix).Pin();
					check(ReverbPluginMixerSubmixPtr.IsValid());

					const TWeakObjectPtr<USoundSubmix> ReverbPluginSubmixPtr = ReverbPluginSubmix;
					FMixerSubmixWeakPtr ReverbPluginMixerSubmixWeakPtr = ReverbPluginMixerSubmixPtr;
					AudioRenderThreadCommand([ReverbPluginMixerSubmixWeakPtr, ReverbPluginSubmixPtr, ReverbPluginEffectSubmix]()
					{
						FMixerSubmixPtr PluginSubmixPtr = ReverbPluginMixerSubmixWeakPtr.Pin();
						if (PluginSubmixPtr.IsValid() && ReverbPluginSubmixPtr.IsValid())
						{
							PluginSubmixPtr->ReplaceSoundEffectSubmix(0, ReverbPluginEffectSubmix);
						}
					});
				}
			}
			else
			{
				UE_LOGF(LogAudioMixer, Error, "Reverb plugin failed to provide valid effect submix.  Plugin audio processing disabled.");
			}
		}
	}

	static inline FString GetRequiredSubmixName(const ERequiredSubmixes InRequiredSubmix)
	{
		switch (InRequiredSubmix) 
		{
			case ERequiredSubmixes::Main: { return TEXT("MasterSubmixDefault"); }
			case ERequiredSubmixes::BaseDefault: { return TEXT("BaseDefault"); }
			case ERequiredSubmixes::Reverb: { return TEXT("MasterReverbSubmixDefault"); }
			case ERequiredSubmixes::EQ: { return TEXT("MasterEQSubmixDefault"); }
			case ERequiredSubmixes::Count: break;
		}

		UE_LOGF(LogAudioMixer, Warning, "Unsupported submix name for provided ERequiredSubmix");
		return "";
	}

	void FMixerDevice::InitSoundSubmixes()
	{
		if (IsInGameThread())
		{
			bSubmixRegistrationDisabled = true;

			UAudioSettings* AudioSettings = GetMutableDefault<UAudioSettings>();
			check(AudioSettings);

			if (RequiredSubmixes.Num() > 0)
			{
				UE_LOGF(LogAudioMixer, Display, "Re-initializing Sound Submixes...");
			}
			else
			{
				UE_LOGF(LogAudioMixer, Display, "Initializing Sound Submixes...");
			}

			// 1. Load or reload all sound submixes/instances
			LoadRequiredSubmix(ERequiredSubmixes::Main, GetRequiredSubmixName(ERequiredSubmixes::Main), false /* DefaultMuteWhenBackgrounded */, AudioSettings->MasterSubmix);

#if UE_AUDIO_PROFILERTRACE_ENABLED
			SendMainSubmixLoadedMessage();
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

			// BaseDefaultSubmix is an optional master submix type set by project settings
			if (AudioSettings->BaseDefaultSubmix.IsValid())
			{
				LoadRequiredSubmix(ERequiredSubmixes::BaseDefault, GetRequiredSubmixName(ERequiredSubmixes::BaseDefault), false /* DefaultMuteWhenBackgrounded */, AudioSettings->BaseDefaultSubmix);
			}

			LoadRequiredSubmix(ERequiredSubmixes::Reverb, GetRequiredSubmixName(ERequiredSubmixes::Reverb), true /* DefaultMuteWhenBackgrounded */, AudioSettings->ReverbSubmix);

			if (!DisableSubmixEffectEQCvar)
			{
				LoadRequiredSubmix(ERequiredSubmixes::EQ, GetRequiredSubmixName(ERequiredSubmixes::EQ), false /* DefaultMuteWhenBackgrounded */, AudioSettings->EQSubmix);
			}

			LoadPluginSoundSubmixes();

			for (TObjectIterator<USoundSubmixBase> It; It; ++It)
			{
				USoundSubmixBase* SubmixToLoad = *It;
				check(SubmixToLoad);

				if (!IsRequiredSubmixType(SubmixToLoad) && !SubmixToLoad->IsDynamic( true /* bIncludeAncestors */) ) // Do not load dynamic submixes until they've been connected.
				{
					LoadSoundSubmix(*SubmixToLoad);
					InitSoundfieldAndEndpointDataForSubmix(*SubmixToLoad, GetSubmixInstance(SubmixToLoad).Pin(), false);
				}
			}
			bSubmixRegistrationDisabled = false;
		}

		if (!IsInAudioThread())
		{
			DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.InitSoundSubmixes"), STAT_InitSoundSubmixes, STATGROUP_AudioThreadCommands);

			FAudioThread::RunCommandOnAudioThread([this]()
			{
				CSV_SCOPED_TIMING_STAT(Audio, InitSubmix);
				InitSoundSubmixes();
			}, GET_STATID(STAT_InitSoundSubmixes));
			return;
		}

		for (int32 i = 0; i < static_cast<int32>(ERequiredSubmixes::Count); ++i)
		{
			if (DisableSubmixEffectEQCvar && i == static_cast<int32>(ERequiredSubmixes::EQ))
			{
				continue;
			}

			USoundSubmixBase* SoundSubmix = RequiredSubmixes[i];
			if (SoundSubmix && SoundSubmix != RequiredSubmixes[static_cast<int32>(ERequiredSubmixes::Main)])
			{
				FMixerSubmixPtr& MainSubmixInstance = RequiredSubmixInstances[i];

				RebuildSubmixLinks(*SoundSubmix, MainSubmixInstance);
			}
		}

		for (TObjectIterator<const USoundSubmixBase> It; It; ++It)
		{
			if (const USoundSubmixBase* SubmixBase = *It)
			{
				if (IsRequiredSubmixType(SubmixBase))
				{
					continue;
				}

				FMixerSubmixPtr SubmixPtr = Submixes.FindRef(SubmixBase->GetUniqueID());
				if (SubmixPtr.IsValid())
				{
					RebuildSubmixLinks(*SubmixBase, SubmixPtr);
				}
			}
		}
	}

	void FMixerDevice::RebuildSubmixLinks(const USoundSubmixBase& SoundSubmix, FMixerSubmixPtr& SubmixInstance)
	{
		// Setup up the submix instance's parent and add the submix instance as a child
		FMixerSubmixPtr ParentSubmixInstance;
		if (const USoundSubmixWithParentBase* SubmixWithParent = Cast<const USoundSubmixWithParentBase>(&SoundSubmix))
		{
			if (TObjectPtr<USoundSubmixBase> Parent = SubmixWithParent->GetParent(DeviceID); Parent)
			{
				ParentSubmixInstance = GetSubmixInstance(Parent).Pin();
			}
			else if (!SubmixWithParent->IsDynamic( true /*bIncludeAncestors*/ )) // Dynamic submixes do not auto connect.
			{
				// If this submix is itself the broadcast submix, set its parent to the master submix
				if (SubmixInstance == RequiredSubmixInstances[static_cast<int32>(ERequiredSubmixes::BaseDefault)])
				{
					ParentSubmixInstance = GetMasterSubmix().Pin();
				}
				else
				{
					ParentSubmixInstance = GetBaseDefaultSubmix().Pin();
				}
			}
		}

		if (ParentSubmixInstance.IsValid())
		{
			SubmixInstance->SetParentSubmix(ParentSubmixInstance);
			ParentSubmixInstance->AddChildSubmix(SubmixInstance);
		}
	}

 	FAudioPlatformSettings FMixerDevice::GetPlatformSettings() const
 	{
		FAudioPlatformSettings
			Settings;

		if (AudioMixerPlatform)
		{
			Settings = AudioMixerPlatform->GetPlatformSettings();

			const int32 DefaultMaxChannels = GetDefault<UAudioSettings>()->GetHighestMaxChannels();
			UE_LOGF(LogAudioMixer, Display, "Audio Mixer Platform Settings:");
			UE_LOGF(LogAudioMixer, Display, "	Sample Rate:						  %d", Settings.SampleRate);
			UE_LOGF(LogAudioMixer, Display, "	Callback Buffer Frame Size Requested: %d", Settings.CallbackBufferFrameSize);
			UE_LOGF(LogAudioMixer, Display, "	Callback Buffer Frame Size To Use:	  %d", AudioMixerPlatform->GetNumFrames(Settings.CallbackBufferFrameSize));
			UE_LOGF(LogAudioMixer, Display, "	Number of buffers to queue:			  %d", Settings.NumBuffers);
			UE_LOGF(LogAudioMixer, Display, "	Max Channels (voices):				  %d", (Settings.MaxChannels > 0) ? Settings.MaxChannels : DefaultMaxChannels);
			UE_LOGF(LogAudioMixer, Display, "	Number of Async Source Workers:		  %d", Settings.NumSourceWorkers);
		}

 		return Settings;
 	}

	FMixerSubmixWeakPtr FMixerDevice::GetMasterSubmix()
	{
		return GetMainSubmix();
	}

	FMixerSubmixWeakPtr FMixerDevice::GetMainSubmix()
	{
		return RequiredSubmixInstances[(int32)ERequiredSubmixes::Main];
	}

	FMixerSubmixWeakPtr FMixerDevice::GetBaseDefaultSubmix()
	{
		if (RequiredSubmixInstances[(int32)ERequiredSubmixes::BaseDefault].IsValid())
		{
			return RequiredSubmixInstances[(int32)ERequiredSubmixes::BaseDefault];
		}
		return GetMasterSubmix();
	}

	FMixerSubmixWeakPtr FMixerDevice::GetReverbSubmix()
	{
		return RequiredSubmixInstances[(int32)ERequiredSubmixes::Reverb];
	}

	FMixerSubmixWeakPtr FMixerDevice::GetEQSubmix()
	{
		return RequiredSubmixInstances[(int32)ERequiredSubmixes::EQ];
	}

	FMixerSubmixWeakPtr FMixerDevice::GetMasterReverbSubmix()
	{
		return RequiredSubmixInstances[(int32)ERequiredSubmixes::Reverb];
	}

	FMixerSubmixWeakPtr FMixerDevice::GetMasterEQSubmix()
	{
		return RequiredSubmixInstances[(int32)ERequiredSubmixes::EQ];
	}

	void FMixerDevice::AddMainSubmixEffect(FSoundEffectSubmixPtr SoundEffectSubmix)
	{
		AudioRenderThreadCommand([this, SoundEffectSubmix]()
		{
			RequiredSubmixInstances[(int32)ERequiredSubmixes::Main]->AddSoundEffectSubmix(SoundEffectSubmix);
		});
	}

	void FMixerDevice::AddMasterSubmixEffect(FSoundEffectSubmixPtr SoundEffectSubmix)
	{
		AddMainSubmixEffect(SoundEffectSubmix);
	}

	void FMixerDevice::RemoveMainSubmixEffect(uint32 SubmixEffectId)
	{
		AudioRenderThreadCommand([this, SubmixEffectId]()
		{
			RequiredSubmixInstances[(int32)ERequiredSubmixes::Main]->RemoveSoundEffectSubmix(SubmixEffectId);
		});
	}

	void FMixerDevice::RemoveMasterSubmixEffect(uint32 SubmixEffectId)
	{
		RemoveMainSubmixEffect(SubmixEffectId);
	}

	void FMixerDevice::ClearMasterSubmixEffects()
	{
		ClearMainSubmixEffects();
	}

	void FMixerDevice::ClearMainSubmixEffects()
	{
		AudioRenderThreadCommand([this]()
		{
			RequiredSubmixInstances[(int32)ERequiredSubmixes::Main]->ClearSoundEffectSubmixes();
		});
	}

	int32 FMixerDevice::AddSubmixEffect(USoundSubmix* InSoundSubmix, FSoundEffectSubmixPtr SoundEffect)
	{
		FMixerSubmixPtr MixerSubmixPtr = GetSubmixInstance(InSoundSubmix).Pin();
		if (MixerSubmixPtr.IsValid())
		{
			int32 NumEffects = MixerSubmixPtr->GetNumEffects();

			AudioRenderThreadCommand([this, MixerSubmixPtr, SoundEffect]()
				{
					MixerSubmixPtr->AddSoundEffectSubmix(SoundEffect);
				});

			return ++NumEffects;
		}
		else
		{
			UE_LOGF(LogAudio, Warning, "Submix instance %ls not found.", *InSoundSubmix->GetName());
		}
		return 0;
	}

	void FMixerDevice::RemoveSubmixEffect(USoundSubmix* InSoundSubmix, uint32 SubmixEffectId)
	{
		FMixerSubmixPtr MixerSubmixPtr = GetSubmixInstance(InSoundSubmix).Pin();
		if (MixerSubmixPtr.IsValid())
		{
			AudioRenderThreadCommand([MixerSubmixPtr, SubmixEffectId]()
			{
				MixerSubmixPtr->RemoveSoundEffectSubmix(SubmixEffectId);
			});
		}
	}

	void FMixerDevice::RemoveSubmixEffectAtIndex(USoundSubmix* InSoundSubmix, int32 SubmixChainIndex)
	{
		FMixerSubmixPtr MixerSubmixPtr = GetSubmixInstance(InSoundSubmix).Pin();
		if (MixerSubmixPtr.IsValid())
		{
			AudioRenderThreadCommand([MixerSubmixPtr, SubmixChainIndex]()
			{
				MixerSubmixPtr->RemoveSoundEffectSubmixAtIndex(SubmixChainIndex);
			});
		}
	}

	void FMixerDevice::ReplaceSoundEffectSubmix(USoundSubmix* InSoundSubmix, int32 InSubmixChainIndex, FSoundEffectSubmixPtr SoundEffect)
	{
		FMixerSubmixPtr MixerSubmixPtr = GetSubmixInstance(InSoundSubmix).Pin();
		if (MixerSubmixPtr.IsValid())
		{
			AudioRenderThreadCommand([MixerSubmixPtr, InSubmixChainIndex, SoundEffect]()
			{
				MixerSubmixPtr->ReplaceSoundEffectSubmix(InSubmixChainIndex, SoundEffect);
			});
		}
	}

	void FMixerDevice::ClearSubmixEffects(USoundSubmix* InSoundSubmix)
	{
		FMixerSubmixPtr MixerSubmixPtr = GetSubmixInstance(InSoundSubmix).Pin();
		if (MixerSubmixPtr.IsValid())
		{
			AudioRenderThreadCommand([MixerSubmixPtr]()
			{
				MixerSubmixPtr->ClearSoundEffectSubmixes();
			});
		}
	}

	void FMixerDevice::SetSubmixEffectChainOverride(USoundSubmix* InSoundSubmix, const TArray<FSoundEffectSubmixPtr>& InSubmixEffectPresetChain, float InFadeTimeSec)
	{
		FMixerSubmixPtr MixerSubmixPtr = GetSubmixInstance(InSoundSubmix).Pin();
		if (MixerSubmixPtr.IsValid())
		{
			AudioRenderThreadCommand([MixerSubmixPtr, InSubmixEffectPresetChain, InFadeTimeSec]()
			{
				MixerSubmixPtr->SetSubmixEffectChainOverride(InSubmixEffectPresetChain, InFadeTimeSec);
			});
		}
	}

	void FMixerDevice::ClearSubmixEffectChainOverride(USoundSubmix* InSoundSubmix, float InFadeTimeSec)
	{
		FMixerSubmixPtr MixerSubmixPtr = GetSubmixInstance(InSoundSubmix).Pin();
		if (MixerSubmixPtr.IsValid())
		{
			AudioRenderThreadCommand([MixerSubmixPtr, InFadeTimeSec]()
			{
				MixerSubmixPtr->ClearSubmixEffectChainOverride(InFadeTimeSec);
			});
		}
	}

	void FMixerDevice::UpdateSourceEffectChain(const uint32 SourceEffectChainId, const TArray<FSourceEffectChainEntry>& SourceEffectChain, const bool bPlayEffectChainTails)
	{
		TArray<FSourceEffectChainEntry>* ExistingOverride = SourceEffectChainOverrides.Find(SourceEffectChainId);
		if (ExistingOverride)
		{
			*ExistingOverride = SourceEffectChain;
		}
		else
		{
			SourceEffectChainOverrides.Add(SourceEffectChainId, SourceEffectChain);
		}

		FAudioThread::RunCommandOnAudioThread([MixerDeviceID = DeviceID, SourceEffectChainId, SourceEffectChain, bPlayEffectChainTails]()
		{
			if (FAudioDeviceManager* Manager = FAudioDeviceManager::Get())
			{
				if (FAudioDevice* Device = Manager->GetAudioDeviceRaw(MixerDeviceID))
				{
					FMixerDevice* MixerDevice = static_cast<FMixerDevice*>(Device);
					if (MixerDevice && MixerDevice->SourceManager)
					{
						MixerDevice->SourceManager->UpdateSourceEffectChain(SourceEffectChainId, SourceEffectChain, bPlayEffectChainTails);
					}
				}
			}
		});

		
	}

	void FMixerDevice::UpdateSubmixProperties(USoundSubmixBase* InSoundSubmix)
	{
		check(InSoundSubmix);

		// Output volume is only supported on USoundSubmixes.
		USoundSubmix* CastedSubmix = Cast<USoundSubmix>(InSoundSubmix);

		if (!CastedSubmix)
		{
			return;
		}

#if WITH_EDITOR
		check(IsInAudioThread());

		FMixerSubmixPtr MixerSubmix = GetSubmixInstance(InSoundSubmix).Pin();
		if (MixerSubmix.IsValid())
		{
			const float NewVolume = CastedSubmix->OutputVolumeModulation.Value;
			AudioRenderThreadCommand([MixerSubmix, NewVolume]()
			{
				MixerSubmix->SetOutputVolume(NewVolume);
			});
		}
#endif // WITH_EDITOR
	}

	void FMixerDevice::SetSubmixWetDryLevel(USoundSubmix* InSoundSubmix, float InOutputVolume, float InWetLevel, float InDryLevel)
	{
		if (!IsInAudioThread())
		{
			FMixerDevice* MixerDevice = this;
			FAudioThread::RunCommandOnAudioThread([MixerDevice, InSoundSubmix, InOutputVolume, InWetLevel, InDryLevel]()
			{
				MixerDevice->SetSubmixWetDryLevel(InSoundSubmix, InOutputVolume, InWetLevel, InDryLevel);
			});
			return;
		}

		FMixerSubmixPtr MixerSubmixPtr = GetSubmixInstance(InSoundSubmix).Pin();
		if (MixerSubmixPtr.IsValid())
		{
			AudioRenderThreadCommand([MixerSubmixPtr, InOutputVolume, InWetLevel, InDryLevel]()
			{
				MixerSubmixPtr->SetOutputVolume(InOutputVolume);
				MixerSubmixPtr->SetWetLevel(InWetLevel);
				MixerSubmixPtr->SetDryLevel(InDryLevel);
			});
		}
	}

	void FMixerDevice::SetSubmixOutputVolume(USoundSubmix* InSoundSubmix, float InOutputVolume)
	{
		if (!IsInAudioThread())
		{
			FMixerDevice* MixerDevice = this;
			FAudioThread::RunCommandOnAudioThread([MixerDevice, InSoundSubmix, InOutputVolume]()
			{
				MixerDevice->SetSubmixOutputVolume(InSoundSubmix, InOutputVolume);
			});
			return;
		}

		FMixerSubmixPtr MixerSubmixPtr = GetSubmixInstance(InSoundSubmix).Pin();
		if (MixerSubmixPtr.IsValid())
		{
			AudioRenderThreadCommand([MixerSubmixPtr, InOutputVolume]()
			{
				MixerSubmixPtr->SetOutputVolume(InOutputVolume);
			});
		}
	}

	void FMixerDevice::SetSubmixWetLevel(USoundSubmix* InSoundSubmix, float InWetLevel)
	{
		if (!IsInAudioThread())
		{
			FMixerDevice* MixerDevice = this;
			FAudioThread::RunCommandOnAudioThread([MixerDevice, InSoundSubmix, InWetLevel]()
			{
				MixerDevice->SetSubmixWetLevel(InSoundSubmix, InWetLevel);
			});
			return;
		}

		FMixerSubmixPtr MixerSubmixPtr = GetSubmixInstance(InSoundSubmix).Pin();
		if (MixerSubmixPtr.IsValid())
		{
			AudioRenderThreadCommand([MixerSubmixPtr, InWetLevel]()
			{
				MixerSubmixPtr->SetWetLevel(InWetLevel);
			});
		}
	}

	void FMixerDevice::SetSubmixDryLevel(USoundSubmix* InSoundSubmix, float InDryLevel)
	{
		if (!IsInAudioThread())
		{
			FMixerDevice* MixerDevice = this;
			FAudioThread::RunCommandOnAudioThread([MixerDevice, InSoundSubmix, InDryLevel]()
			{
				MixerDevice->SetSubmixDryLevel(InSoundSubmix, InDryLevel);
			});
			return;
		}

		FMixerSubmixPtr MixerSubmixPtr = GetSubmixInstance(InSoundSubmix).Pin();
		if (MixerSubmixPtr.IsValid())
		{
			AudioRenderThreadCommand([MixerSubmixPtr, InDryLevel]()
			{
				MixerSubmixPtr->SetDryLevel(InDryLevel);
			});
		}
	}

	void FMixerDevice::SetSubmixAutoDisable(USoundSubmix* InSoundSubmix, bool bInAutoDisable)
	{
		if (!IsInAudioThread())
		{
			FMixerDevice* MixerDevice = this;
			FAudioThread::RunCommandOnAudioThread([MixerDevice, InSoundSubmix, bInAutoDisable]()
			{
				MixerDevice->SetSubmixAutoDisable(InSoundSubmix, bInAutoDisable);
			});
			return;
		}

		FMixerSubmixPtr MixerSubmixPtr = GetSubmixInstance(InSoundSubmix).Pin();
		if (MixerSubmixPtr.IsValid())
		{
			AudioRenderThreadCommand([MixerSubmixPtr, bInAutoDisable]()
			{
				MixerSubmixPtr->SetAutoDisable(bInAutoDisable);
			});
		}
	}

	void FMixerDevice::SetSubmixAutoDisableTime(USoundSubmix* InSoundSubmix, float InDisableTime)
	{
		if (!IsInAudioThread())
		{
			FMixerDevice* MixerDevice = this;
			FAudioThread::RunCommandOnAudioThread([MixerDevice, InSoundSubmix, InDisableTime]()
			{
				MixerDevice->SetSubmixAutoDisableTime(InSoundSubmix, InDisableTime);
			});
			return;
		}

		FMixerSubmixPtr MixerSubmixPtr = GetSubmixInstance(InSoundSubmix).Pin();
		if (MixerSubmixPtr.IsValid())
		{
			AudioRenderThreadCommand([MixerSubmixPtr, InDisableTime]()
			{
				MixerSubmixPtr->SetAutoDisableTime(InDisableTime);
			});
		}
	}

	void FMixerDevice::UpdateSubmixModulationSettings(USoundSubmix* InSoundSubmix, const TSet<TObjectPtr<USoundModulatorBase>>& InOutputModulation, const TSet<TObjectPtr<USoundModulatorBase>>& InWetLevelModulation, const TSet<TObjectPtr<USoundModulatorBase>>& InDryLevelModulation)
	{
		TWeakObjectPtr<USoundSubmix> SubmixWeakPtr = InSoundSubmix;

		if (!IsInAudioThread())
		{
			FAudioThread::RunCommandOnAudioThread([ThisDeviceID = DeviceID, SubmixWeakPtr, OutMod = InOutputModulation, WetMod = InWetLevelModulation, DryMod = InDryLevelModulation]()
			{
				if (USoundSubmix* Submix = SubmixWeakPtr.Get())
				{
					if (FAudioDevice* Device = FAudioDeviceManager::Get()->GetAudioDeviceRaw(ThisDeviceID))
					{
						FMixerDevice* ThisMixerDevice = static_cast<FMixerDevice*>(Device);
						ThisMixerDevice->UpdateSubmixModulationSettings(Submix, OutMod, WetMod, DryMod);
					}
				}
			});
			return;
		}

		if (IsModulationPluginEnabled() && ModulationInterface.IsValid())
		{
			FMixerSubmixWeakPtr MixerSubmixWeakPtr = GetSubmixInstance(InSoundSubmix);

			if (SubmixWeakPtr.IsValid())
			{
				FMixerSubmixPtr MixerSubmixPtr = MixerSubmixWeakPtr.Pin();
				if (MixerSubmixPtr.IsValid())
				{
					MixerSubmixPtr->UpdateModulationSettings(InOutputModulation, InWetLevelModulation, InDryLevelModulation);
				}
			}
		}
	}

	void FMixerDevice::SetSubmixModulationBaseLevels(USoundSubmix* InSoundSubmix, float InVolumeModBase, float InWetModBase, float InDryModBase)
	{
		if (!IsInAudioThread())
		{
			TWeakObjectPtr<USoundSubmix> SubmixWeakPtr = InSoundSubmix;
			FAudioThread::RunCommandOnAudioThread([ThisDeviceID = DeviceID, SubmixWeakPtr, InVolumeModBase, InWetModBase, InDryModBase]()
			{
				if (USoundSubmix* Submix = SubmixWeakPtr.Get())
				{
					if (FAudioDevice* Device = FAudioDeviceManager::Get()->GetAudioDeviceRaw(ThisDeviceID))
					{
						FMixerDevice* ThisMixerDevice = static_cast<FMixerDevice*>(Device);
						ThisMixerDevice->SetSubmixModulationBaseLevels(Submix, InVolumeModBase, InWetModBase, InDryModBase);
					}
				}
			});
			return;
		}

		FMixerSubmixPtr MixerSubmixPtr = GetSubmixInstance(InSoundSubmix).Pin();
		if (MixerSubmixPtr.IsValid())
		{
			AudioRenderThreadCommand([MixerSubmixPtr, InVolumeModBase, InWetModBase, InDryModBase]() 
			{
				MixerSubmixPtr->SetModulationBaseLevels(InVolumeModBase, InWetModBase, InDryModBase);
			});
		}
	}

	bool FMixerDevice::GetCurrentSourceEffectChain(const uint32 SourceEffectChainId, TArray<FSourceEffectChainEntry>& OutCurrentSourceEffectChainEntries)
	{
		TArray<FSourceEffectChainEntry>* ExistingOverride = SourceEffectChainOverrides.Find(SourceEffectChainId);
		if (ExistingOverride)
		{
			OutCurrentSourceEffectChainEntries = *ExistingOverride;
			return true;
		}
		return false;
	}

	void FMixerDevice::AudioRenderThreadCommand(TFunction<void()> Command)
	{
		CommandQueue.Enqueue(MoveTemp(Command));
	}

	void FMixerDevice::GameThreadMPSCCommand(TFunction<void()> InCommand)
	{
		GameThreadCommandQueue.Enqueue(MoveTemp(InCommand));
	}
	
	void FMixerDevice::PumpCommandQueue()
	{
		// Execute the pushed lambda functions
		TFunction<void()> Command;
		while (CommandQueue.Dequeue(Command))
		{
			Command();
		}
	}

	void FMixerDevice::PumpGameThreadCommandQueue()
	{
		TOptional Opt { GameThreadCommandQueue.Dequeue() };
		while (Opt.IsSet())
		{
			TFunction<void()> Command = MoveTemp(Opt.GetValue());
			Command();
				
			Opt = GameThreadCommandQueue.Dequeue();
		}
	}
	
	void FMixerDevice::FlushAudioRenderingCommands(bool bPumpSynchronously, bool bRenderThreadStopped)
	{
		if (IsInitialized() && (FPlatformProcess::SupportsMultithreading() && !AudioMixerPlatform->IsNonRealtime()))
		{
			SourceManager->FlushCommandQueue(bPumpSynchronously, bRenderThreadStopped);
		}
		else if (AudioMixerPlatform->IsNonRealtime())
		{
			SourceManager->FlushCommandQueue(true);
		}
		else
		{
			// Pump the audio device's command queue
			PumpCommandQueue();

			// And also directly pump the source manager command queue
			SourceManager->PumpCommandQueue();
			SourceManager->PumpCommandQueue();

			SourceManager->UpdatePendingReleaseData(true);
		}
	}

	bool FMixerDevice::IsRequiredSubmixType(const USoundSubmixBase* InSubmix) const
	{
		for (int32 i = 0; i < (int32)EMasterSubmixType::Count; ++i)
		{
			if (InSubmix == RequiredSubmixes[i])
			{
				return true;
			}
		}
		return false;
	}

	FMixerSubmixPtr FMixerDevice::GetRequiredSubmixInstance(uint32 InObjectId) const
	{
		check(RequiredSubmixes.Num() == (int32)ERequiredSubmixes::Count);
		for (int32 i = 0; i < (int32)ERequiredSubmixes::Count; ++i)
		{
			if (RequiredSubmixes[i] && InObjectId == RequiredSubmixes[i]->GetUniqueID())
			{
				return RequiredSubmixInstances[i];
			}
		}
		return nullptr;
	}

	FMixerSubmixPtr FMixerDevice::GetRequiredSubmixInstance(const USoundSubmixBase* InSubmix) const
	{
		check(RequiredSubmixes.Num() == (int32)ERequiredSubmixes::Count);
		for (int32 i = 0; i < (int32)ERequiredSubmixes::Count; ++i)
		{
			if (InSubmix == RequiredSubmixes[i])
			{
				return RequiredSubmixInstances[i];
			}
		}
		return nullptr;
	}

	void FMixerDevice::RegisterSoundSubmix(USoundSubmixBase* InSoundSubmix, bool bInit)
	{
		if (InSoundSubmix && bSubmixRegistrationDisabled)
		{
			UE_LOGF(LogAudioMixer, Warning, "Attempted register Submix %ls before the submix graph was initialized.", *InSoundSubmix->GetFullName());
			return;
		}

		if (!InSoundSubmix)
		{
			return;
		}

		if (!IsInAudioThread())
		{
			DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.RegisterSoundSubmix"), STAT_AudioRegisterSoundSubmix, STATGROUP_AudioThreadCommands);

			FMixerDevice* MixerDevice = this;
			FAudioThread::RunCommandOnAudioThread([MixerDevice, InSoundSubmix, bInit]()
			{
				CSV_SCOPED_TIMING_STAT(Audio, RegisterSubmix);
				MixerDevice->RegisterSoundSubmix(InSoundSubmix, bInit);
			}, GET_STATID(STAT_AudioRegisterSoundSubmix));
			return;
		}

		UE_LOGF(LogAudioMixer, Display, "Registering submix %ls.", *InSoundSubmix->GetFullName());

		const bool bIsMainSubmix = IsRequiredSubmixType(InSoundSubmix);

		if (!bIsMainSubmix)
		{
			// Ensure parent structure is registered prior to current submix if missing
			if (const USoundSubmixWithParentBase* SubmixWithParent = Cast<const USoundSubmixWithParentBase>(InSoundSubmix))
			{
				if (TObjectPtr<USoundSubmixBase> Parent = SubmixWithParent->GetParent(DeviceID))
				{
					FMixerSubmixPtr ParentSubmix = GetSubmixInstance(Parent).Pin();
					if (!ParentSubmix.IsValid())
					{
						RegisterSoundSubmix(Parent, bInit);
					}
				}
			}

			LoadSoundSubmix(*InSoundSubmix);
		}
		else
		{
			UE_LOGF(LogAudioMixer, Display, "Submix %ls was already registered as one of the master submixes.", *InSoundSubmix->GetFullName());
		}

		FMixerSubmixPtr SubmixPtr = GetSubmixInstance(InSoundSubmix).Pin();

		if (bInit)
		{
			InitSoundfieldAndEndpointDataForSubmix(*InSoundSubmix, SubmixPtr, true);
		}

		if (!bIsMainSubmix)
		{
			RebuildSubmixLinks(*InSoundSubmix, SubmixPtr);
		}

		// Now that the submix is loaded, we can register the submix buffer listeners that were waiting for it
		if (PendingSubmixBufferListeners.Contains(InSoundSubmix->GetPathName()))
		{
			UE_LOGF(LogAudioMixer, Display, "Submix %ls had pending Submix Buffer Listeners, registering them now...", *InSoundSubmix->GetName());
			
			TMap<FString, TSharedRef<ISubmixBufferListener, ESPMode::ThreadSafe>>& PendingListeners = PendingSubmixBufferListeners[InSoundSubmix->GetPathName()];

			if (USoundSubmix* Submix = Cast<USoundSubmix>(InSoundSubmix))
			{
				for (TPair<FString, TSharedRef<ISubmixBufferListener, ESPMode::ThreadSafe>>& Listener : PendingListeners)
				{
					RegisterSubmixBufferListener(Listener.Value, *Submix);
				}
				PendingSubmixBufferListeners.Remove(InSoundSubmix->GetPathName());
			}
		}
	}

	void FMixerDevice::LoadSoundSubmix(USoundSubmixBase& InSoundSubmix)
	{
		// If submix not already found, load it.
		FMixerSubmixPtr MixerSubmix = GetSubmixInstance(&InSoundSubmix).Pin();
		if (!MixerSubmix.IsValid())
		{
			if (GRemoveSubmixesFromRoot)
			{
				InSoundSubmix.IncrementLoadRefCount();
			}
			else
			{
				InSoundSubmix.AddToRoot();
			}

			MixerSubmix = MakeShared<FMixerSubmix, ESPMode::ThreadSafe>(this);
			Submixes.Add(InSoundSubmix.GetUniqueID(), MixerSubmix);
		}
	}

	void FMixerDevice::InitSoundfieldAndEndpointDataForSubmix(const USoundSubmixBase& InSoundSubmix, FMixerSubmixPtr MixerSubmix, bool bAllowReInit)
	{
		{
			FScopeLock ScopeLock(&EndpointSubmixesMutationLock);

			// Check to see if this is an endpoint or soundfield submix:
			if (const USoundfieldSubmix* SoundfieldSubmix = Cast<const USoundfieldSubmix>(&InSoundSubmix))
			{
				MixerSubmix->SetSoundfieldFactory(SoundfieldSubmix->GetSoundfieldFactoryForSubmix());
			}
			else if (const USoundfieldEndpointSubmix* SoundfieldEndpointSubmix = Cast<const USoundfieldEndpointSubmix>(&InSoundSubmix))
			{
				MixerSubmix->SetSoundfieldFactory(SoundfieldEndpointSubmix->GetSoundfieldEndpointForSubmix());
			}

			if (DefaultEndpointSubmixes.Contains(MixerSubmix))
			{
				DefaultEndpointSubmixes.RemoveSwap(MixerSubmix);
			}

			if (ExternalEndpointSubmixes.Contains(MixerSubmix))
			{
				ExternalEndpointSubmixes.RemoveSwap(MixerSubmix);
			}

			MixerSubmix->Init(&InSoundSubmix, bAllowReInit);

			if (IsEndpointSubmix(&InSoundSubmix) && MixerSubmix->IsDefaultEndpointSubmix())
			{
				DefaultEndpointSubmixes.Add(MixerSubmix);
			}
			else if (MixerSubmix->IsExternalEndpointSubmix())
			{
				ExternalEndpointSubmixes.Add(MixerSubmix);
			}
		}
	}

	void FMixerDevice::UnregisterSoundSubmix(const USoundSubmixBase* InSoundSubmix, const bool bReparentChildren)
	{
		if (!InSoundSubmix || bSubmixRegistrationDisabled || IsRequiredSubmixType(InSoundSubmix))
		{
			return;
		}

		if (!IsInAudioThread())
		{
			DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.UnregisterSoundSubmix"), STAT_AudioUnregisterSoundSubmix, STATGROUP_AudioThreadCommands);

			const TWeakObjectPtr<const USoundSubmixBase> SubmixToUnload = InSoundSubmix;
			FAudioThread::RunCommandOnAudioThread([this, SubmixToUnload, bReparentChildren]()
			{
				CSV_SCOPED_TIMING_STAT(Audio, UnregisterSubmix);
				if (SubmixToUnload.IsValid())
				{
					UnloadSoundSubmix(*SubmixToUnload.Get(), bReparentChildren);
				}
			}, GET_STATID(STAT_AudioUnregisterSoundSubmix));
			return;
		}

		UnloadSoundSubmix(*InSoundSubmix, bReparentChildren);
	}

	void FMixerDevice::UnloadSoundSubmix(const USoundSubmixBase& InSoundSubmix, const bool bReparentChildren)
	{
		check(IsInAudioThread());

		FMixerSubmixWeakPtr MainSubmix = GetMasterSubmix();

		// Check if this is a submix type that has a parent.
		FMixerSubmixPtr ParentSubmixInstance;
		if (const USoundSubmixWithParentBase* InSoundSubmixWithParent = Cast<const USoundSubmixWithParentBase>(&InSoundSubmix))
		{
			ParentSubmixInstance = InSoundSubmixWithParent->GetParent(DeviceID)
				? GetSubmixInstance(InSoundSubmixWithParent->GetParent(DeviceID)).Pin()
				: MainSubmix.Pin();
		}

		if (ParentSubmixInstance.IsValid())
		{
			ParentSubmixInstance->RemoveChildSubmix(GetSubmixInstance(&InSoundSubmix));
		}

		if (bReparentChildren)
		{
			for (USoundSubmixBase* ChildSubmix : InSoundSubmix.ChildSubmixes)
			{
				FMixerSubmixPtr ChildSubmixPtr = GetSubmixInstance(ChildSubmix).Pin();
				if (ChildSubmixPtr.IsValid())
				{
					ChildSubmixPtr->SetParentSubmix(ParentSubmixInstance.IsValid()
						? ParentSubmixInstance
						: MainSubmix);
				}
			}
		}

		FMixerSubmixWeakPtr MixerSubmixWeakPtr = GetSubmixInstance(&InSoundSubmix);
		FMixerSubmixPtr MixerSubmix = MixerSubmixWeakPtr.Pin();

		if (MixerSubmix && MixerSubmix->IsDefaultEndpointSubmix())
		{
			FScopeLock ScopeLock(&EndpointSubmixesMutationLock);
			DefaultEndpointSubmixes.Remove(MixerSubmix);
		}
		else if (MixerSubmix && MixerSubmix->IsExternalEndpointSubmix())
		{
			FScopeLock ScopeLock(&EndpointSubmixesMutationLock);
			ExternalEndpointSubmixes.Remove(MixerSubmix);
		}

#if UE_AUDIO_PROFILERTRACE_ENABLED
		if (MixerSubmix.IsValid())
		{
			FAudioThread::RunCommandOnGameThread([this, SubmixID = MixerSubmix->GetId(), SubmixName = MixerSubmix->GetName()]()
			{
				SubmixesWithAutoEnvelopeFollowers.Remove(SubmixID);
				SubmixesWithCVarEnvelopeFollowers.Remove(SubmixName);
			});
		}
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

		const int32 RemoveCount = Submixes.Remove(InSoundSubmix.GetUniqueID());
		
		if (GRemoveSubmixesFromRoot && RemoveCount > 0)
		{
			USoundSubmixBase& MutableSubmix = const_cast<USoundSubmixBase&>(InSoundSubmix);
			MutableSubmix.DecrementLoadRefCount();
		}
	}

	FMixerSubmixPtr FMixerDevice::FindSubmixInstanceByObjectId(uint32 InObjectId)
	{
		for (int32 i = 0; i < RequiredSubmixes.Num(); i++)
		{
			if (const USoundSubmix* MainSubmix = RequiredSubmixes[i])
			{
				if (MainSubmix->GetUniqueID() == InObjectId)
				{
					return GetRequiredSubmixInstance(MainSubmix);
				}
			}
			else
			{
				const ERequiredSubmixes SubmixType = static_cast<ERequiredSubmixes>(i);
				ensureAlwaysMsgf(ERequiredSubmixes::Main != SubmixType,
					TEXT("Top-level main submix has to be registered before anything else, and is required for the lifetime of the application.")
				);

				if (!DisableSubmixEffectEQCvar && ERequiredSubmixes::EQ == SubmixType)
				{
					UE_LOGF(LogAudioMixer, Warning, "Failed to query EQ Submix when it was expected to be loaded.");
				}
			}
		}

		return Submixes.FindRef(InObjectId);
	}

	FMixerSubmixWeakPtr FMixerDevice::GetSubmixInstance(const USoundSubmixBase* SoundSubmix) const
	{
		LLM_SCOPE(ELLMTag::AudioMixer);

		FMixerSubmixPtr MixerSubmix = GetRequiredSubmixInstance(SoundSubmix);
		if (MixerSubmix.IsValid())
		{
			return MixerSubmix;
		}
		
		if (SoundSubmix)
		{
			return Submixes.FindRef(SoundSubmix->GetUniqueID());
		}

		return nullptr;
	}

	ISoundfieldFactory* FMixerDevice::GetFactoryForSubmixInstance(USoundSubmix* SoundSubmix)
	{
		FMixerSubmixWeakPtr WeakSubmixPtr = GetSubmixInstance(SoundSubmix);
		return GetFactoryForSubmixInstance(WeakSubmixPtr);
	}

	ISoundfieldFactory* FMixerDevice::GetFactoryForSubmixInstance(FMixerSubmixWeakPtr& SoundSubmixPtr)
	{
		FMixerSubmixPtr SubmixPtr = SoundSubmixPtr.Pin();
		if (SubmixPtr.IsValid())
		{
			return SubmixPtr->GetSoundfieldFactory();
		}
		else
		{
			return nullptr;
		}
	}

	FMixerSourceVoice* FMixerDevice::GetMixerSourceVoice()
	{
		LLM_SCOPE(ELLMTag::AudioMixer);

		FMixerSourceVoice* Voice = nullptr;
		if (!SourceVoices.Dequeue(Voice))
		{
			Voice = new FMixerSourceVoice();
		}

		Voice->Reset(this);
		return Voice;
	}

	void FMixerDevice::ReleaseMixerSourceVoice(FMixerSourceVoice* InSourceVoice)
	{
		SourceVoices.Enqueue(InSourceVoice);
	}

	int32 FMixerDevice::GetNumSources() const
	{
		return Sources.Num();
	}

	IAudioLinkFactory* FMixerDevice::GetAudioLinkFactory() const
	{
		return AudioLinkFactory;
	}

	int32 FMixerDevice::GetNumActiveSources() const
	{
		return SourceManager->GetNumActiveSources();
	}

	void FMixerDevice::Get3DChannelMap(const int32 InSubmixNumChannels, const FWaveInstance* InWaveInstance, float EmitterAzimith, float InNonSpatializedAmount, const TMap<EAudioMixerChannel::Type, float>* InOmniMap, float InDefaultOmniValue, Audio::FAlignedFloatBuffer& OutChannelMap)
	{
		// If we're center-channel only, then no need for spatial calculations, but need to build a channel map
		if (InWaveInstance->bCenterChannelOnly)
		{
			int32 NumOutputChannels = InSubmixNumChannels;
			const TArray<EAudioMixerChannel::Type>& ChannelArray = GetChannelArray();

			// If we are only spatializing to stereo output
			if (NumOutputChannels == 2)
			{
				// Equal volume in left + right channel with equal power panning
				static const float Pan = 1.0f / FMath::Sqrt(2.0f);
				OutChannelMap.Add(Pan);
				OutChannelMap.Add(Pan);
			}
			else
			{
				for (EAudioMixerChannel::Type Channel : ChannelArray)
				{
					float Pan = (Channel == EAudioMixerChannel::FrontCenter) ? 1.0f : 0.0f;
					OutChannelMap.Add(Pan);
				}
			}

			return;
		}

		float Azimuth = EmitterAzimith;

		const FChannelPositionInfo* PrevChannelInfo = nullptr;
		const FChannelPositionInfo* NextChannelInfo = nullptr;

		for (int32 i = 0; i < DeviceChannelAzimuthPositions.Num(); ++i)
		{
			const FChannelPositionInfo& ChannelPositionInfo = DeviceChannelAzimuthPositions[i];

			if (Azimuth <= ChannelPositionInfo.Azimuth)
			{
				NextChannelInfo = &DeviceChannelAzimuthPositions[i];

				int32 PrevIndex = i - 1;
				if (PrevIndex < 0)
				{
					PrevIndex = DeviceChannelAzimuthPositions.Num() - 1;
				}

				PrevChannelInfo = &DeviceChannelAzimuthPositions[PrevIndex];
				break;
			}
		}

		// If we didn't find anything, that means our azimuth position is at the top of the mapping
		if (PrevChannelInfo == nullptr)
		{
			PrevChannelInfo = &DeviceChannelAzimuthPositions[DeviceChannelAzimuthPositions.Num() - 1];
			NextChannelInfo = &DeviceChannelAzimuthPositions[0];
			AUDIO_MIXER_CHECK(PrevChannelInfo != NextChannelInfo);
		}

		float NextChannelAzimuth = NextChannelInfo->Azimuth;
		float PrevChannelAzimuth = PrevChannelInfo->Azimuth;

		if (NextChannelAzimuth < PrevChannelAzimuth)
		{
			NextChannelAzimuth += 360.0f;
		}

		if (Azimuth < PrevChannelAzimuth)
		{
			Azimuth += 360.0f;
		}

		AUDIO_MIXER_CHECK(NextChannelAzimuth > PrevChannelAzimuth);
		AUDIO_MIXER_CHECK(Azimuth > PrevChannelAzimuth);
		float Fraction = (Azimuth - PrevChannelAzimuth) / (NextChannelAzimuth - PrevChannelAzimuth);
		AUDIO_MIXER_CHECK(Fraction >= 0.0f && Fraction <= 1.0f);

		// Compute the panning values using equal-power panning law
		float PrevChannelPan; 
		float NextChannelPan;

		if (PanningMethod == EPanningMethod::EqualPower)
		{
			FMath::SinCos(&NextChannelPan, &PrevChannelPan, Fraction * 0.5f * PI);

			// Note that SinCos can return values slightly greater than 1.0 when very close to PI/2
			NextChannelPan = FMath::Clamp(NextChannelPan, 0.0f, 1.0f);
			PrevChannelPan = FMath::Clamp(PrevChannelPan, 0.0f, 1.0f);
		}
		else
		{
			NextChannelPan = Fraction;
			PrevChannelPan = 1.0f - Fraction;
		}

		float OmniAmount = InNonSpatializedAmount;

		// Build the output channel map based on the current platform device output channel array 

		int32 NumSpatialChannels = DeviceChannelAzimuthPositions.Num();
		if (DeviceChannelAzimuthPositions.Num() > 4)
		{
			NumSpatialChannels--;
		}

		const TArray<EAudioMixerChannel::Type>& ChannelArray = GetChannelArray();

		if (OmniAmount > 0.0f)
		{
			for (EAudioMixerChannel::Type Channel : ChannelArray)
			{
				float OmniPanFactor = InDefaultOmniValue;

				if (InOmniMap)
				{
					const float* MappedOmniPanFactor = InOmniMap->Find(Channel);
					if (MappedOmniPanFactor)
					{
						OmniPanFactor = *MappedOmniPanFactor;
					}
					else
					{
						OmniPanFactor = 0.0f;
					}
				}

				float EffectivePan = 0.0f;

				// Check for manual channel mapping parameters (LFE and Front Center)
				if (Channel == EAudioMixerChannel::LowFrequency)
				{
					EffectivePan = InWaveInstance->LFEBleed;
				}
				else if (Channel == PrevChannelInfo->Channel)
				{
					EffectivePan = FMath::Lerp(PrevChannelPan, OmniPanFactor, OmniAmount);
				}
				else if (Channel == NextChannelInfo->Channel)
				{
					EffectivePan = FMath::Lerp(NextChannelPan, OmniPanFactor, OmniAmount);
				}
				else if (Channel == EAudioMixerChannel::FrontCenter)
				{
					EffectivePan = FMath::Lerp(0.0f, OmniPanFactor, OmniAmount);
					EffectivePan = FMath::Max(InWaveInstance->VoiceCenterChannelVolume, EffectivePan);
				}
				else
				{
					EffectivePan = FMath::Lerp(0.0f, OmniPanFactor, OmniAmount);
				}

				AUDIO_MIXER_CHECK(EffectivePan >= 0.0f && EffectivePan <= 1.0f);
				OutChannelMap.Add(EffectivePan);
			}
		}
		else
		{
			for (EAudioMixerChannel::Type Channel : ChannelArray)
			{
				float EffectivePan = 0.0f;

				// Check for manual channel mapping parameters (LFE and Front Center)
				if (Channel == EAudioMixerChannel::LowFrequency)
				{
					EffectivePan = InWaveInstance->LFEBleed;
				}
				else if (Channel == PrevChannelInfo->Channel)
				{
					EffectivePan = PrevChannelPan;
				}
				else if (Channel == NextChannelInfo->Channel)
				{
					EffectivePan = NextChannelPan;
				}
				else if (Channel == EAudioMixerChannel::FrontCenter)
				{
					EffectivePan = FMath::Max(InWaveInstance->VoiceCenterChannelVolume, EffectivePan);
				}

				AUDIO_MIXER_CHECK(EffectivePan >= 0.0f && EffectivePan <= 1.0f);
				OutChannelMap.Add(EffectivePan);
			}
		}
	}

	const TArray<FTransform>* FMixerDevice::GetListenerTransforms()
	{
		return SourceManager->GetListenerTransforms();
	}

	void FMixerDevice::StartRecording(USoundSubmix* InSubmix, float ExpectedRecordingDuration)
	{
		if (!IsInAudioThread())
		{
			DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.PauseRecording"), STAT_StartRecording, STATGROUP_AudioThreadCommands);

			FAudioThread::RunCommandOnAudioThread([this, InSubmix, ExpectedRecordingDuration]()
			{
				CSV_SCOPED_TIMING_STAT(Audio, StartRecording);
				StartRecording(InSubmix, ExpectedRecordingDuration);
			}, GET_STATID(STAT_StartRecording));
			return;
		}

		// if we can find the submix here, record that submix. Otherwise, just record the master submix.
		FMixerSubmixPtr FoundSubmix = GetSubmixInstance(InSubmix).Pin();
		if (FoundSubmix.IsValid())
		{
			FoundSubmix->OnStartRecordingOutput(ExpectedRecordingDuration);
		}
		else
		{
			FMixerSubmixWeakPtr MainSubmix = GetMasterSubmix();
			FMixerSubmixPtr MainSubmixPtr = MainSubmix.Pin();
			check(MainSubmixPtr.IsValid());

			MainSubmixPtr->OnStartRecordingOutput(ExpectedRecordingDuration);
		}
	}

	Audio::FAlignedFloatBuffer& FMixerDevice::StopRecording(USoundSubmix* InSubmix, float& OutNumChannels, float& OutSampleRate)
	{
		// if we can find the submix here, record that submix. Otherwise, just record the master submix.
		FMixerSubmixPtr FoundSubmix = GetSubmixInstance(InSubmix).Pin();
		if (FoundSubmix.IsValid())
		{
			return FoundSubmix->OnStopRecordingOutput(OutNumChannels, OutSampleRate);
		}
		else
		{
			FMixerSubmixPtr MainSubmixPtr = GetMasterSubmix().Pin();
			check(MainSubmixPtr.IsValid());

			return MainSubmixPtr->OnStopRecordingOutput(OutNumChannels, OutSampleRate);
		}
	}

	void FMixerDevice::PauseRecording(USoundSubmix* InSubmix)
	{
		if (!IsInAudioThread())
		{
			DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.PauseRecording"), STAT_PauseRecording, STATGROUP_AudioThreadCommands);

			FAudioThread::RunCommandOnAudioThread([this, InSubmix]()
			{
				CSV_SCOPED_TIMING_STAT(Audio, PauseRecording);
				PauseRecording(InSubmix);
			}, GET_STATID(STAT_PauseRecording));
			return;
		}

		// if we can find the submix here, pause that submix. Otherwise, just pause the master submix.
		FMixerSubmixPtr FoundSubmix = GetSubmixInstance(InSubmix).Pin();
		if (FoundSubmix.IsValid())
		{
			FoundSubmix->PauseRecordingOutput();
		}
		else
		{
			FMixerSubmixWeakPtr MainSubmix = GetMasterSubmix();
			FMixerSubmixPtr MainSubmixPtr = MainSubmix.Pin();
			check(MainSubmixPtr.IsValid());

			MainSubmixPtr->PauseRecordingOutput();
		}
	}

	void FMixerDevice::ResumeRecording(USoundSubmix* InSubmix)
	{
		if (!IsInAudioThread())
		{
			DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.ResumeRecording"), STAT_ResumeRecording, STATGROUP_AudioThreadCommands);

			FAudioThread::RunCommandOnAudioThread([this, InSubmix]()
			{
				CSV_SCOPED_TIMING_STAT(Audio, ResumeRecording);
				ResumeRecording(InSubmix);
			}, GET_STATID(STAT_ResumeRecording));
			return;
		}

		// if we can find the submix here, resume that submix. Otherwise, just resume the master submix.
		FMixerSubmixPtr FoundSubmix = GetSubmixInstance(InSubmix).Pin();
		if (FoundSubmix.IsValid())
		{
			FoundSubmix->ResumeRecordingOutput();
		}
		else
		{
			FMixerSubmixWeakPtr MainSubmix = GetMasterSubmix();
			FMixerSubmixPtr MainSubmixPtr = MainSubmix.Pin();
			check(MainSubmixPtr.IsValid());

			MainSubmixPtr->ResumeRecordingOutput();
		}
	}

	void FMixerDevice::StartEnvelopeFollowing(USoundSubmix* InSubmix)
	{
		if (!IsInAudioThread())
		{
			DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.StartEnvelopeFollowing"), STAT_StartEnvelopeFollowing, STATGROUP_AudioThreadCommands);

			FAudioThread::RunCommandOnAudioThread([this, InSubmix]()
			{
				CSV_SCOPED_TIMING_STAT(Audio, StartEnvelopeFollowing);
				StartEnvelopeFollowing(InSubmix);
			}, GET_STATID(STAT_StartEnvelopeFollowing));
			return;
		}

		// if we can find the submix here, record that submix. Otherwise, just record the master submix.
		FMixerSubmixPtr FoundSubmix = GetSubmixInstance(InSubmix).Pin();
		if (FoundSubmix.IsValid())
		{
			FoundSubmix->StartEnvelopeFollowing(InSubmix->EnvelopeFollowerAttackTime, InSubmix->EnvelopeFollowerReleaseTime);
		}
		else
		{
			FMixerSubmixWeakPtr MainSubmix = GetMasterSubmix();
			FMixerSubmixPtr MainSubmixPtr = MainSubmix.Pin();
			check(MainSubmixPtr.IsValid());

			MainSubmixPtr->StartEnvelopeFollowing(InSubmix->EnvelopeFollowerAttackTime, InSubmix->EnvelopeFollowerReleaseTime);
		}

		DelegateBoundSubmixes.AddUnique(InSubmix);
	}

	void FMixerDevice::StopEnvelopeFollowing(USoundSubmix* InSubmix)
	{
		if (!IsInAudioThread())
		{
			DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.StopEnvelopeFollowing"), STAT_StopEnvelopeFollowing, STATGROUP_AudioThreadCommands);

			FAudioThread::RunCommandOnAudioThread([this, InSubmix]()
			{
				CSV_SCOPED_TIMING_STAT(Audio, StopEnvelopeFollowing);
				StopEnvelopeFollowing(InSubmix);
			}, GET_STATID(STAT_StopEnvelopeFollowing));
			return;
		}

		// if we can find the submix here, record that submix. Otherwise, just record the master submix.
		FMixerSubmixPtr FoundSubmix = GetSubmixInstance(InSubmix).Pin();
		if (FoundSubmix.IsValid())
		{
			FoundSubmix->StopEnvelopeFollowing();
		}
		else
		{
			FMixerSubmixWeakPtr MainSubmix = GetMasterSubmix();
			FMixerSubmixPtr MainSubmixPtr = MainSubmix.Pin();
			check(MainSubmixPtr.IsValid());

			MainSubmixPtr->StopEnvelopeFollowing();
		}

		DelegateBoundSubmixes.RemoveSingleSwap(InSubmix);
	}

	void FMixerDevice::AddEnvelopeFollowerDelegate(USoundSubmix* InSubmix, const FOnSubmixEnvelopeBP& OnSubmixEnvelopeBP)
	{
		if (!IsInAudioThread())
		{
			DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.AddEnvelopeFollowerDelegate"), STAT_AddEnvelopeFollowerDelegate, STATGROUP_AudioThreadCommands);

			FAudioThread::RunCommandOnAudioThread([this, InSubmix, OnSubmixEnvelopeBP]()
			{
				CSV_SCOPED_TIMING_STAT(Audio, AddEnvelopeFollowerDelegate);
				AddEnvelopeFollowerDelegate(InSubmix, OnSubmixEnvelopeBP);
			}, GET_STATID(STAT_AddEnvelopeFollowerDelegate));
			return;
		}

		// if we can find the submix here, record that submix. Otherwise, just record the master submix.
		FMixerSubmixPtr FoundSubmix = GetSubmixInstance(InSubmix).Pin();
		if (FoundSubmix.IsValid())
		{
			FoundSubmix->AddEnvelopeFollowerDelegate(OnSubmixEnvelopeBP);
		}
		else
		{
			FMixerSubmixWeakPtr MainSubmix = GetMasterSubmix();
			FMixerSubmixPtr MainSubmixPtr = MainSubmix.Pin();
			check(MainSubmixPtr.IsValid());

			MainSubmixPtr->AddEnvelopeFollowerDelegate(OnSubmixEnvelopeBP);
		}
	}

	void FMixerDevice::RemoveEnvelopeFollowerDelegate(USoundSubmix* InSubmix, const FOnSubmixEnvelopeBP& OnSubmixEnvelopeBP)
	{
		if (!IsInAudioThread())
		{
			DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.RemoveEnvelopeFollowerDelegate"), STAT_RemoveEnvelopeFollowerDelegate, STATGROUP_AudioThreadCommands);

			FAudioThread::RunCommandOnAudioThread([this, InSubmix, OnSubmixEnvelopeBP]()
			{
				CSV_SCOPED_TIMING_STAT(Audio, RemoveEnvelopeFollowerDelegate);
				RemoveEnvelopeFollowerDelegate(InSubmix, OnSubmixEnvelopeBP);
			}, GET_STATID(STAT_RemoveEnvelopeFollowerDelegate));
			return;
		}

		// Fallback to the master submix if the provided submix isn't found to match behavior from ::AddEnvelopeFollowerDelegate
		FMixerSubmixPtr FoundSubmix = GetSubmixInstance(InSubmix).Pin();
		if (FoundSubmix.IsValid())
		{
			FoundSubmix->RemoveEnvelopeFollowerDelegate(OnSubmixEnvelopeBP);
		}
		else
		{
			FMixerSubmixWeakPtr MainSubmix = GetMasterSubmix();
			FMixerSubmixPtr MainSubmixPtr = MainSubmix.Pin();
			check(MainSubmixPtr.IsValid());

			MainSubmixPtr->RemoveEnvelopeFollowerDelegate(OnSubmixEnvelopeBP);
		}
	}

	void FMixerDevice::StartSpectrumAnalysis(USoundSubmix* InSubmix, const FSoundSpectrumAnalyzerSettings& InSettings)
	{
		if (!IsInAudioThread())
		{
			DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.StartSpectrumAnalysis"), STAT_StartSpectrumAnalysis, STATGROUP_AudioThreadCommands);

			FAudioThread::RunCommandOnAudioThread([this, InSubmix, InSettings]()
			{
				CSV_SCOPED_TIMING_STAT(Audio, StartSpectrumAnalysis);
				StartSpectrumAnalysis(InSubmix, InSettings);
			}, GET_STATID(STAT_StartSpectrumAnalysis));
			return;
		}

		FMixerSubmixPtr FoundSubmix = GetSubmixInstance(InSubmix).Pin();
		if (FoundSubmix.IsValid())
		{
			FoundSubmix->StartSpectrumAnalysis(InSettings);
		}
		else
		{
			FMixerSubmixWeakPtr MainSubmix = GetMasterSubmix();
			FMixerSubmixPtr MainSubmixPtr = MainSubmix.Pin();
			check(MainSubmixPtr.IsValid());

			MainSubmixPtr->StartSpectrumAnalysis(InSettings);
		}

		DelegateBoundSubmixes.AddUnique(InSubmix);
	}

	void FMixerDevice::StopSpectrumAnalysis(USoundSubmix* InSubmix)
	{
		if (!IsInAudioThread())
		{
			DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.StopSpectrumAnalysis"), STAT_StopSpectrumAnalysis, STATGROUP_AudioThreadCommands);

			FAudioThread::RunCommandOnAudioThread([this, InSubmix]()
			{
				CSV_SCOPED_TIMING_STAT(Audio, StopSpectrumAnalysis);
				StopSpectrumAnalysis(InSubmix);
			}, GET_STATID(STAT_StopSpectrumAnalysis));
			return;
		}

		FMixerSubmixPtr FoundSubmix = GetSubmixInstance(InSubmix).Pin();
		if (FoundSubmix.IsValid())
		{
			FoundSubmix->StopSpectrumAnalysis();
		}
		else
		{
			FMixerSubmixWeakPtr MainSubmix = GetMasterSubmix();
			FMixerSubmixPtr MainSubmixPtr = MainSubmix.Pin();
			check(MainSubmixPtr.IsValid());

			MainSubmixPtr->StopSpectrumAnalysis();
		}

		DelegateBoundSubmixes.RemoveSingleSwap(InSubmix);

	}

	void FMixerDevice::GetMagnitudesForFrequencies(USoundSubmix* InSubmix, const TArray<float>& InFrequencies, TArray<float>& OutMagnitudes)
	{
		FMixerSubmixPtr FoundSubmix = GetSubmixInstance(InSubmix).Pin();
		if (FoundSubmix.IsValid())
		{
			FoundSubmix->GetMagnitudeForFrequencies(InFrequencies, OutMagnitudes);
		}
		else
		{
			FMixerSubmixWeakPtr MainSubmix = GetMasterSubmix();
			FMixerSubmixPtr MainSubmixPtr = MainSubmix.Pin();
			check(MainSubmixPtr.IsValid());

			MainSubmixPtr->GetMagnitudeForFrequencies(InFrequencies, OutMagnitudes);
		}
	}

	void FMixerDevice::GetPhasesForFrequencies(USoundSubmix* InSubmix, const TArray<float>& InFrequencies, TArray<float>& OutPhases)
	{
		FMixerSubmixPtr FoundSubmix = GetSubmixInstance(InSubmix).Pin();
		if (FoundSubmix.IsValid())
		{
			FoundSubmix->GetPhaseForFrequencies(InFrequencies, OutPhases);
		}
		else
		{
			FMixerSubmixWeakPtr MainSubmix = GetMasterSubmix();
			FMixerSubmixPtr MainSubmixPtr = MainSubmix.Pin();
			check(MainSubmixPtr.IsValid());

			MainSubmixPtr->GetPhaseForFrequencies(InFrequencies, OutPhases);
		}
	}

	void FMixerDevice::AddSpectralAnalysisDelegate(USoundSubmix* InSubmix, const FSoundSpectrumAnalyzerDelegateSettings& InDelegateSettings, const FOnSubmixSpectralAnalysisBP& OnSubmixSpectralAnalysisBP)
	{

		if (!IsInAudioThread())
		{
			DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.AddSpectralAnalysisDelegate"), STAT_AddSpectralAnalysisDelegate, STATGROUP_AudioThreadCommands);

			FAudioThread::RunCommandOnAudioThread([this, InSubmix, InDelegateSettings, OnSubmixSpectralAnalysisBP]()
			{
				CSV_SCOPED_TIMING_STAT(Audio, AddSpectralAnalysisDelegate);
				AddSpectralAnalysisDelegate(InSubmix, InDelegateSettings, OnSubmixSpectralAnalysisBP);
			}, GET_STATID(STAT_AddSpectralAnalysisDelegate));
			return;
		}

		// get submix if it is available.
		FMixerSubmixPtr FoundSubmix = GetSubmixInstance(InSubmix).Pin();

		if (!FoundSubmix.IsValid())
		{
			// If can't find the submix isntance, use master submix.
			FMixerSubmixWeakPtr MainSubmix = GetMasterSubmix();
			FoundSubmix = MainSubmix.Pin();
		}

		if (ensure(FoundSubmix.IsValid()))
		{
			FoundSubmix->AddSpectralAnalysisDelegate(InDelegateSettings, OnSubmixSpectralAnalysisBP);
		}
	}

	void FMixerDevice::RemoveSpectralAnalysisDelegate(USoundSubmix* InSubmix, const FOnSubmixSpectralAnalysisBP& InDelegate)
	{
		if (!IsInAudioThread())
		{
			DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.RemoveSpectralAnalysisDelegate"), STAT_RemoveSpectralAnalysisDelegate, STATGROUP_AudioThreadCommands);

			FAudioThread::RunCommandOnAudioThread([this, InSubmix, InDelegate]()
			{
				CSV_SCOPED_TIMING_STAT(Audio, RemoveSpectralAnalysisDelegate);
				RemoveSpectralAnalysisDelegate(InSubmix, InDelegate);
			}, GET_STATID(STAT_RemoveSpectralAnalysisDelegate));
			return;
		}

		// get submix if it is available.
		FMixerSubmixPtr FoundSubmix = GetSubmixInstance(InSubmix).Pin();

		if (!FoundSubmix.IsValid())
		{
			// If can't find the submix isntance, use master submix.
			FMixerSubmixWeakPtr MainSubmix = GetMasterSubmix();
			FoundSubmix = MainSubmix.Pin();
		}

		if (ensure(FoundSubmix.IsValid()))
		{
			FoundSubmix->RemoveSpectralAnalysisDelegate(InDelegate);
		}
	}

	USoundSubmix& FMixerDevice::GetMainSubmixObject() const
	{
		const int32 SubmixIndex = static_cast<int32>(ERequiredSubmixes::Main);
		USoundSubmix* MainSubmix = RequiredSubmixes[SubmixIndex];
		check(MainSubmix);
		return *MainSubmix;
	}

	void FMixerDevice::RegisterSubmixBufferListener(TSharedRef<ISubmixBufferListener, ESPMode::ThreadSafe> InSubmixBufferListener, USoundSubmix& InSubmix)
	{
		DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.RegisterSubmixBufferListener"), STAT_RegisterSubmixBufferListener, STATGROUP_AudioThreadCommands);

		const TWeakObjectPtr<USoundSubmix> SubmixPtr(&InSubmix);

		auto RegisterLambda = [this, InSubmixBufferListener, SubmixPtr]()
		{
			CSV_SCOPED_TIMING_STAT(Audio, RegisterSubmixBufferListener);

			FMixerSubmixPtr FoundSubmix = GetSubmixInstance(SubmixPtr.Get()).Pin();
			const FString& ListenerName = InSubmixBufferListener->GetListenerName();

			// Submix is not currently registered
			if (!FoundSubmix.IsValid() && SubmixPtr.IsValid())
			{
				const FString SubmixName = SubmixPtr->GetPathName();

				if (SubmixBufferListenerForceRegistrationCVar)
				{
					RegisterSoundSubmix(SubmixPtr.Get(), true /* bInit */);
					FoundSubmix = GetSubmixInstance(SubmixPtr.Get()).Pin();
					
					UE_LOGF(LogAudioMixer, Warning, "Submix '%ls' force-enabled by Submix Buffer Listener '%ls' due to CVar 'au.SubmixBufferListenerForceRegistration'."
						"Please disable the CVar and manually ensure the submix is registered before registering the submix buffer listener.", *SubmixPtr->GetName(), *ListenerName);
				}
				else
				{
					UE_LOGF(LogAudioMixer, Display, "Submix '%ls' not currently loaded, Submix buffer listener '%ls' added to pending list.", *SubmixName, *ListenerName);
					if (!PendingSubmixBufferListeners.Contains(SubmixName))
					{
						PendingSubmixBufferListeners.Add(SubmixName, TMap<FString, TSharedRef<ISubmixBufferListener, ESPMode::ThreadSafe>>());
					}

					TMap<FString, TSharedRef<ISubmixBufferListener, ESPMode::ThreadSafe>>& PendingMap = PendingSubmixBufferListeners[SubmixName];
					PendingMap.Add(ListenerName, InSubmixBufferListener);
				}
			}

			if (FoundSubmix.IsValid())
			{
				FoundSubmix->RegisterBufferListener(InSubmixBufferListener);
				UE_LOGF(LogAudioMixer, Display, "Submix buffer listener '%ls' registered with submix '%ls'", *ListenerName, *FoundSubmix->SubmixName);
			}
		};

		UE_LOGF(LogAudioMixer, Display, "Sending SubmixBufferListener '%ls' register command...", *InSubmixBufferListener->GetListenerName());
		FAudioThread::RunCommandOnAudioThread(MoveTemp(RegisterLambda));
	}

	void FMixerDevice::UnregisterSubmixBufferListener(TSharedRef<ISubmixBufferListener, ESPMode::ThreadSafe> InSubmixBufferListener, USoundSubmix& InSubmix)
	{
		DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.UnregisterSubmixBufferListener"), STAT_UnregisterSubmixBufferListener, STATGROUP_AudioThreadCommands);

		const TWeakObjectPtr<USoundSubmix> SubmixPtr(&InSubmix);
		
		FString ListenerName = InSubmixBufferListener->GetListenerName();
		
		auto UnregisterLambda = [this, SubmixPtr, InSubmixBufferListener, ListenerName]()
		{
			CSV_SCOPED_TIMING_STAT(Audio, UnregisterSubmixBufferListener);

			if (SubmixPtr.IsValid())
			{
				if (PendingSubmixBufferListeners.Contains(SubmixPtr->GetPathName()))
				{
					PendingSubmixBufferListeners[SubmixPtr->GetPathName()].Remove(InSubmixBufferListener->GetListenerName());
				}
			}
				
			FMixerSubmixPtr FoundSubmix = GetSubmixInstance(SubmixPtr.Get()).Pin();
			if (FoundSubmix.IsValid())
			{
				UE_LOGF(LogAudioMixer, Display, "Unregistering submix buffer listener '%ls' from submix '%ls'", *ListenerName, *FoundSubmix->SubmixName);
				UPTRINT ListenerPtr = reinterpret_cast<UPTRINT>(&InSubmixBufferListener.Get());
				FoundSubmix->UnregisterBufferListenerInternal(ListenerPtr);
			}
			else
			{
				UE_LOGF(LogAudioMixer, Display, "Submix buffer listener '%ls' not unregistered. Submix not loaded.", *ListenerName);
			}
		};

		FAudioThread::RunCommandOnAudioThread(MoveTemp(UnregisterLambda));
	}

	void FMixerDevice::FlushExtended(UWorld* WorldToFlush, bool bClearActivatedReverb)
	{
		QuantizedEventClockManager.Flush();
	}

	FPatchOutputStrongPtr FMixerDevice::MakePatch(int32 InFrames, int32 InChannels, float InGain) const
	{
		// Assume the mixer will consume SourceManager->GetNumOutputFrames() per iteration and an input patch will generate InFrames per iteration.
		// An input patch must have adequate space to contain as many frames as the mixer might consume, as well as as many as might be pushed to the patch.
		// This should be twice the ceiling of the ratio of the larger number of frames to the smaller number, times InFrames.
		// An output patch must have adequate space to contain as many frames as the mixer might generate, as well as as many as might be consumed from the patch.
		// This should be the same number.
		int32 MaxSizeFrames = FMath::Max(InFrames, SourceManager->GetNumOutputFrames()), MinSizeFrames = FMath::Min(InFrames, SourceManager->GetNumOutputFrames());
		return MakeShared<Audio::FPatchOutput, ESPMode::ThreadSafe>(AudioMixerPatchBufferBlocks * InFrames * FMath::DivideAndRoundUp(MaxSizeFrames, MinSizeFrames) * InChannels, InGain);
	}

	FPatchOutputStrongPtr FMixerDevice::AddPatchForSubmix(uint32 InObjectId, float InPatchGain)
	{
		if (!ensure(IsAudioRenderingThread()))
		{
			return nullptr;
		}

		FMixerSubmixPtr SubmixPtr = FindSubmixInstanceByObjectId(InObjectId);
		if (SubmixPtr.IsValid())
		{
			return SubmixPtr->AddPatch(InPatchGain);
		}

		return nullptr;
	}

	int32 FMixerDevice::GetDeviceSampleRate() const
	{
		return SampleRate;
	}

	int32 FMixerDevice::GetDeviceOutputChannels() const
	{
		return PlatformInfo.NumChannels;
	}

	FMixerSourceManager* FMixerDevice::GetSourceManager()
	{
		return SourceManager.Get();
	}

	const FMixerSourceManager* FMixerDevice::GetSourceManager() const
	{
		return SourceManager.Get();
	}

	bool FMixerDevice::IsMainAudioDevice() const
	{
		bool bIsMain = (this == FAudioDeviceManager::Get()->GetMainAudioDeviceRaw());
		return bIsMain;
	}

	void FMixerDevice::WhiteNoiseTest(FAlignedFloatBuffer& Output)
	{
		const int32 NumFrames = OpenStreamParams.NumFrames;
		const int32 NumChannels = PlatformInfo.NumChannels;

		static FWhiteNoise WhiteNoise;

		for (int32 FrameIndex = 0; FrameIndex < NumFrames; ++FrameIndex)
		{
			for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
			{
				int32 Index = FrameIndex * NumChannels + ChannelIndex;
				Output[Index] += WhiteNoise.Generate(DebugGeneratorAmpCVar, 0.f);
			}
		}
	}

	void FMixerDevice::SineOscTest(FAlignedFloatBuffer& Output)
	{
		const int32 NumFrames = OpenStreamParams.NumFrames;
		const int32 NumChannels = PlatformInfo.NumChannels;

		check(NumChannels > 0);

		// Constrain user setting if channel index not supported
		const int32 ChannelIndex = FMath::Clamp(DebugGeneratorChannelCVar, 0, NumChannels - 1);

		static FSineOsc SineOscLeft(PlatformInfo.SampleRate, DebugGeneratorFreqCVar, DebugGeneratorAmpCVar);
		static FSineOsc SineOscRight(PlatformInfo.SampleRate, DebugGeneratorFreqCVar / 2.0f, DebugGeneratorAmpCVar);

		SineOscLeft.SetFrequency(DebugGeneratorFreqCVar);
		SineOscLeft.SetScale(DebugGeneratorAmpCVar);

		if (!DebugGeneratorEnableCVar)
		{
			SineOscRight.SetFrequency(DebugGeneratorFreqCVar / 2.0f);
			SineOscRight.SetScale(DebugGeneratorAmpCVar);
		}

		for (int32 FrameIndex = 0; FrameIndex < NumFrames; ++FrameIndex)
		{
			int32 Index = FrameIndex * NumChannels;

			Output[Index + ChannelIndex] += SineOscLeft.ProcessAudio();

			// Using au. commands for debug only supports discrete channel
			if (!DebugGeneratorEnableCVar)
			{
				if (NumChannels > 1 && DebugGeneratorChannelCVar == 0)
				{
					Output[Index + 1] += SineOscRight.ProcessAudio();
				}
			}
		}
	}

	void FMixerDevice::CreateSynchronizedAudioTaskQueue(AudioTaskQueueId QueueId)
	{
		// No-op: procedural sources now render directly on the audio render thread.
	}

	void FMixerDevice::DestroySynchronizedAudioTaskQueue(AudioTaskQueueId QueueId, bool RunCurrentQueue)
	{
		// No-op: procedural sources now render directly on the audio render thread.
	}

	int FMixerDevice::KickQueuedTasks(AudioTaskQueueId QueueId)
	{
		// No-op: procedural sources now render directly on the audio render thread.
		return 0;
	}

	TUniquePtr<IAudioMixerGeneratorSource> FMixerDevice::CreateGeneratorSource(ISoundGeneratorRef InGenerator)
	{
		return MakeUnique<FAudioMixerGeneratorSource>(*this, InGenerator);
	}

	double FAudioClockTimingData::GetInterpolatedAudioClock(const double InAudioClock, const double InAudioClockDelta) const
	{
		if (UpdateTime > 0.0)
		{
			const double TargetClock = InAudioClock + InAudioClockDelta;
			const double CurrentDeltaSeconds = FPlatformTime::Seconds() - UpdateTime;
			const double Alpha = FMath::Clamp(CurrentDeltaSeconds / InAudioClockDelta, 0.0f, 1.0f);

			const double InterpolatedClock = FMath::Lerp(InAudioClock, TargetClock, Alpha);

			return InterpolatedClock;
		}

		// Fall back to quantized clock if no timing data is available
		return InAudioClock;
	}
}
