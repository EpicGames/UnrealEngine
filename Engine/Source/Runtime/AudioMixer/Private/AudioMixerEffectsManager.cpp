// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioMixerEffectsManager.h"
#include "AudioMixerDevice.h"
#include "AudioMixerSubmix.h"

namespace Audio
{
	FAudioMixerEffectsManager::FAudioMixerEffectsManager(FAudioDevice* InDevice)
		: FAudioEffectsManager(InDevice)
	{
	}

	FAudioMixerEffectsManager::~FAudioMixerEffectsManager()
	{}

	void FAudioMixerEffectsManager::SetReverbEffectParameters(const FAudioEffectParameters& InEffectParameters)
	{
		FMixerDevice* MixerDevice = (FMixerDevice*)AudioDevice;

		FMixerSubmixWeakPtr MasterReverbSubmix = MixerDevice->GetMasterReverbSubmix();
		FMixerSubmixPtr MasterReverbSubmixPtr = MasterReverbSubmix.Pin();
		
		if (MasterReverbSubmixPtr.IsValid())
		{
			bool bReportFailure = false;

			FSoundEffectSubmixPtr SoundEffectSubmix = MasterReverbSubmixPtr->GetSubmixEffect(0);
			if (SoundEffectSubmix.IsValid())
			{
				InEffectParameters.PrintSettings();

				if (!SoundEffectSubmix->SupportsDefaultReverb() || !SoundEffectSubmix->SetParameters(InEffectParameters))
				{
					bReportFailure = true;
				}
				else
				{
					InvalidReverbEffect.Reset();
				}
			}
			else
			{
				bReportFailure = true;
			}

			if (bReportFailure && InvalidReverbEffect != SoundEffectSubmix)
			{
				UE_LOGF(LogAudioMixer, Error, "Failed to update reverb parameters on Default Reverb Submix. Ensure first submix effect is supported type");
				InvalidReverbEffect = SoundEffectSubmix;
			}
		}
	}

	void FAudioMixerEffectsManager::SetEQEffectParameters(const FAudioEffectParameters& InEffectParameters)
	{
		FMixerDevice* MixerDevice = (FMixerDevice*)AudioDevice;

		FMixerSubmixWeakPtr MasterEQSubmix = MixerDevice->GetMasterEQSubmix();
		FMixerSubmixPtr MasterEQSubmixPtr = MasterEQSubmix.Pin();

		if (MasterEQSubmixPtr.IsValid())
		{
			bool bReportFailure = false;
			FSoundEffectSubmixPtr SoundEffectSubmix = MasterEQSubmixPtr->GetSubmixEffect(0);
			if (SoundEffectSubmix.IsValid())
			{
				InEffectParameters.PrintSettings();

				if (!SoundEffectSubmix->SupportsDefaultEQ() || !SoundEffectSubmix->SetParameters(InEffectParameters))
				{
					bReportFailure = true;
				}
				else
				{
					InvalidEQEffect.Reset();
				}
			}
			else
			{
				bReportFailure = true;
			}

			if (bReportFailure && InvalidEQEffect != SoundEffectSubmix)
			{
				UE_LOGF(LogAudioMixer, Error, "Failed to update EQ parameters on legacy Default EQ Submix. Ensure first submix effect is supported type");
				InvalidEQEffect = SoundEffectSubmix;
			}
		}
	}

	void FAudioMixerEffectsManager::SetRadioEffectParameters(const FAudioEffectParameters& InEffectParameters)
	{
		// Effect system deprecated
	}
}
