// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
AudioMixerEffectsManager.h: Implementation of backwards compatible effects
manager for the multi-platform audio mixer device.
=============================================================================*/

#pragma once

#include "AudioEffect.h"
#include "Curves/CurveFloat.h"
#include "Sound/SoundEffectSubmix.h"


namespace Audio
{
	class FAudioMixerEffectsManager : public FAudioEffectsManager
	{
	public:
		FAudioMixerEffectsManager(FAudioDevice* InDevice);
		~FAudioMixerEffectsManager() override;

		//~ Begin FAudioEffectsManager
		virtual void SetReverbEffectParameters(const FAudioEffectParameters& InEffectParameters) override;
		virtual void SetEQEffectParameters(const FAudioEffectParameters& InEffectParameters) override;
		virtual void SetRadioEffectParameters(const FAudioEffectParameters& InEffectParameters) override;
		//~ End FAudioEffectsManager

	protected:
		FRuntimeFloatCurve MasterReverbWetLevelCurve;

	private:
		FSoundEffectSubmixPtr InvalidReverbEffect;
		FSoundEffectSubmixPtr InvalidEQEffect;
	};
} // namespace Audio
