// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioMixerGeneratorSource.h"

#include "AudioMixerDevice.h"
#include "AudioMixerSourceManager.h"

namespace Audio
{
	FAudioMixerGeneratorSource::FAudioMixerGeneratorSource(FMixerDevice& InMixerDevice, ISoundGeneratorRef InGenerator)
		: MixerDevice(InMixerDevice)
	{
		SourceId = MixerDevice.GetSourceManager()->AllocateGeneratorSource(InGenerator);
	}

	FAudioMixerGeneratorSource::~FAudioMixerGeneratorSource()
	{
		if (SourceId != INDEX_NONE)
		{
			MixerDevice.GetSourceManager()->FreeGeneratorSource(SourceId);
		}
	}

	void FAudioMixerGeneratorSource::Play()
	{
		if (SourceId != INDEX_NONE)
		{
			MixerDevice.GetSourceManager()->PlayGeneratorSource(SourceId);
		}
	}

	void FAudioMixerGeneratorSource::Stop()
	{
		if (SourceId != INDEX_NONE)
		{
			MixerDevice.GetSourceManager()->StopGeneratorSource(SourceId);
		}
	}

	bool FAudioMixerGeneratorSource::IsDone()
	{
		if (SourceId != INDEX_NONE)
		{
			return MixerDevice.GetSourceManager()->IsGeneratorSourceDone(SourceId);
		}

		return true;
	}
}