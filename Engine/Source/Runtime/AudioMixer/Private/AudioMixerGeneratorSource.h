// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAudioMixerGeneratorSource.h"
#include "Sound/SoundGenerator.h"

namespace Audio
{
	class FMixerDevice;

	class FAudioMixerGeneratorSource : public IAudioMixerGeneratorSource
	{
		UE_NONCOPYABLE(FAudioMixerGeneratorSource);

	public:
		FAudioMixerGeneratorSource(FMixerDevice& InMixerDevice, ISoundGeneratorRef InGenerator);
		virtual ~FAudioMixerGeneratorSource() override;

		virtual void Play() override;
		virtual void Stop() override;

		virtual bool IsDone() override;
	private:
		FMixerDevice& MixerDevice;
		int32 SourceId = INDEX_NONE;
	};
}
