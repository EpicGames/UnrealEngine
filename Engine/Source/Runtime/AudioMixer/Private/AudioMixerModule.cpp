// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioMixerModule.h"
#include "Modules/ModuleManager.h"

class FAudioMixerModule : public IModuleInterface
{
public:

	virtual void StartupModule() override
	{
		FModuleManager::Get().LoadModuleChecked(TEXT("AudioMixerCore"));
		FModuleManager::Get().LoadModuleChecked(TEXT("SignalProcessing"));
	}
};

IMPLEMENT_MODULE(FAudioMixerModule, AudioMixer);
