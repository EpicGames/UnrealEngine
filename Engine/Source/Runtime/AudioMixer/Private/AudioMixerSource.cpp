// Copyright Epic Games, Inc. All Rights Reserved.
#include "AudioMixerSource.h"

#include "Audio/AudioTimingLog.h"
#include "Audio/AudioTraceUtil.h"
#include "AudioDefines.h"
#include "AudioMixerSourceBuffer.h"
#include "ActiveSound.h"
#include "AudioMixerSourceBuffer.h"
#include "AudioMixerDevice.h"
#include "AudioMixerSourceVoice.h"
#include "AudioMixerTrace.h"
#include "ContentStreaming.h"
#include "IAudioExtensionPlugin.h"
#include "IAudioModulation.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Sound/AudioSettings.h"
#include "Sound/SoundModulationDestination.h"
#include "Misc/ScopeRWLock.h"
#include "Templates/Function.h"
#include "Trace/Trace.h"
#include "Engine/Engine.h"


CSV_DECLARE_CATEGORY_MODULE_EXTERN(AUDIOMIXERCORE_API, Audio);

#if UE_AUDIO_PROFILERTRACE_ENABLED
UE_TRACE_EVENT_BEGIN(Audio, SoundWaveStart)
	UE_TRACE_EVENT_FIELD(uint32, DeviceId)
	UE_TRACE_EVENT_FIELD(uint64, Timestamp)
	UE_TRACE_EVENT_FIELD(uint32, PlayOrder)
	UE_TRACE_EVENT_FIELD(uint32, ActiveSoundPlayOrder)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, AssetPath)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, SoundClassName)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, ActorLabel)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, ActorIconName)
	UE_TRACE_EVENT_FIELD(int32, SourceBusID)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, AudioComponentName)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Audio, SoundWaveIsAlivePing)
	UE_TRACE_EVENT_FIELD(uint32, DeviceId)
	UE_TRACE_EVENT_FIELD(uint64, Timestamp)
	UE_TRACE_EVENT_FIELD(uint32, PlayOrder)
	UE_TRACE_EVENT_FIELD(uint32, ActiveSoundPlayOrder)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, AssetPath)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, SoundClassName)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, ActorLabel)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, ActorIconName)
	UE_TRACE_EVENT_FIELD(int32, SourceBusID)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, AudioComponentName)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Audio, SoundWaveStop)
	UE_TRACE_EVENT_FIELD(uint32, DeviceId)
	UE_TRACE_EVENT_FIELD(uint64, Timestamp)
	UE_TRACE_EVENT_FIELD(uint32, PlayOrder)
	UE_TRACE_EVENT_FIELD(uint32, ActiveSoundPlayOrder)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Audio, SoundRelativeRenderCost)
	UE_TRACE_EVENT_FIELD(uint32, DeviceId)
	UE_TRACE_EVENT_FIELD(uint64, Timestamp)
	UE_TRACE_EVENT_FIELD(uint32, PlayOrder)
	UE_TRACE_EVENT_FIELD(uint32, ActiveSoundPlayOrder)
	UE_TRACE_EVENT_FIELD(float, RelativeRenderCost)
UE_TRACE_EVENT_END()
#endif // UE_AUDIO_PROFILERTRACE_ENABLED


static int32 UseListenerOverrideForSpreadCVar = 0;
FAutoConsoleVariableRef CVarUseListenerOverrideForSpread(
	TEXT("au.UseListenerOverrideForSpread"),
	UseListenerOverrideForSpreadCVar,
	TEXT("Zero attenuation override distance stereo panning\n")
	TEXT("0: Use actual distance, 1: use listener override"),
	ECVF_Default);

static int32 bForceAudioLinkOnAllSourcesCVAr = 0;
FAutoConsoleVariableRef CVarForceAudioLinkOnAllSources(
	TEXT("au.AudioLink.ForceOnAllSource"),
	bForceAudioLinkOnAllSourcesCVAr,
	TEXT("0 (off), 1 (enabled). Will force AudioLink on all Sources (if the plugin is enabled)"),
	ECVF_Default);

static uint32 AudioMixerSourceFadeMinCVar = 512;
static FAutoConsoleCommand GSetAudioMixerSourceFadeMin(
	TEXT("au.SourceFadeMin"),
	TEXT("Sets the length (in samples) of minimum fade when a sound source is stopped. Must be divisible by 4 (vectorization requirement). Ignored for some procedural source types. (Default: 512, Min: 4). \n"),
	FConsoleCommandWithArgsDelegate::CreateStatic(
		[](const TArray<FString>& Args)
		{
			if (Args.Num() > 0)
			{
				const int32 SourceFadeMin = FMath::Max(FCString::Atoi(*Args[0]), 4);
				AudioMixerSourceFadeMinCVar = AlignArbitrary(SourceFadeMin, 4);
			}
		}
	)
);

#if ENABLE_AUDIO_DEBUG
static float RelativeRenderCostLogThresholdCVar = -1.f;
FAutoConsoleVariableRef CVarRelativeRenderCostLogThreshold(
	TEXT("au.RelativeRenderCostLogThresholdCVar"),
	RelativeRenderCostLogThresholdCVar,
	TEXT("Log (verbose) if the Relative Render Cost of a playing wave exceeds this threshold. " \
		"Default: -1.f (disabled), Enabled for values >= 0"),
	ECVF_Default);
#endif // if ENABLE_AUDIO_DEBUG

namespace Audio
{
	namespace MixerSourcePrivate
	{
		EMixerSourceSubmixSendStage SubmixSendStageToMixerSourceSubmixSendStage(ESubmixSendStage InSendStage)
		{
			switch(InSendStage)
			{
				case ESubmixSendStage::PreDistanceAttenuation:
					return EMixerSourceSubmixSendStage::PreDistanceAttenuation;

				case ESubmixSendStage::PostDistanceAttenuation:
				default:
					return EMixerSourceSubmixSendStage::PostDistanceAttenuation;
			}
		}
		
		int32 GetSourceBusID(const TObjectPtr<USoundWave> WaveData)
		{
			if (WaveData && WaveData->bIsSourceBus)
			{
				const TObjectPtr<USoundSourceBus> SoundSourceBus = CastChecked<USoundSourceBus>(WaveData);
				return SoundSourceBus->AudioBus ? SoundSourceBus->AudioBus->GetUniqueID() : WaveData->GetUniqueID();
			}

			return INDEX_NONE;
		}
	} // namespace MixerSourcePrivate

	namespace ModulationUtils
	{
		FSoundModulationDestinationSettings InitRoutedVolumeModulation(const FWaveInstance& InWaveInstance, const USoundWave& InWaveData, const FActiveSound& InActiveSound)
		{
			return InWaveInstance.GetEffectiveModulationSettings(EModulationDestination::Volume);
		}

		float GetRoutedVolume(const FWaveInstance& InWaveInstance, const USoundWave& InWaveData, const FActiveSound& InActiveSound)
		{
			return InWaveInstance.GetEffectiveModulationValue(EModulationDestination::Volume);
		}

		FSoundModulationDestinationSettings InitRoutedPitchModulation(const FWaveInstance& InWaveInstance, const USoundWave& InWaveData, const FActiveSound& InActiveSound)
		{
			return InWaveInstance.GetEffectiveModulationSettings(EModulationDestination::Pitch);
		}

		float GetRoutedPitch(const FWaveInstance& InWaveInstance, const USoundWave& InWaveData, const FActiveSound& InActiveSound)
		{
			return InWaveInstance.GetEffectiveModulationValue(EModulationDestination::Pitch);
		}

		FSoundModulationDestinationSettings InitRoutedHighpassModulation(const FWaveInstance& InWaveInstance, const USoundWave& InWaveData, const FActiveSound& InActiveSound)
		{
			return InWaveInstance.GetEffectiveModulationSettings(EModulationDestination::Highpass);
		}

		float GetRoutedHighpass(const FWaveInstance& InWaveInstance, const USoundWave& InWaveData, const FActiveSound& InActiveSound)
		{
			return InWaveInstance.GetEffectiveModulationValue(EModulationDestination::Highpass);
		}

		FSoundModulationDestinationSettings InitRoutedLowpassModulation(const FWaveInstance& InWaveInstance, const USoundWave& InWaveData, const FActiveSound& InActiveSound)
		{
			return InWaveInstance.GetEffectiveModulationSettings(EModulationDestination::Lowpass);
		}

		float GetRoutedLowpass(const FWaveInstance& InWaveInstance, const USoundWave& InWaveData, const FActiveSound& InActiveSound)
		{
			return InWaveInstance.GetEffectiveModulationValue(EModulationDestination::Lowpass);
		}

		FSoundModulationDefaultSettings InitRoutedModulation(const FWaveInstance& InWaveInstance, const USoundWave& InWaveData, const FActiveSound* InActiveSound)
		{
			FSoundModulationDefaultSettings Settings;
			if (InActiveSound)
			{
				Settings.VolumeModulationDestination = InitRoutedVolumeModulation(InWaveInstance, InWaveData, *InActiveSound);
				Settings.PitchModulationDestination = InitRoutedPitchModulation(InWaveInstance, InWaveData, *InActiveSound);
				Settings.HighpassModulationDestination = InitRoutedHighpassModulation(InWaveInstance, InWaveData, *InActiveSound);
				Settings.LowpassModulationDestination = InitRoutedLowpassModulation(InWaveInstance, InWaveData, *InActiveSound);
			}

			return Settings;
		}

		FSoundModulationDefaultRoutingSettings UpdateRoutedModulation(const FWaveInstance& InWaveInstance, const USoundWave& InWaveData, const FActiveSound* InActiveSound)
		{
			FSoundModulationDefaultRoutingSettings NewRouting;

			if (InActiveSound)
			{
				NewRouting.VolumeModulationDestination = InitRoutedVolumeModulation(InWaveInstance, InWaveData, *InActiveSound);
				NewRouting.PitchModulationDestination = InitRoutedPitchModulation(InWaveInstance, InWaveData, *InActiveSound);
				NewRouting.HighpassModulationDestination = InitRoutedHighpassModulation(InWaveInstance, InWaveData, *InActiveSound);
				NewRouting.LowpassModulationDestination = InitRoutedLowpassModulation(InWaveInstance, InWaveData, *InActiveSound);
			}

			return NewRouting;
		}

	} // namespace ModulationUtils

	FMixerSource::FMixerSource(FAudioDevice* InAudioDevice)
		: FSoundSource(InAudioDevice)
		, MixerDevice(static_cast<FMixerDevice*>(InAudioDevice))
		, MixerBuffer(nullptr)
		, MixerSourceVoice(nullptr)
		, bBypassingSubmixModulation(false)
		, bPreviousBusEnablement(false)
		, bPreviousBaseSubmixEnablement(false)
		, PreviousAzimuth(-1.0f)
		, PreviousPlaybackPercent(0.0f)
		, InitializationState(EMixerSourceInitializationState::NotInitialized)
		, bPlayedCachedBuffer(false)
		, bPlaying(false)
		, bLoopCallback(false)
		, bIsDone(false)
		, bIsEffectTailsDone(false)
		, bIsPlayingEffectTails(false)
		, bEditorWarnedChangedSpatialization(false)
		, bIs3D(false)
		, bDebugMode(false)
		, bIsVorbis(false)
		, bIsStoppingVoicesEnabled(InAudioDevice->IsStoppingVoicesEnabled())
		, bSendingAudioToBuses(false)
		, bPrevAllowedSpatializationSetting(false)		
	{
	}

	FMixerSource::~FMixerSource()
	{
		FreeResources();
	}

	bool FMixerSource::Is3D(const FSoundBuffer* SoundBuffer) const
	{
		// If we're not spatialized, we're not 3d.
		if (!WaveInstance->GetUseSpatialization())
		{
			return false;
		}

		// If we're using object spatialization, we can't be panned in 3d.
		if (UseObjectBasedSpatialization())
		{
			return false;
		}

		// We've got this far, and sending this source to audiolink, allow before
		// stereo check below.
		if (AudioLink.IsValid())
		{
			return true;
		}
		
		// 3d panner can't handle more than stereo currently.
		if (SoundBuffer->NumChannels > 2)
		{
			return false;
		}

		// We are 3d.
		return true;
	}

	bool FMixerSource::Init(FWaveInstance* InWaveInstance)
	{
		AUDIO_MIXER_TRACE_CPUPROFILER_EVENT_SCOPE(AudioMixerSource::Init);
		AUDIO_MIXER_CHECK(MixerBuffer);
		AUDIO_MIXER_CHECK(MixerBuffer->IsRealTimeSourceReady());

		// We've already been passed the wave instance in PrepareForInitialization, make sure we have the same one
		AUDIO_MIXER_CHECK(WaveInstance && WaveInstance == InWaveInstance);
		AUDIO_MIXER_CHECK(WaveInstance->WaveData);
		LLM_SCOPE(ELLMTag::AudioMixer);
		
		FSoundSource::InitCommon();

		NumChannels = WaveInstance->WaveData->NumChannels;
		if (!ensure(InWaveInstance))
		{
			return false;
		}

		USoundWave* WaveData = WaveInstance->WaveData;
		check(WaveData);

		if (WaveData->NumChannels == 0)
		{
			UE_LOGF(LogAudioMixer, Warning, "Soundwave %ls has invalid compressed data.", *(WaveData->GetName()));
			FreeResources();
			return false;
		}

		// Get the number of frames before creating the buffer
		int32 NumFrames = INDEX_NONE;
		if (WaveData->DecompressionType != DTYPE_Procedural)
		{
			check(!WaveData->RawPCMData || WaveData->RawPCMDataSize);
			const int32 NumBytes = WaveData->RawPCMDataSize;
			if (NumChannels > 0)
			{
				NumFrames = NumBytes / (WaveData->NumChannels * sizeof(int16));
			}
		}

		// Reset all 'previous' state.
		PreviousSubmixResolved.Reset();
		bPreviousBusEnablement = false;
		bPreviousBaseSubmixEnablement = false;
		PreviousAzimuth = -1.f;
		PreviousPlaybackPercent = 0.f;
		PreviousSubmixSends.Reset();

		// Unfortunately, we need to know if this is a vorbis source since channel maps are different for 5.1 vorbis files
		bIsVorbis = WaveData->bDecompressedFromOgg;

		bIsStoppingVoicesEnabled = AudioDevice->IsStoppingVoicesEnabled();

		bIsStopping = false;
		bIsEffectTailsDone = true;
		bIsDone = false;

		bBypassingSubmixModulation = false;

		FSoundBuffer* SoundBuffer = static_cast<FSoundBuffer*>(MixerBuffer);
		if (SoundBuffer->NumChannels > 0)
		{
			CSV_SCOPED_TIMING_STAT(Audio, InitSources);
			SCOPE_CYCLE_COUNTER(STAT_AudioSourceInitTime);

			AUDIO_MIXER_CHECK(MixerDevice);
			MixerSourceVoice = MixerDevice->GetMixerSourceVoice();
			if (!MixerSourceVoice)
			{
				FreeResources();
				UE_LOGF(LogAudioMixer, Warning, "Failed to get a mixer source voice for sound %ls.", *InWaveInstance->GetName());
				return false;
			}

			// Initialize the source voice with the necessary format information
			FMixerSourceVoiceInitParams InitParams;
			InitParams.SourceListener = this;
			InitParams.NumInputChannels = WaveData->NumChannels;
			InitParams.NumInputFrames = NumFrames;
			InitParams.SourceVoice = MixerSourceVoice;
			InitParams.bUseHRTFSpatialization = UseObjectBasedSpatialization();

			// in this file once spat override is implemented
			InitParams.bIsExternalSend = MixerDevice->GetCurrentSpatializationPluginInterfaceInfo().bSpatializationIsExternalSend;
			InitParams.bIsSoundfield = WaveInstance->bIsAmbisonics && (WaveData->NumChannels == 4);

			FActiveSound* ActiveSound = WaveInstance->ActiveSound;
			InitParams.ModulationSettings = ModulationUtils::InitRoutedModulation(*WaveInstance, *WaveData, ActiveSound);

			// Copy quantization request data
			if (WaveInstance->QuantizedRequestData)
			{
				InitParams.QuantizedRequestData = *WaveInstance->QuantizedRequestData;
			}

			if (WaveInstance->bIsAmbisonics && (WaveData->NumChannels != 4))
			{
				UE_LOGF(LogAudioMixer, Warning, "Sound wave %ls was flagged as being ambisonics but had a channel count of %d. Currently the audio engine only supports FOA sources that have four channels.", *InWaveInstance->GetName(), WaveData->NumChannels);
			}
			if (ActiveSound)
			{
				InitParams.AudioComponentUserID = WaveInstance->ActiveSound->GetAudioComponentUserID();
#if AUDIO_MIXER_ENABLE_DEBUG_MODE
				if (InitParams.AudioComponentUserID.IsNone())
				{
					InitParams.AudioComponentUserID = ActiveSound->GetSound()->GetFName();

				}
#endif // AUDIO_MIXER_ENABLE_DEBUG_MODE
				InitParams.AudioComponentID = WaveInstance->ActiveSound->GetAudioComponentID();
			}

			InitParams.EnvelopeFollowerAttackTime = WaveInstance->EnvelopeFollowerAttackTime;
			InitParams.EnvelopeFollowerReleaseTime = WaveInstance->EnvelopeFollowerReleaseTime;

			InitParams.SourceEffectChainId = 0;

			InitParams.SourceBufferListener = WaveInstance->SourceBufferListener;
			InitParams.bShouldSourceBufferListenerZeroBuffer = WaveInstance->bShouldSourceBufferListenerZeroBuffer;

			if (WaveInstance->bShouldUseAudioLink || bForceAudioLinkOnAllSourcesCVAr)
			{
				if (IAudioLinkFactory* LinkFactory = MixerDevice->GetAudioLinkFactory())
				{				
					IAudioLinkFactory::FAudioLinkSourcePushedCreateArgs CreateArgs;					
					if (WaveInstance->AudioLinkSettingsOverride)
					{
						CreateArgs.Settings = WaveInstance->AudioLinkSettingsOverride->GetProxy();
					}
					else
					{
						CreateArgs.Settings = GetDefault<UAudioLinkSettingsAbstract>(LinkFactory->GetSettingsClass())->GetProxy();
					}
					
					CreateArgs.OwnerName = *WaveInstance->GetName();			// <-- FIXME: String FName conversion.
					CreateArgs.NumChannels = SoundBuffer->NumChannels;
					CreateArgs.NumFramesPerBuffer = MixerDevice->GetBufferLength();
					CreateArgs.SampleRate = MixerDevice->GetSampleRate();
					CreateArgs.TotalNumFramesInSource = NumTotalFrames;
					AudioLink = LinkFactory->CreateSourcePushedAudioLink(CreateArgs);
					InitParams.AudioLink = AudioLink;
				}
			}

			// Source manager needs to know if this is a vorbis source for rebuilding speaker maps
			InitParams.bIsVorbis = bIsVorbis;

			// Support stereo by default
			// Check the min number of channels the source effect chain supports
			// We don't want to instantiate the effect chain if it has an effect that doesn't support its channel count
			// E.g. we shouldn't instantiate a chain on a quad source if there is an effect that only supports stereo
			InitParams.SourceEffectChainMaxSupportedChannels = WaveInstance->SourceEffectChain ? 
				WaveInstance->SourceEffectChain->GetSupportedChannelCount() :
				USoundEffectSourcePreset::DefaultSupportedChannels;

			if (InitParams.NumInputChannels <= InitParams.SourceEffectChainMaxSupportedChannels)
			{
				if (WaveInstance->SourceEffectChain)
				{
					InitParams.SourceEffectChainId = WaveInstance->SourceEffectChain->GetUniqueID();

					for (int32 i = 0; i < WaveInstance->SourceEffectChain->Chain.Num(); ++i)
					{
						InitParams.SourceEffectChain.Add(WaveInstance->SourceEffectChain->Chain[i]);
						InitParams.bPlayEffectChainTails = WaveInstance->SourceEffectChain->bPlayEffectChainTails;
					}
				}

				// Only need to care about effect chain tails finishing if we're told to play them
				if (InitParams.bPlayEffectChainTails)
				{
					bIsEffectTailsDone = false;
				}

				// Setup the bus Id if this source is a bus
				if (WaveData->bIsSourceBus)
				{
					// We need to check if the source bus has an audio bus specified
					USoundSourceBus* SoundSourceBus = CastChecked<USoundSourceBus>(WaveData);

					// If it does, we will use that audio bus as the source of the audio data for the source bus
					if (SoundSourceBus->AudioBus)
					{
						InitParams.AudioBusId = SoundSourceBus->AudioBus->GetUniqueID();
						InitParams.AudioBusChannels = (int32)SoundSourceBus->AudioBus->GetNumChannels();
#if ENABLE_AUDIO_DEBUG
						InitParams.AudioBusName = SoundSourceBus->AudioBus->GetPathName();
#endif // ENABLE_AUDIO_DEBUG
					}
					else
					{
						InitParams.AudioBusId = WaveData->GetUniqueID();
						InitParams.AudioBusChannels = WaveData->NumChannels;
#if ENABLE_AUDIO_DEBUG
						InitParams.AudioBusName = WaveData->GetPathName();
#endif // ENABLE_AUDIO_DEBUG
					}

					if (!WaveData->IsLooping())
					{
						InitParams.SourceBusDuration = WaveData->GetDuration();
					}
				}
			}

			// Toggle muting the source if sending only to output bus.
			// This can get set even if the source doesn't have bus sends since bus sends can be dynamically enabled.
			InitParams.bEnableBusSends = WaveInstance->bEnableBusSends;
			InitParams.bEnableBaseSubmix = WaveInstance->bEnableBaseSubmix && !bForceAudioLinkOnAllSourcesCVAr;
			InitParams.bEnableSubmixSends = WaveInstance->bEnableSubmixSends;
			InitParams.bHasAttenuationSubmixSends = WaveInstance->AttenuationSubmixSends.Num() > 0;
			InitParams.PlayOrder = WaveInstance->GetPlayOrder();
			InitParams.ActiveSoundPlayOrder = WaveInstance->ActiveSound != nullptr ? WaveInstance->ActiveSound->GetPlayOrder() : INDEX_NONE;
			bPreviousBusEnablement = WaveInstance->bEnableBusSends;
			DynamicBusSendInfos.Reset();

			SetupBusData(InitParams.AudioBusSends, InitParams.bEnableBusSends);

			// Don't set up any submixing if we're set to output to bus only
	
			// If we're spatializing using HRTF and its an external send, don't need to setup a default/base submix send to master or EQ submix
			// We'll only be using non-default submix sends (e.g. reverb).
			if (!(InitParams.bUseHRTFSpatialization && InitParams.bIsExternalSend))
			{
				FMixerSubmixWeakPtr SubmixPtr;
				// If a sound specifies a base submix manually, always use that
				if (WaveInstance->SoundSubmix)
				{
					SubmixPtr = MixerDevice->GetSubmixInstance(WaveInstance->SoundSubmix);
				}
				else
				{
					// Retrieve the base default submix if one is not explicitly set
					SubmixPtr = MixerDevice->GetBaseDefaultSubmix();
				}

				FMixerSourceSubmixSend SubmixSend;
				SubmixSend.Submix = SubmixPtr;
				SubmixSend.SubmixSendStage = EMixerSourceSubmixSendStage::PostDistanceAttenuation;
				SubmixSend.SendLevel = InitParams.bEnableBaseSubmix;
				SubmixSend.bIsMainSend = true;
				SubmixSend.SoundfieldFactory = MixerDevice->GetFactoryForSubmixInstance(SubmixSend.Submix);
				InitParams.SubmixSends.Add(SubmixSend);
				bPreviousBaseSubmixEnablement = InitParams.bEnableBaseSubmix;
			}
			else
			{
				// Warn about sending a source marked as Binaural directly to a soundfield submix:
				// This is a bit of a gray area as soundfield submixes are intended to be their own spatial format
				// So to send a source to this, and also flagging the source as Binaural are probably conflicting forms of spatialazition.
				FMixerSubmixWeakPtr SubmixWeakPtr = MixerDevice->GetSubmixInstance(WaveInstance->SoundSubmix);

				if (FMixerSubmixPtr SubmixPtr = SubmixWeakPtr.Pin())
				{
					if ((SubmixPtr->IsSoundfieldSubmix() || SubmixPtr->IsSoundfieldEndpointSubmix()))
					{
						UE_LOGF(LogAudioMixer, Warning, "Ignoring soundfield Base Submix destination being set on SoundWave (%ls) because spatialization method is set to Binaural."
							, *InWaveInstance->GetName());
					}
					
					bBypassingSubmixModulation = true;
				}
			}

			// Add submix sends for this source
			for (FSoundSubmixSendInfo& SendInfo : WaveInstance->SoundSubmixSends)
			{
				if (SendInfo.SoundSubmix != nullptr)
				{
					FMixerSourceSubmixSend SubmixSend;
					SubmixSend.Submix = MixerDevice->GetSubmixInstance(SendInfo.SoundSubmix);

					SubmixSend.SubmixSendStage = EMixerSourceSubmixSendStage::PostDistanceAttenuation;
					if (SendInfo.SendStage == ESubmixSendStage::PreDistanceAttenuation)
					{
						SubmixSend.SubmixSendStage = EMixerSourceSubmixSendStage::PreDistanceAttenuation;
					}
					if (!WaveInstance->bEnableSubmixSends)
					{
						SubmixSend.SendLevel = 0.0f;
					}
					else
					{
						SubmixSend.SendLevel = SendInfo.SendLevel;
					}
					
					SubmixSend.bIsMainSend = false;
					SubmixSend.SoundfieldFactory = MixerDevice->GetFactoryForSubmixInstance(SubmixSend.Submix);
					SubmixSend.Filter = FSendFilterSettings(SendInfo.bEnableLPFCutoff != 0, SendInfo.bEnableHPFCutoff != 0,
						SendInfo.LPFCutoff, SendInfo.HPFCutoff);
					InitParams.SubmixSends.Add(SubmixSend);
				}
			}

			// Loop through all submix sends to figure out what speaker maps this source is using
			for (FMixerSourceSubmixSend& Send : InitParams.SubmixSends)
			{
				FMixerSubmixPtr SubmixPtr = Send.Submix.Pin();
				if (SubmixPtr.IsValid())
				{
					FRWScopeLock Lock(ChannelMapLock, SLT_Write);
					ChannelMap.Reset();
				}
			}

			// Check to see if this sound has been flagged to be in debug mode
#if AUDIO_MIXER_ENABLE_DEBUG_MODE
			InitParams.DebugName = WaveInstance->GetName();

			bool bIsDebug = false;
			FString WaveInstanceName = WaveInstance->GetName();
			FString TestName = GEngine->GetAudioDeviceManager()->GetDebugger().GetAudioMixerDebugSoundName();
			if (!TestName.IsEmpty() && WaveInstanceName.Contains(TestName))
			{
				bDebugMode = true;
				InitParams.bIsDebugMode = bDebugMode;
			}
#endif

			UE_CLOGF(Audio::MatchesLogFilter(*WaveInstance->GetName()), LogAudioTiming, Verbose,
				"FMixerSource::Init Name=%ls,BufferType=%d,CachedRealtimeFirstBuffer=0x%p",*WaveInstance->GetName(),
					(int32)MixerBuffer->GetType(),WaveData->CachedRealtimeFirstBuffer);

			// Whether or not we're 3D
			bIs3D = Is3D(SoundBuffer);

			// Pass on the fact that we're 3D to the init params
			InitParams.bIs3D = bIs3D;

			// Grab the source's reverb plugin settings
			InitParams.SpatializationPluginSettings = UseSpatializationPlugin() ? WaveInstance->SpatializationPluginSettings : nullptr;

			// Grab the source's occlusion plugin settings
			InitParams.OcclusionPluginSettings = UseOcclusionPlugin() ? WaveInstance->OcclusionPluginSettings : nullptr;

			// Grab the source's reverb plugin settings
			InitParams.ReverbPluginSettings = UseReverbPlugin() ? WaveInstance->ReverbPluginSettings : nullptr;

			// Grab the source's source data override plugin settings
			InitParams.SourceDataOverridePluginSettings = UseSourceDataOverridePlugin() ? WaveInstance->SourceDataOverridePluginSettings : nullptr;

			// Update the buffer sample rate to the wave instance sample rate, as it could have changed during decoder parse.
			MixerBuffer->InitSampleRate(WaveData->GetSampleRateForCurrentPlatform());
			MixerBuffer->InitNumFrames(WaveData->GetNumFrames());

			// Retrieve the raw pcm buffer data and the precached buffers before initializing so we can avoid having USoundWave ptrs in audio renderer thread
			EBufferType::Type BufferType = MixerBuffer->GetType();
			if (BufferType == EBufferType::PCM || BufferType == EBufferType::PCMPreview)
			{
				FRawPCMDataBuffer RawPCMDataBuffer;
				MixerBuffer->GetPCMData(&RawPCMDataBuffer.Data, &RawPCMDataBuffer.DataSize);
				MixerSourceBuffer->SetPCMData(RawPCMDataBuffer);
			}
#if PLATFORM_NUM_AUDIODECOMPRESSION_PRECACHE_BUFFERS > 0
			else if (BufferType == EBufferType::PCMRealTime || BufferType == EBufferType::Streaming)
			{
				if (WaveData->CachedRealtimeFirstBuffer)
				{
					const uint32 NumPrecacheSamples = (uint32)(WaveData->NumPrecacheFrames * WaveData->NumChannels);
					const uint32 BufferSize = NumPrecacheSamples * sizeof(int16) * PLATFORM_NUM_AUDIODECOMPRESSION_PRECACHE_BUFFERS;

					TArray<uint8> PrecacheBufferCopy;
					PrecacheBufferCopy.AddUninitialized(BufferSize);

					FMemory::Memcpy(PrecacheBufferCopy.GetData(), WaveData->CachedRealtimeFirstBuffer, BufferSize);

					MixerSourceBuffer->SetCachedRealtimeFirstBuffers(MoveTemp(PrecacheBufferCopy));
				}
			}
#endif

			// Pass the decompression state off to the mixer source buffer if it hasn't already done so
			ICompressedAudioInfo* Decoder = MixerBuffer->GetDecompressionState(true);
			MixerSourceBuffer->SetDecoder(Decoder);

			// Hand off the mixer source buffer decoder
			InitParams.MixerSourceBuffer = MixerSourceBuffer;
			MixerSourceBuffer = nullptr;

			UE_LOGF(LogAudioMixer, Verbose, "Initializing mixer source voice: %ls", *InWaveInstance->GetName());
			if (MixerSourceVoice->Init(InitParams))
			{
				// Initialize the propagation interface as soon as we have a valid source id
				if (AudioDevice->SourceDataOverridePluginInterface)
				{
					uint32 SourceId = MixerSourceVoice->GetSourceId();
					AudioDevice->SourceDataOverridePluginInterface->OnInitSource(SourceId, InitParams.AudioComponentUserID, InitParams.SourceDataOverridePluginSettings);
				}

				InitializationState = EMixerSourceInitializationState::Initialized;

				Update();

				return true;
			}
			else
			{
				InitializationState = EMixerSourceInitializationState::NotInitialized;
				UE_LOGF(LogAudioMixer, Warning, "Failed to initialize mixer source voice '%ls'.", *InWaveInstance->GetName());
			}
		}
		else
		{
			UE_LOGF(LogAudioMixer, Warning, "Num channels was 0 for sound buffer '%ls'.", *InWaveInstance->GetName());
		}

		FreeResources();
		return false;
	}

	void FMixerSource::SetupBusData(TArray<FInitAudioBusSend>* OutAudioBusSends, bool bEnableBusSends)
	{
		for (int32 BusSendType = 0; BusSendType < (int32)EBusSendType::Count; ++BusSendType)
		{
			// And add all the source bus sends
			for (FSoundSourceBusSendInfo& SendInfo : WaveInstance->BusSends[BusSendType])
			{
				// Avoid redoing duplicate code for sending audio to source bus or audio bus. Most of it is the same other than the bus id.
				auto SetupBusSend = [this](TArray<FInitAudioBusSend>* AudioBusSends, const FSoundSourceBusSendInfo& InSendInfo, int32 InBusSendType, uint32 InBusId, bool bEnableBusSends, int32 InBusChannels, const FString& InBusName)
				{
					FInitAudioBusSend BusSend;
					BusSend.AudioBusId = InBusId;
					BusSend.BusChannels = InBusChannels;
#if ENABLE_AUDIO_DEBUG
					BusSend.AudioBusName = InBusName;
#endif // ENABLE_AUDIO_DEBUG
					
					if(bEnableBusSends)
					{
						BusSend.SendLevel = InSendInfo.SendLevel;
					}
					else
					{
						BusSend.SendLevel = 0;
					}

					BusSend.Filter = FSendFilterSettings(InSendInfo.bEnableLPFCutoff != 0, InSendInfo.bEnableHPFCutoff != 0,
						InSendInfo.LPFCutoff, InSendInfo.HPFCutoff);

					if (AudioBusSends)
					{
						AudioBusSends[InBusSendType].Add(BusSend);
					}

					FDynamicBusSendInfo NewDynamicBusSendInfo;
					NewDynamicBusSendInfo.SendLevel = InSendInfo.SendLevel;
					NewDynamicBusSendInfo.BusId = BusSend.AudioBusId;
#if ENABLE_AUDIO_DEBUG
					NewDynamicBusSendInfo.BusName = BusSend.AudioBusName;
#endif // ENABLE_AUDIO_DEBUG
					NewDynamicBusSendInfo.BusSendLevelControlMethod = InSendInfo.SourceBusSendLevelControlMethod;
					NewDynamicBusSendInfo.BusSendType = (EBusSendType)InBusSendType;
					NewDynamicBusSendInfo.MinSendLevel = InSendInfo.MinSendLevel;
					NewDynamicBusSendInfo.MaxSendLevel = InSendInfo.MaxSendLevel;
					NewDynamicBusSendInfo.MinSendDistance = InSendInfo.MinSendDistance;
					NewDynamicBusSendInfo.MaxSendDistance = InSendInfo.MaxSendDistance;
					NewDynamicBusSendInfo.CustomSendLevelCurve = InSendInfo.CustomSendLevelCurve;
					NewDynamicBusSendInfo.Filter = FSendFilterSettings(InSendInfo.bEnableLPFCutoff != 0, InSendInfo.bEnableHPFCutoff != 0,
						InSendInfo.LPFCutoff, InSendInfo.HPFCutoff);

					// Copy the bus SourceBusSendInfo structs to a local copy so we can update it in the update tick
					bool bIsNew = true;
					for (FDynamicBusSendInfo& BusSendInfo : DynamicBusSendInfos)
					{
						if (BusSendInfo.BusId == NewDynamicBusSendInfo.BusId)
						{
							// Detect filter param changes before overwriting so the dirty-check can dispatch them.
							const bool bFilteringChanged = BusSendInfo.Filter.HasChanged(NewDynamicBusSendInfo.Filter);
							BusSendInfo = NewDynamicBusSendInfo;
							BusSendInfo.bFilterStateChanged |= bFilteringChanged;
							BusSendInfo.bIsInit = false;
							bIsNew = false;
							break;
						}
					}

					if (bIsNew)
					{
						DynamicBusSendInfos.Add(NewDynamicBusSendInfo);
					}

					// Flag that we're sending audio to buses so we can check for updates to send levels
					bSendingAudioToBuses = true;
				};

				// Retrieve bus id of the audio bus to use
				if (SendInfo.SoundSourceBus)
				{						
					uint32 BusId;
					int32 BusChannels;
					FString BusName;

					// Either use the bus id of the source bus's audio bus id if it was specified
					if (SendInfo.SoundSourceBus->AudioBus)
					{
						BusId = SendInfo.SoundSourceBus->AudioBus->GetUniqueID();
						BusChannels = (int32)SendInfo.SoundSourceBus->AudioBus->GetNumChannels();
#if ENABLE_AUDIO_DEBUG
						BusName = SendInfo.SoundSourceBus->AudioBus.GetPathName();
#endif // ENABLE_AUDIO_DEBUG
					}
					else
					{
						// otherwise, use the id of the source bus itself (for an automatic source bus)
						BusId = SendInfo.SoundSourceBus->GetUniqueID();
						BusChannels = SendInfo.SoundSourceBus->NumChannels;
#if ENABLE_AUDIO_DEBUG
						BusName = SendInfo.SoundSourceBus->GetPathName();
#endif // ENABLE_AUDIO_DEBUG
					}

					// Call lambda w/ the correctly derived bus id
					SetupBusSend(OutAudioBusSends, SendInfo, BusSendType, BusId, bEnableBusSends, BusChannels, BusName);
				}

				if (SendInfo.AudioBus)
				{
					// Only need to send audio to just the specified audio bus
					uint32 BusId = SendInfo.AudioBus->GetUniqueID();
					int32 BusChannels = (int32)SendInfo.AudioBus->AudioBusChannels + 1;
					FString BusName;

#if ENABLE_AUDIO_DEBUG
					BusName = SendInfo.AudioBus->GetPathName();
#endif // ENABLE_AUDIO_DEBUG

					// Note we will be sending audio to both the specified source bus and the audio bus with the same send level
					SetupBusSend(OutAudioBusSends, SendInfo, BusSendType, BusId, bEnableBusSends, BusChannels, BusName);
				}
			}
		}
	}

	void FMixerSource::Update()
	{
		CSV_SCOPED_TIMING_STAT(Audio, UpdateSources);
		SCOPE_CYCLE_COUNTER(STAT_AudioUpdateSources);

		LLM_SCOPE(ELLMTag::AudioMixer);

		if (!WaveInstance || !MixerSourceVoice || Paused || InitializationState == EMixerSourceInitializationState::NotInitialized)
		{
			return;
		}

		AUDIO_MIXER_TRACE_CPUPROFILER_EVENT_SCOPE(FMixerSource::Update);

		// if MarkAsGarbage() was called, WaveInstance->WaveData is null
		if (!WaveInstance->WaveData)
		{
			StopNow();
			return;
		}

		++TickCount;
		
		UE_CLOGF(Audio::MatchesLogFilter(*WaveInstance->GetName()), LogAudioTiming, Verbose,
			"FMixerSource::Update, Name=%ls, StartTime=%.2f bProcedural=%d, bIsSourceBus=%d, CachedRealTimeFirstBuffer=0x%p, MixerSourceBufferValid=%ls, TickCount=%d",
			*WaveInstance->GetName(), WaveInstance->StartTime, WaveInstance->WaveData->bProcedural, WaveInstance->WaveData->bIsSourceBus,
			WaveInstance->WaveData->CachedRealtimeFirstBuffer,ToCStr(LexToString(MixerSourceBuffer.IsValid())), TickCount);

		// Allow plugins to override any data in a waveinstance
		if (AudioDevice->SourceDataOverridePluginInterface && WaveInstance->bEnableSourceDataOverride)
		{
			uint32 SourceId = MixerSourceVoice->GetSourceId();
			int32 ListenerIndex = WaveInstance->ActiveSound->GetClosestListenerIndex();

			FTransform ListenerTransform;
			AudioDevice->GetListenerTransform(ListenerIndex, ListenerTransform);

			AudioDevice->SourceDataOverridePluginInterface->GetSourceDataOverrides(SourceId, ListenerTransform, WaveInstance);
		}

		// AudioLink, push state if we're enabled and 3d.
		if (bIs3D && AudioLink.IsValid())
		{
			IAudioLinkSourcePushed::FOnUpdateWorldStateParams Params;
			Params.WorldTransform = WaveInstance->ActiveSound->Transform;
			AudioLink->OnUpdateWorldState(Params);
		}

		UpdateModulation();

		UpdatePitch();

		UpdateVolume();

		UpdateSpatialization();

		UpdateEffects();

		UpdateSourceBusSends();

		UpdateChannelMaps();

		UpdateRelativeRenderCost();

#if ENABLE_AUDIO_DEBUG
		UpdateCPUCoreUtilization();

		Audio::FAudioDebugger::DrawDebugInfo(*this);
#endif // ENABLE_AUDIO_DEBUG

#if UE_AUDIO_PROFILERTRACE_ENABLED
		UpdateIsAlivePingTraceMessage();
#endif // UE_AUDIO_PROFILERTRACE_ENABLED
	}

	bool FMixerSource::PrepareForInitialization(FWaveInstance* InWaveInstance)
	{
		LLM_SCOPE(ELLMTag::AudioMixer);

		if (!ensure(InWaveInstance))
		{
			return false;
		}

		// We are currently not supporting playing audio on a controller
		if (InWaveInstance->OutputTarget == EAudioOutputTarget::Controller)
		{
			return false;
		}

		AUDIO_MIXER_TRACE_CPUPROFILER_EVENT_SCOPE(AudioMixerSource::PrepareForInitialization);

		// We are not initialized yet. We won't be until the sound file finishes loading and parsing the header.
		InitializationState = EMixerSourceInitializationState::Initializing;

		//  Reset so next instance will warn if algorithm changes in-flight
		bEditorWarnedChangedSpatialization = false;

		const bool bIsSeeking = InWaveInstance->StartTime > 0.0f;

		check(InWaveInstance);
		check(AudioDevice);

		check(!MixerBuffer);
		MixerBuffer = FMixerBuffer::Init(AudioDevice, InWaveInstance->WaveData, bIsSeeking /* bForceRealtime */);

		if (!MixerBuffer)
		{
			FreeResources(); // APM: maybe need to call this here too? 
			return false;
		}

		// WaveData must be valid beyond this point, otherwise MixerBuffer
		// would have failed to init.
		check(InWaveInstance->WaveData);
		USoundWave& SoundWave = *InWaveInstance->WaveData;

		WaveInstance = InWaveInstance;

		LPFFrequency = MAX_FILTER_FREQUENCY;

		HPFFrequency = 0.0f;

		bIsDone = false;

		// Not all wave data types have a non-zero duration
		if (SoundWave.Duration > 0.0f)
		{
			if (!SoundWave.bIsSourceBus)
			{
				NumTotalFrames = SoundWave.Duration * SoundWave.GetSampleRateForCurrentPlatform();
				check(NumTotalFrames > 0);
			}
			else if (!SoundWave.IsLooping())
			{
				NumTotalFrames = SoundWave.Duration * AudioDevice->GetSampleRate();
				check(NumTotalFrames > 0);
			}

			StartFrame = FMath::Clamp<int32>((InWaveInstance->StartTime / SoundWave.Duration) * NumTotalFrames, 0, NumTotalFrames);
		}

		check(!MixerSourceBuffer.IsValid());

		// Active sound instance ID is the audio component ID of active sound.
		uint64 InstanceID = 0;
		uint32 PlayOrder = 0;
		bool bActiveSoundIsPreviewSound = false;
		TArray<FAudioParameter> DefaultParameters;
		FActiveSound* ActiveSound = WaveInstance->ActiveSound;
		if (ActiveSound)
		{
			InstanceID = ActiveSound->GetAudioComponentID();
			PlayOrder = ActiveSound->GetPlayOrder();
			bActiveSoundIsPreviewSound = ActiveSound->bIsPreviewSound;
			if (Audio::IParameterTransmitter* Transmitter = ActiveSound->GetTransmitter())
			{
				// This copying of parameters is for the case of virtual loop realization. 
				// The most up-to-date parameters exist on the instance transmitter. 
				Transmitter->CopyParameters(DefaultParameters);
 				SoundWave.InitParameters(DefaultParameters);
			}
		}

		FMixerSourceBufferInitArgs BufferInitArgs;
		BufferInitArgs.AudioDeviceID = AudioDevice->DeviceID;
		BufferInitArgs.AudioComponentID = InstanceID;
		BufferInitArgs.InstanceID = GetTransmitterID(InstanceID, WaveInstance->WaveInstanceHash, PlayOrder);
		BufferInitArgs.SampleRate = AudioDevice->GetSampleRate();
		BufferInitArgs.AudioMixerNumOutputFrames = MixerDevice->GetNumOutputFrames();
		BufferInitArgs.Buffer = MixerBuffer;
		BufferInitArgs.SoundWave = &SoundWave;
		BufferInitArgs.LoopingMode = InWaveInstance->LoopingMode;
		BufferInitArgs.bIsSeeking = bIsSeeking;
		BufferInitArgs.bIsPreviewSound = bActiveSoundIsPreviewSound;
		BufferInitArgs.StartTime = InWaveInstance->StartTime;

		MixerSourceBuffer = FMixerSourceBuffer::Create(BufferInitArgs, MoveTemp(DefaultParameters));
		
		if (!MixerSourceBuffer.IsValid())
		{
			FreeResources();

			// Guarantee that this wave instance does not try to replay by disabling looping.
			WaveInstance->LoopingMode = LOOP_Never;

			if (ensure(ActiveSound))
			{
				ActiveSound->bShouldRemainActiveIfDropped = false;
			}
		}
		
		UE_CLOGF(Audio::MatchesLogFilter(*WaveInstance->GetName()), LogAudioTiming, Verbose,
			"FMixerSource::PrepareForInitialization, Name=%ls, StartTime=%.2f bProcedural=%d, bIsSourceBus=%d, CachedRealTimeFirstBuffer=0x%p, MixerSourceBufferValid=%ls",
			*WaveInstance->GetName(), WaveInstance->StartTime, WaveInstance->WaveData->bProcedural, WaveInstance->WaveData->bIsSourceBus,
			WaveInstance->WaveData->CachedRealtimeFirstBuffer,ToCStr(LexToString(MixerSourceBuffer.IsValid())));
		
		return MixerSourceBuffer.IsValid();
	}

	bool FMixerSource::IsPreparedToInit()
	{
		LLM_SCOPE(ELLMTag::AudioMixer);
		AUDIO_MIXER_TRACE_CPUPROFILER_EVENT_SCOPE(AudioMixerSource::IsPreparedToInit);

		if (MixerBuffer && MixerBuffer->IsRealTimeSourceReady())
		{
			check(MixerSourceBuffer.IsValid());

			// Check if we have a realtime audio task already (doing first decode)
			if (MixerSourceBuffer->IsAsyncTaskInProgress())
			{
				const bool bAsyncTaskDone = MixerSourceBuffer->IsAsyncTaskDone();
				UE_CLOGF(Audio::MatchesLogFilter(*WaveInstance->GetName()), LogAudioTiming, Verbose,
					"FMixerSource::IsPreparedToInit (not ready), Name=%ls, StartTime=%.2f bProcedural=%d, bIsSourceBus=%d, CachedRealTimeFirstBuffer=0x%p, IsAyncTaskDone=%ls, (IsAsyncTaskInProgress)",
					*WaveInstance->GetName(), WaveInstance->StartTime, WaveInstance->WaveData->bProcedural, WaveInstance->WaveData->bIsSourceBus, WaveInstance->WaveData->CachedRealtimeFirstBuffer,ToCStr(LexToString(bAsyncTaskDone)));
				
				// not ready
				return bAsyncTaskDone;
			}
			else if (WaveInstance)
			{
				if (WaveInstance->WaveData->bIsSourceBus)
				{
					// Buses don't need to do anything to play audio
					return true;
				}
				else
				{
					// Now check to see if we need to kick off a decode the first chunk of audio
					const EBufferType::Type BufferType = MixerBuffer->GetType();
					if ((BufferType == EBufferType::PCMRealTime || BufferType == EBufferType::Streaming) && WaveInstance->WaveData && !WaveInstance->WaveData->bProcedural)
					{
						// If any of these conditions meet, we need to do an initial async decode before we're ready to start playing the sound
						if (WaveInstance->StartTime > 0.0f || WaveInstance->WaveData->bIsSourceBus || !WaveInstance->WaveData->CachedRealtimeFirstBuffer)
						{
							// Before reading more PCMRT data, we first need to seek the buffer
							if (WaveInstance->IsSeekable())
							{
								MixerBuffer->Seek(WaveInstance->StartTime);
							}

							check(MixerSourceBuffer.IsValid());

							UE_CLOGF(Audio::MatchesLogFilter(*WaveInstance->GetName()), LogAudioTiming, Verbose,
								"FMixerSource::IsPreparedToInit (not ready), Name=%ls, StartTime=%.2f bProcecural=%d, bIsSourceBus=%d, CachedRealTimeFirstBuffer=0x%p, (Kicking Off Initial Async Decode)",
								*WaveInstance->GetName(), WaveInstance->StartTime, WaveInstance->WaveData->bProcedural, WaveInstance->WaveData->bIsSourceBus, WaveInstance->WaveData->CachedRealtimeFirstBuffer);

							ICompressedAudioInfo* Decoder = MixerBuffer->GetDecompressionState(false);
							MixerSourceBuffer->ReadMoreRealtimeData(Decoder, 0, EBufferReadMode::Asynchronous);

							// not ready
							return false;
						}
					}
				}
			}

			return true;
		}
		UE_CLOGF(Audio::MatchesLogFilter(*WaveInstance->GetName()), LogAudioTiming, Verbose,
			"FMixerSource::IsPreparedToInit (not ready), Name=%ls, StartTime=%.2f bProcedural=%d, bIsSourceBus=%d, CachedRealTimeFirstBuffer=0x%p, MixerBuffer=0x%p, IsRealTimeSourceReady=%ls (realtime source not ready, or no mixer buffer)",
			*WaveInstance->GetName(), WaveInstance->StartTime, WaveInstance->WaveData->bProcedural, WaveInstance->WaveData->bIsSourceBus, WaveInstance->WaveData->CachedRealtimeFirstBuffer,
				MixerBuffer, ToCStr(LexToString(MixerBuffer->IsRealTimeSourceReady())));
		
		return false;
	}

	bool FMixerSource::IsInitialized() const
	{
		return InitializationState == EMixerSourceInitializationState::Initialized;
	}

	void FMixerSource::Play()
	{
		if (!WaveInstance)
		{
			return;
		}

		// Don't restart the sound if it was stopping when we paused, just stop it.
		if (Paused && (bIsStopping || bIsDone))
		{
			StopNow();
			return;
		}

		if (bIsStopping)
		{
			UE_LOGF(LogAudioMixer, Warning, "Restarting a source which was stopping. Stopping now.");
			return;
		}

		AUDIO_MIXER_TRACE_CPUPROFILER_EVENT_SCOPE(AudioMixerSource::Play);

		UE_CLOGF(WaveInstance && Audio::MatchesLogFilter(*WaveInstance->GetName()), LogAudioTiming, Verbose, 
			"FMixerSource::Play, Name=%ls, StartTime=%.2f bProcedural=%d, bIsSourceBus=%d, CachedRealTimeFirstBuffer=0x%p, MixerSourceBufferValid=%ls, TickCount=%d, InitState=%ls",
			*WaveInstance->GetName(), WaveInstance->StartTime, WaveInstance->WaveData->bProcedural, WaveInstance->WaveData->bIsSourceBus,
			WaveInstance->WaveData->CachedRealtimeFirstBuffer,ToCStr(LexToString(MixerSourceBuffer.IsValid())), TickCount, ToCStr(LexToString(InitializationState)));

		// It's possible if Pause and Play are called while a sound is async initializing. In this case
		// we'll just not actually play the source here. Instead we'll call play when the sound finishes loading.
		if (MixerSourceVoice && InitializationState == EMixerSourceInitializationState::Initialized)
		{
			UE_CLOGF(WaveInstance && Audio::MatchesLogFilter(*WaveInstance->GetName()), LogAudioTiming, Verbose,
				"FMixerSourceVoice::Play, Name=%ls, StartTime=%.2f bProcedural=%d, bIsSourceBus=%d, CachedRealTimeFirstBuffer=0x%p, MixerSourceBufferValid=%ls, TickCount=%d, InitState=%ls",
				*WaveInstance->GetName(), WaveInstance->StartTime, WaveInstance->WaveData->bProcedural, WaveInstance->WaveData->bIsSourceBus,
				WaveInstance->WaveData->CachedRealtimeFirstBuffer,ToCStr(LexToString(MixerSourceBuffer.IsValid())), TickCount, ToCStr(LexToString(InitializationState)));

			MixerSourceVoice->Play();
		}

#if UE_AUDIO_PROFILERTRACE_ENABLED
		SendSoundWaveStartTraceMessage();
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

		bIsStopping = false;
		Paused = false;
		Playing = true;
		bLoopCallback = false;
		bIsDone = false;
	}

	void FMixerSource::Stop()
	{
		LLM_SCOPE(ELLMTag::AudioMixer);

		if (InitializationState == EMixerSourceInitializationState::NotInitialized)
		{
			return;
		}

		if (!MixerSourceVoice)
		{
			StopNow();
			return;
		}

		USoundWave* SoundWave = WaveInstance ? WaveInstance->WaveData.Get() : nullptr;

		// If MarkAsGarbage() was called, SoundWave can be null
		if (!SoundWave)
		{
			StopNow();
			return;
		}

		// Stop procedural sounds immediately that don't require fade
		if (SoundWave->bProcedural && !SoundWave->bRequiresStopFade)
		{
			StopNow();
			return;
		}

		if (bIsDone)
		{
			StopNow();
			return;
		}

		if (Playing && !bIsStoppingVoicesEnabled)
		{
			StopNow();
			return;
		}

		// Otherwise, we need to do a quick fade-out of the sound and put the state
		// of the sound into "stopping" mode. This prevents this source from
		// being put into the "free" pool and prevents the source from freeing its resources
		// until the sound has finished naturally (i.e. faded all the way out)

		// Let the wave instance know it's stopping
		if (!bIsStopping)
		{
			WaveInstance->SetStopping(true);

			MixerSourceVoice->StopFade(AudioMixerSourceFadeMinCVar);
			bIsStopping = true;
			Paused = false;
		}
	}

	void FMixerSource::StopNow()
	{
		LLM_SCOPE(ELLMTag::AudioMixer);

		// Immediately stop the sound source

		InitializationState = EMixerSourceInitializationState::NotInitialized;

		bIsStopping = false;

		if (WaveInstance)
		{
			if (MixerSourceVoice && Playing)
			{
#if UE_AUDIO_PROFILERTRACE_ENABLED
				SendSoundWaveStopTraceMessage();
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

				MixerSourceVoice->Stop();
			}

			Paused = false;
			Playing = false;

			FreeResources();
		}

		FSoundSource::Stop();
	}

	void FMixerSource::Pause()
	{
		if (!WaveInstance)
		{
			return;
		}

		if (bIsStopping)
		{
			return;
		}

		if (MixerSourceVoice)
		{
			MixerSourceVoice->Pause();
		}

		Paused = true;
	}

	bool FMixerSource::IsFinished()
	{
		// A paused source is not finished.
		if (Paused)
		{
			return false;
		}

		if (InitializationState == EMixerSourceInitializationState::NotInitialized)
		{
			return true;
		}

		if (InitializationState == EMixerSourceInitializationState::Initializing)
		{
			return false;
		}

		if (WaveInstance && MixerSourceVoice)
		{
			if (bIsDone && bIsEffectTailsDone)
			{
				WaveInstance->NotifyFinished();
				bIsStopping = false;
				return true;
			}
			else if (bLoopCallback && WaveInstance->LoopingMode == LOOP_WithNotification)
			{
				WaveInstance->NotifyFinished();
				bLoopCallback = false;
			}

			return false;
		}
		return true;
	}

	float FMixerSource::GetPlaybackPercent() const
	{
		if (InitializationState != EMixerSourceInitializationState::Initialized)
		{
			return PreviousPlaybackPercent;
		}

		if (MixerSourceVoice && NumTotalFrames > 0)
		{
			int64 NumFrames = StartFrame + MixerSourceVoice->GetNumFramesPlayed();
			AUDIO_MIXER_CHECK(NumTotalFrames > 0);
			PreviousPlaybackPercent = (float)NumFrames / NumTotalFrames;
			if (WaveInstance->LoopingMode == LOOP_Never)
			{
				PreviousPlaybackPercent = FMath::Min(PreviousPlaybackPercent, 1.0f);
			}
			return PreviousPlaybackPercent;
		}
		else
		{
			// If we don't have any frames, that means it's a procedural sound wave, which means
			// that we're never going to have a playback percentage.
			return 1.0f;
		}
	}

	int64 FMixerSource::GetNumFramesPlayed() const
	{
		if (InitializationState == EMixerSourceInitializationState::Initialized && MixerSourceVoice != nullptr)
		{
			return MixerSourceVoice->GetNumFramesPlayed();
		}

		return 0;
	}

	float FMixerSource::GetEnvelopeValue() const
	{
		if (MixerSourceVoice)
		{
			return MixerSourceVoice->GetEnvelopeValue();
		}
		return 0.0f;
	}

	float FMixerSource::GetRelativeRenderCost() const
	{
		if (MixerSourceVoice)
		{
			return MixerSourceVoice->GetRelativeRenderCost();
		}
		return 1.0f;
	}

	void FMixerSource::OnBeginGenerate()
	{
	}

	void FMixerSource::OnDone()
	{
		bIsDone = true;
	}

	void FMixerSource::OnEffectTailsDone()
	{
		bIsEffectTailsDone = true;
	}

	void FMixerSource::FreeResources()
	{
		LLM_SCOPE(ELLMTag::AudioMixer);

		if (MixerBuffer)
		{
			MixerBuffer->EnsureHeaderParseTaskFinished();
		}

		check(!bIsStopping);
		check(!Playing);

		if (AudioLink.IsValid())
		{
			AudioLink.Reset();
		}

		// Make a new pending release data ptr to pass off release data
		if (MixerSourceVoice)
		{
			// Release the source using the propagation interface
			if (AudioDevice->SourceDataOverridePluginInterface)
			{
				uint32 SourceId = MixerSourceVoice->GetSourceId();
				AudioDevice->SourceDataOverridePluginInterface->OnReleaseSource(SourceId);
			}

			// We're now "releasing" so don't recycle this voice until we get notified that the source has finished
			bIsReleasing = true;

			// This will trigger FMixerSource::OnRelease from audio render thread.
			MixerSourceVoice->Release();
			MixerSourceVoice = nullptr;
		}

		MixerSourceBuffer.Reset();
		bLoopCallback = false;
		NumTotalFrames = 0;

		if (MixerBuffer)
		{
			EBufferType::Type BufferType = MixerBuffer->GetType();
			if (BufferType == EBufferType::PCMRealTime || BufferType == EBufferType::Streaming)
			{
				delete MixerBuffer;
			}

			MixerBuffer = nullptr;
		}

		// Reset the source's channel maps
		FRWScopeLock Lock(ChannelMapLock, SLT_Write);
		ChannelMap.Reset();

		InitializationState = EMixerSourceInitializationState::NotInitialized;
	}

	void FMixerSource::UpdatePitch()
	{
		AUDIO_MIXER_CHECK(MixerBuffer);

		check(WaveInstance);

		FActiveSound* ActiveSound = WaveInstance->ActiveSound;
		check(ActiveSound);

		Pitch = WaveInstance->GetPitch();

		// Don't apply global pitch scale to UI sounds
		if (!WaveInstance->bIsUISound)
		{
			Pitch *= AudioDevice->GetGlobalPitchScale().GetValue();
		}

		Pitch = AudioDevice->ClampPitch(Pitch);

		// Scale the pitch by the ratio of the audio buffer sample rate and the actual sample rate of the hardware
		if (MixerBuffer)
		{
			const float MixerBufferSampleRate = MixerBuffer->GetSampleRate();
			const float AudioDeviceSampleRate = AudioDevice->GetSampleRate();
			Pitch *= MixerBufferSampleRate / AudioDeviceSampleRate;

			MixerSourceVoice->SetPitch(Pitch);
		}

		USoundWave* WaveData = WaveInstance->WaveData;
		check(WaveData);
		const float ModPitchBase = ModulationUtils::GetRoutedPitch(*WaveInstance, *WaveData, *ActiveSound);
		MixerSourceVoice->SetModPitch(ModPitchBase);

		WaveInstance->PlaybackPercent = GetPlaybackPercent();
	}

	float FMixerSource::GetInheritedSubmixVolumeModulation() const
	{
		if (!MixerDevice)
		{
			return 1.0f;
		}

		FAudioDevice::FAudioSpatializationInterfaceInfo SpatializationInfo = MixerDevice->GetCurrentSpatializationPluginInterfaceInfo();
		// We only hit this condition if, while the sound is playing, the spatializer changes from an external send to a non-external one.
		// If that happens, the submix will catch all modulation so this function's logic is not needed.
		if (!SpatializationInfo.bSpatializationIsExternalSend)
		{
			return 1.0f;
		}

		// if there is a return submix, we need to figure out where to stop manually attenuating
		// Because the submix will modulate itself later
		// Since the graph has tree-like structure, we can create a list of the return submix's ancestors
		// to use while traversing the other submix's ancestors
		TArray<uint32> ReturnSubmixAncestors;
		if (SpatializationInfo.bReturnsToSubmixGraph)
		{
			if (MixerDevice && MixerDevice->ReverbPluginInterface)
			{
				USoundSubmix* ReturnSubmix = MixerDevice->ReverbPluginInterface->GetSubmix();
				if (ReturnSubmix)
				{
					FMixerSubmixWeakPtr CurrReturnSubmixWeakPtr = MixerDevice->GetSubmixInstance(ReturnSubmix);
					FMixerSubmixPtr CurrReturnSubmixPtr = CurrReturnSubmixWeakPtr.Pin();
					while (CurrReturnSubmixPtr && CurrReturnSubmixPtr->IsOwnerValid())
					{
						ReturnSubmixAncestors.Add(CurrReturnSubmixPtr->GetId());

						CurrReturnSubmixWeakPtr = CurrReturnSubmixPtr->GetParent();
						CurrReturnSubmixPtr = CurrReturnSubmixWeakPtr.Pin();
					}
				}
			}
		}

		float SubmixModVolume = 1.0f;

		FMixerSubmixWeakPtr CurrSubmixWeakPtr = MixerDevice->GetSubmixInstance(WaveInstance->SoundSubmix);
		FMixerSubmixPtr CurrSubmixPtr = CurrSubmixWeakPtr.Pin();
		// Check the submix and all its parents in the graph for active modulation
		while (CurrSubmixPtr && CurrSubmixPtr->IsOwnerValid())
		{
			// Matching ID means the external spatializer has returned to the submix graph at this point,
			// so we no longer need to manually apply volume modulation
			if (SpatializationInfo.bReturnsToSubmixGraph && ReturnSubmixAncestors.Contains(CurrSubmixPtr->GetId()))
			{
				break;
			}

			SubmixModVolume *= CurrSubmixPtr->GetOutputVolumeModulationValueThreadSafe();
			SubmixModVolume *= CurrSubmixPtr->GetWetVolumeModulationValueThreadSafe();

			CurrSubmixWeakPtr = CurrSubmixPtr->GetParent();
			CurrSubmixPtr = CurrSubmixWeakPtr.Pin();
		}

		return SubmixModVolume;
	}

	void FMixerSource::UpdateVolume()
	{
		// TODO: investigate if occlusion should be split from raw distance attenuation
		MixerSourceVoice->SetDistanceAttenuation(WaveInstance->GetDistanceAndOcclusionAttenuation());

		float CurrentVolume = 0.0f;
		if (!AudioDevice->IsAudioDeviceMuted())
		{
			// 1. Apply device gain stage(s)
			CurrentVolume = WaveInstance->ActiveSound->bIsPreviewSound ? 1.0f : AudioDevice->GetPrimaryVolume();
			CurrentVolume *= AudioDevice->GetPlatformAudioHeadroom();

			// 2. Apply instance gain stage(s)
			CurrentVolume *= WaveInstance->GetVolume();
			CurrentVolume *= WaveInstance->GetDynamicVolume();

			// 3. Submix Volume Modulation (this only happens if the asset is binaural and we're sending to an external submix)
			if (bBypassingSubmixModulation)
			{
				CurrentVolume *= GetInheritedSubmixVolumeModulation();
			}

			// 4. Apply editor gain stage(s)
			CurrentVolume = FMath::Clamp<float>(GetDebugVolume(CurrentVolume), 0.0f, MAX_VOLUME);

			FActiveSound* ActiveSound = WaveInstance->ActiveSound;
			check(ActiveSound);

			USoundWave* WaveData = WaveInstance->WaveData;
			check(WaveData);
			const float ModVolumeBase = ModulationUtils::GetRoutedVolume(*WaveInstance, *WaveData, *ActiveSound);
			MixerSourceVoice->SetModVolume(ModVolumeBase);
		}
		MixerSourceVoice->SetVolume(CurrentVolume);
	}

	void FMixerSource::UpdateSpatialization()
	{
		FQuat LastEmitterWorldRotation = SpatializationParams.EmitterWorldRotation;
		SpatializationParams = GetSpatializationParams();
		SpatializationParams.LastEmitterWorldRotation = LastEmitterWorldRotation;

		if (WaveInstance->GetUseSpatialization() || WaveInstance->bIsAmbisonics)
		{
			MixerSourceVoice->SetSpatializationParams(SpatializationParams);
		}
	}
	
	void FMixerSource::UpdateSubmixSendLevels(const FSoundSubmixSendInfoBase& InSendInfo, const EMixerSourceSubmixSendStage InSendStage, TSet<FMixerSubmixWeakPtr>& OutTouchedSubmixes)
	{
		if (InSendInfo.SoundSubmix != nullptr)
		{
			const FMixerSubmixWeakPtr SubmixInstance = MixerDevice->GetSubmixInstance(InSendInfo.SoundSubmix);
			float SendLevel = 1.0f;

			// Add it to our touched submix list.
			OutTouchedSubmixes.Add(SubmixInstance);

			// calculate send level based on distance if that method is enabled
			if (InSendInfo.SendLevelControlMethod == ESendLevelControlMethod::Manual)
			{
				if (InSendInfo.DisableManualSendClamp)
				{
					SendLevel = InSendInfo.SendLevel;
				}
				else
				{
					SendLevel = FMath::Clamp(InSendInfo.SendLevel, 0.0f, 1.0f);
				}
			}
			else
			{
				// The alpha value is determined identically between manual and custom curve methods
				const FVector2D SendRadialRange = { InSendInfo.MinSendDistance, InSendInfo.MaxSendDistance};
				const FVector2D SendLevelRange = { InSendInfo.MinSendLevel, InSendInfo.MaxSendLevel };
				const float Denom = FMath::Max(SendRadialRange.Y - SendRadialRange.X, 1.0f);
				const float Alpha = FMath::Clamp((WaveInstance->ListenerToSoundDistance - SendRadialRange.X) / Denom, 0.0f, 1.0f);

				if (InSendInfo.SendLevelControlMethod == ESendLevelControlMethod::Linear)
				{
					SendLevel = FMath::Clamp(FMath::Lerp(SendLevelRange.X, SendLevelRange.Y, Alpha), 0.0f, 1.0f);
				}
				else // use curve
				{
					SendLevel = FMath::Clamp(InSendInfo.CustomSendLevelCurve.GetRichCurveConst()->Eval(Alpha), 0.0f, 1.0f);
				}
			}

			// set the level and stage for this send
			MixerSourceVoice->SetSubmixSendInfo(SubmixInstance, SendLevel, InSendStage,
				FSendFilterSettings(InSendInfo.bEnableLPFCutoff != 0, InSendInfo.bEnableHPFCutoff != 0,
					InSendInfo.LPFCutoff, InSendInfo.HPFCutoff));
		}
	}

	void FMixerSource::UpdateEffects()
	{
		// Update the default LPF filter frequency
		SetFilterFrequency();

		MixerSourceVoice->SetLPFFrequency(LPFFrequency);
		MixerSourceVoice->SetHPFFrequency(HPFFrequency);

		check(WaveInstance);
		FActiveSound* ActiveSound = WaveInstance->ActiveSound;
		check(ActiveSound);

		USoundWave* WaveData = WaveInstance->WaveData;
		check(WaveData);

		float ModHighpassBase = ModulationUtils::GetRoutedHighpass(*WaveInstance, *WaveData, *ActiveSound);
		MixerSourceVoice->SetModHPFFrequency(ModHighpassBase);

		float ModLowpassBase = ModulationUtils::GetRoutedLowpass(*WaveInstance, *WaveData, *ActiveSound);
		MixerSourceVoice->SetModLPFFrequency(ModLowpassBase);

		// If reverb is applied, figure out how of the source to "send" to the reverb.
		if (WaveInstance->bReverb)
		{
			// Send the source audio to the reverb plugin if enabled
			if (UseReverbPlugin() && AudioDevice->ReverbPluginInterface)
			{
				check(MixerDevice);
				FMixerSubmixPtr ReverbPluginSubmixPtr = MixerDevice->GetSubmixInstance(AudioDevice->ReverbPluginInterface->GetSubmix()).Pin();
				if (ReverbPluginSubmixPtr.IsValid())
				{
					MixerSourceVoice->SetSubmixSendInfo(ReverbPluginSubmixPtr, WaveInstance->ReverbSendLevel);
				}
			}

			// Send the source audio to the master reverb
			MixerSourceVoice->SetSubmixSendInfo(MixerDevice->GetMasterReverbSubmix(), WaveInstance->ReverbSendLevel);
		}

		// Safely track if the submix has changed between updates.
		bool bSubmixHasChanged = false;
		TObjectKey<USoundSubmixBase> SubmixKey(WaveInstance->SoundSubmix);
		if (SubmixKey != PrevousSubmix )
		{
			bSubmixHasChanged = true;
		}

		// This will reattempt to resolve a submix each update if there's a valid input
		if ((!WaveInstance->SoundSubmix && PreviousSubmixResolved.IsValid()) || 
		     (WaveInstance->SoundSubmix && !PreviousSubmixResolved.IsValid()) )
		{
			bSubmixHasChanged = true;
		}

		//Check whether the base submix send has been enabled or disabled since the last update
		//Or if the submix has now been registered with the world.
		if (WaveInstance->bEnableBaseSubmix != bPreviousBaseSubmixEnablement || bSubmixHasChanged)
		{
			// set the level for this send
			FMixerSubmixWeakPtr SubmixPtr;
			if (WaveInstance->SoundSubmix)
			{
				SubmixPtr = MixerDevice->GetSubmixInstance(WaveInstance->SoundSubmix);
			}
			else if (!WaveInstance->bIsDynamic) // Dynamic submixes don't auto connect.
			{
				SubmixPtr = MixerDevice->GetBaseDefaultSubmix(); // This will try base default and fall back to master if that fails.
			}

			MixerSourceVoice->SetSubmixSendInfo(SubmixPtr, WaveInstance->bEnableBaseSubmix);
			bPreviousBaseSubmixEnablement = WaveInstance->bEnableBaseSubmix;
			PreviousSubmixResolved = SubmixPtr;
			PrevousSubmix = SubmixKey;
		}

		// We clear sends that aren't used between updates. So tally up the ones that are used.
		// Including the submix itself. 
		// It's okay to use "previous" submix here as it's set above or from a previous setting.
		TSet<FMixerSubmixWeakPtr> TouchedSubmixes;
		TouchedSubmixes.Reserve(1 + WaveInstance->AttenuationSubmixSends.Num() + WaveInstance->SoundSubmixSends.Num());
		TouchedSubmixes.Add(PreviousSubmixResolved);

		// Attenuation Submix Sends. (these come from Attenuation assets).
		// Gated independently by FSoundAttenuationSettings::bEnableSubmixSends upstream in
		// FActiveSound::ParseAttenuation, so we do NOT gate them on the SoundBase's flag here.
		for (const FAttenuationSubmixSendSettings& SendSettings : WaveInstance->AttenuationSubmixSends)
		{
			UpdateSubmixSendLevels(SendSettings, EMixerSourceSubmixSendStage::PostDistanceAttenuation, TouchedSubmixes);
		}

		// Sound submix Sends. (these come from SoundBase derived assets) — gated by SoundBase->bEnableSubmixSends.
		// When the gate is closed the loop is skipped; the cleanup loop below will call ClearSubmixSendInfo
		// for any submix that was previously sent to but is no longer in TouchedSubmixes.
		if (WaveInstance->bEnableSubmixSends)
		{
			for (FSoundSubmixSendInfo& SendInfo : WaveInstance->SoundSubmixSends)
			{
				UpdateSubmixSendLevels(SendInfo, MixerSourcePrivate::SubmixSendStageToMixerSourceSubmixSendStage(SendInfo.SendStage), TouchedSubmixes);
			}
		}
		
		// Anything we haven't touched this update we should now clear.
		const TSet<FMixerSubmixWeakPtr> ToClear = PreviousSubmixSends.Difference(TouchedSubmixes);
		PreviousSubmixSends = TouchedSubmixes;

		// Clear sends that aren't touched.	
		for (FMixerSubmixWeakPtr i : ToClear)
		{
			MixerSourceVoice->ClearSubmixSendInfo(i);
		}
		MixerSourceVoice->SetEnablement(WaveInstance->bEnableBusSends, WaveInstance->bEnableBaseSubmix, WaveInstance->bEnableSubmixSends);
#if WITH_EDITOR
		// The following can spam to the command queue. But is mostly here so that the editor live edits are immedately heard
		// For anything less than editor this is perf waste, so predicate this only to be run in editor. 
		MixerSourceVoice->SetSourceBufferListener(WaveInstance->SourceBufferListener, WaveInstance->bShouldSourceBufferListenerZeroBuffer);
#endif	// WITH_EDITOR
	}

	void FMixerSource::UpdateModulation()
	{
		check(WaveInstance);

		FActiveSound* ActiveSound = WaveInstance->ActiveSound;
		check(ActiveSound);

		if (ActiveSound->bModulationRoutingUpdated)
		{
			if (WaveInstance->WaveData)
			{
				FSoundModulationDefaultRoutingSettings UpdatedRouting = ModulationUtils::UpdateRoutedModulation(*WaveInstance, *(WaveInstance->WaveData), ActiveSound);
				MixerSourceVoice->SetModulationRouting(UpdatedRouting);
			}
			else
			{
				MixerSourceVoice->SetModulationRouting(ActiveSound->ModulationRouting);
			}
		}

		ActiveSound->bModulationRoutingUpdated = false;

		// Query a modulation value for the active sound to use during concurrency evaluation
		const float SourceModVolume = MixerSourceVoice->GetVolumeModulationValue();
		ActiveSound->MaxSourceModulationValue = FMath::Max(SourceModVolume, ActiveSound->MaxSourceModulationValue);
	}

	void FMixerSource::UpdateSourceBusSends()
	{
		// 1) loop through all bus sends
		// 2) check for any bus sends that are set to update non-manually
		// 3) Cache previous send level and only do update if it's changed in any significant amount

		SetupBusData();

		FActiveSound* ActiveSound = WaveInstance->ActiveSound;
		check(ActiveSound);

		// Check if the user actively called a function that alters bus sends since the last update
		bool bHasNewBusSends = ActiveSound->HasNewBusSends();

		if (!bSendingAudioToBuses && !bHasNewBusSends && !DynamicBusSendInfos.Num())
		{
			return;
		}

		if (bHasNewBusSends)
		{
			TArray<TTuple<EBusSendType, FSoundSourceBusSendInfo>> NewBusSends = ActiveSound->GetNewBusSends();
			for (TTuple<EBusSendType, FSoundSourceBusSendInfo>& NewSend : NewBusSends)
			{
				if (NewSend.Value.SoundSourceBus)
				{
					FString BusName;

#if ENABLE_AUDIO_DEBUG
					BusName = NewSend.Value.SoundSourceBus->GetPathName();
#endif // if ENABLE_AUDIO_DEBUG

					// Match SetupBusData: prefer the underlying AudioBus ID when present (explicit bus),
					// otherwise fall back to the SoundSourceBus own ID (automatic source bus).
					const uint32 BusId = NewSend.Value.SoundSourceBus->AudioBus
						? NewSend.Value.SoundSourceBus->AudioBus->GetUniqueID()
						: NewSend.Value.SoundSourceBus->GetUniqueID();
					MixerSourceVoice->SetAudioBusSendInfo(NewSend.Key, BusId, NewSend.Value.SendLevel, BusName,
						FSendFilterSettings(NewSend.Value.bEnableLPFCutoff != 0, NewSend.Value.bEnableHPFCutoff != 0,
							NewSend.Value.LPFCutoff, NewSend.Value.HPFCutoff));
					bSendingAudioToBuses = true;
					// Sync DynamicBusSendInfos so the per-tick dirty-check does not double-dispatch this send.
					for (FDynamicBusSendInfo& DynamicInfo : DynamicBusSendInfos)
					{
						if (DynamicInfo.BusId == BusId)
						{
							DynamicInfo.SendLevel = NewSend.Value.SendLevel;
							DynamicInfo.Filter = FSendFilterSettings(NewSend.Value.bEnableLPFCutoff != 0, NewSend.Value.bEnableHPFCutoff != 0,
								NewSend.Value.LPFCutoff, NewSend.Value.HPFCutoff);
							DynamicInfo.bFilterStateChanged = false;
						}
					}
				}

				if (NewSend.Value.AudioBus)
				{
					FString BusName;

#if ENABLE_AUDIO_DEBUG
					BusName = NewSend.Value.AudioBus->GetPathName();
#endif // if ENABLE_AUDIO_DEBUG

					const uint32 AudioBusId = NewSend.Value.AudioBus->GetUniqueID();
					MixerSourceVoice->SetAudioBusSendInfo(NewSend.Key, AudioBusId, NewSend.Value.SendLevel, BusName,
						FSendFilterSettings(NewSend.Value.bEnableLPFCutoff != 0, NewSend.Value.bEnableHPFCutoff != 0,
							NewSend.Value.LPFCutoff, NewSend.Value.HPFCutoff));
					bSendingAudioToBuses = true;
					// Sync DynamicBusSendInfos so the per-tick dirty-check does not double-dispatch this send.
					for (FDynamicBusSendInfo& DynamicInfo : DynamicBusSendInfos)
					{
						if (DynamicInfo.BusId == AudioBusId)
						{
							DynamicInfo.SendLevel = NewSend.Value.SendLevel;
							DynamicInfo.Filter = FSendFilterSettings(NewSend.Value.bEnableLPFCutoff != 0, NewSend.Value.bEnableHPFCutoff != 0,
								NewSend.Value.LPFCutoff, NewSend.Value.HPFCutoff);
							DynamicInfo.bFilterStateChanged = false;
						}
					}
				}
			}

			ActiveSound->ResetNewBusSends();
		}

		// If this source is sending its audio to a bus, we need to check if it needs to be updated
		for (FDynamicBusSendInfo& DynamicBusSendInfo : DynamicBusSendInfos)
		{
			float SendLevel = 0.0f;

			if (DynamicBusSendInfo.BusSendLevelControlMethod == ESourceBusSendLevelControlMethod::Manual)
			{
				SendLevel = FMath::Clamp(DynamicBusSendInfo.SendLevel, 0.0f, 1.0f);
			}
			else
			{
				// The alpha value is determined identically between linear and custom curve methods
				const FVector2D SendRadialRange = { DynamicBusSendInfo.MinSendDistance, DynamicBusSendInfo.MaxSendDistance };
				const FVector2D SendLevelRange = { DynamicBusSendInfo.MinSendLevel, DynamicBusSendInfo.MaxSendLevel };
				const float Denom = FMath::Max(SendRadialRange.Y - SendRadialRange.X, 1.0f);
				const float Alpha = FMath::Clamp((WaveInstance->ListenerToSoundDistance - SendRadialRange.X) / Denom, 0.0f, 1.0f);

				if (DynamicBusSendInfo.BusSendLevelControlMethod == ESourceBusSendLevelControlMethod::Linear)
				{
					SendLevel = FMath::Clamp(FMath::Lerp(SendLevelRange.X, SendLevelRange.Y, Alpha), 0.0f, 1.0f);
				}
				else // use curve
				{
					SendLevel = FMath::Clamp(DynamicBusSendInfo.CustomSendLevelCurve.GetRichCurveConst()->Eval(Alpha), 0.0f, 1.0f);
				}
			}

			// If the send level changed, then we need to send an update to the audio render thread
			const bool bSendLevelChanged = !FMath::IsNearlyEqual(SendLevel, DynamicBusSendInfo.SendLevel);
			const bool bBusEnablementChanged = bPreviousBusEnablement != WaveInstance->bEnableBusSends;
			const bool bFilterStateChanged = DynamicBusSendInfo.bFilterStateChanged;

			if (bSendLevelChanged || bBusEnablementChanged || bFilterStateChanged)
			{
				DynamicBusSendInfo.SendLevel = SendLevel;
				DynamicBusSendInfo.bIsInit = false;
				DynamicBusSendInfo.bFilterStateChanged = false;

				FString BusName;

#if ENABLE_AUDIO_DEBUG
				BusName = DynamicBusSendInfo.BusName;
#endif // if ENABLE_AUDIO_DEBUG

				MixerSourceVoice->SetAudioBusSendInfo(DynamicBusSendInfo.BusSendType, DynamicBusSendInfo.BusId, SendLevel, BusName,
					DynamicBusSendInfo.Filter);

				bPreviousBusEnablement = WaveInstance->bEnableBusSends;
			}

		}
	}

	void FMixerSource::UpdateChannelMaps()
	{
		SetLFEBleed();

		int32 NumOutputDeviceChannels = MixerDevice->GetNumDeviceChannels();
		const FAudioPlatformDeviceInfo& DeviceInfo = MixerDevice->GetPlatformDeviceInfo();

		// Compute a new speaker map for each possible output channel mapping for the source
		bool bShouldSetMap = false;
		{
			FRWScopeLock Lock(ChannelMapLock, SLT_Write);
			bShouldSetMap = ComputeChannelMap(GetNumChannels(), ChannelMap);
		}
		if(bShouldSetMap)
		{			
			FRWScopeLock Lock(ChannelMapLock, SLT_ReadOnly);
			MixerSourceVoice->SetChannelMap(NumChannels, ChannelMap, bIs3D, WaveInstance->bCenterChannelOnly);
		}

		bPrevAllowedSpatializationSetting = IsSpatializationCVarEnabled();
	}

	void FMixerSource::UpdateRelativeRenderCost()
	{
		if (MixerSourceVoice)
		{
			const float RelativeRenderCost = MixerSourceVoice->GetRelativeRenderCost();
			check(WaveInstance);
			WaveInstance->SetRelativeRenderCost(RelativeRenderCost);
#if ENABLE_AUDIO_DEBUG
			if (DebugInfo.IsValid())
			{
				FScopeLock DebugInfoLock(&DebugInfo->CS);
				DebugInfo->RelativeRenderCost = RelativeRenderCost;

				// au.RelativeRenderCostLogThresholdCVar logging
				static constexpr float RelativeRenderCostLogTolerance = 0.1f;
				if (RelativeRenderCostLogThresholdCVar >= 0 &&
					RelativeRenderCost > RelativeRenderCostLogThresholdCVar &&
					RelativeRenderCost - DebugInfo->MaxLoggedRelativeRenderCost > RelativeRenderCostLogTolerance)
				{
					UE_LOGF(LogAudioMixer, Verbose, "New peak Relative Render Cost: %f (exceeds log threshold %f) for wave %ls.",
						RelativeRenderCost,
						RelativeRenderCostLogThresholdCVar,
						*WaveInstance->GetName());
					DebugInfo->MaxLoggedRelativeRenderCost = RelativeRenderCost;
				}
			}
#endif // if ENABLE_AUDIO_DEBUG

#if UE_AUDIO_PROFILERTRACE_ENABLED
			const bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AudioMixerChannel);
			if (bChannelEnabled && WaveInstance->ActiveSound != nullptr)
			{
				UE_TRACE_LOG(Audio, SoundRelativeRenderCost, AudioMixerChannel)
					<< SoundRelativeRenderCost.DeviceId(MixerDevice->DeviceID)
					<< SoundRelativeRenderCost.Timestamp(FPlatformTime::Cycles64())
					<< SoundRelativeRenderCost.PlayOrder(WaveInstance->GetPlayOrder())
					<< SoundRelativeRenderCost.ActiveSoundPlayOrder(WaveInstance->ActiveSound->GetPlayOrder())
					<< SoundRelativeRenderCost.RelativeRenderCost(RelativeRenderCost);
			}
#endif // UE_AUDIO_PROFILERTRACE_ENABLED
		}
	}

#if ENABLE_AUDIO_DEBUG
	void FMixerSource::UpdateCPUCoreUtilization()
	{
		if (MixerSourceVoice)
		{
			if (DebugInfo.IsValid())
			{
				FScopeLock DebugInfoLock(&DebugInfo->CS);
				DebugInfo->CPUCoreUtilization = MixerSourceVoice->GetCPUCoreUtilization();
			}
		}
	}
#endif // if ENABLE_AUDIO_DEBUG

#if UE_AUDIO_PROFILERTRACE_ENABLED
	void FMixerSource::SendSoundWaveStartTraceMessage() const
	{
		using namespace MixerSourcePrivate;

		const bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AudioChannel);
		if (bChannelEnabled && WaveInstance && WaveInstance->WaveData && WaveInstance->ActiveSound && WaveInstance->SoundClass)
		{
			const TObjectPtr<UObject> SoundWaveObjectPtr = FSoftObjectPath(WaveInstance->WaveData->GetPathName()).ResolveObject();
			if (SoundWaveObjectPtr)
			{
				const FString SoundWaveBaseAssetName = Audio::Trace::Util::GetSoundBaseAssetName(SoundWaveObjectPtr->GetClass());
				const FString OwnerActorLabel = Audio::Trace::Util::GetOwnerActorLabel(*WaveInstance->ActiveSound);
				const int32 SourceBusID = GetSourceBusID(WaveInstance->WaveData);

				UE_TRACE_LOG(Audio, SoundWaveStart, AudioChannel)
					<< SoundWaveStart.DeviceId(AudioDevice->DeviceID)
					<< SoundWaveStart.Timestamp(FPlatformTime::Cycles64())
					<< SoundWaveStart.PlayOrder(WaveInstance->GetPlayOrder())
					<< SoundWaveStart.ActiveSoundPlayOrder(WaveInstance->ActiveSound->GetPlayOrder())
					<< SoundWaveStart.AssetPath(*SoundWaveObjectPtr->GetPathName())
					<< SoundWaveStart.SoundClassName(*SoundWaveBaseAssetName)
					<< SoundWaveStart.ActorLabel(*OwnerActorLabel)
					<< SoundWaveStart.ActorIconName(*Audio::Trace::Util::GetOwnerActorIconName(*WaveInstance->ActiveSound).ToString())
					<< SoundWaveStart.SourceBusID(SourceBusID)
					<< SoundWaveStart.AudioComponentName(*Audio::Trace::Util::GetAudioComponentName(*WaveInstance->ActiveSound));
			}
		}
	}

	void FMixerSource::SendSoundWaveStopTraceMessage() const
	{
		const bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AudioChannel);
		if (bChannelEnabled && MixerDevice && WaveInstance && WaveInstance->ActiveSound)
		{
			UE_TRACE_LOG(Audio, SoundWaveStop, AudioChannel)
				<< SoundWaveStop.DeviceId(MixerDevice->DeviceID)
				<< SoundWaveStop.Timestamp(FPlatformTime::Cycles64())
				<< SoundWaveStop.PlayOrder(WaveInstance->GetPlayOrder())
				<< SoundWaveStop.ActiveSoundPlayOrder(WaveInstance->ActiveSound->GetPlayOrder());
		}
	}

	void FMixerSource::UpdateIsAlivePingTraceMessage() const
	{
		using namespace MixerSourcePrivate;

		const bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AudioChannel);

		if (bChannelEnabled && WaveInstance && WaveInstance->IsPlaying() && WaveInstance->WaveData && WaveInstance->ActiveSound && WaveInstance->SoundClass)
		{
			// Occasionally send a ping to Audio Insights to confirm the sound is still playing.
			// This is to ensure long duration sounds (such as ambiences) will still appear in snapshot files if their start action was before the snapshot start
			if (FMath::Floor(WaveInstance->ActiveSound->PlaybackTimeUnscaled + AudioDevice->GetDeviceDeltaTime()) - FMath::Floor(WaveInstance->ActiveSound->PlaybackTimeUnscaled) > 0.0f)
			{
				const TObjectPtr<UObject> SoundWaveObjectPtr = FSoftObjectPath(WaveInstance->WaveData->GetPathName()).ResolveObject();
				if (SoundWaveObjectPtr)
				{
					const FString SoundWaveBaseAssetName = Audio::Trace::Util::GetSoundBaseAssetName(SoundWaveObjectPtr->GetClass());
					const FString OwnerActorLabel = Audio::Trace::Util::GetOwnerActorLabel(*WaveInstance->ActiveSound);
					const int32 SourceBusID = GetSourceBusID(WaveInstance->WaveData);

					UE_TRACE_LOG(Audio, SoundWaveIsAlivePing, AudioChannel)
						<< SoundWaveIsAlivePing.DeviceId(AudioDevice->DeviceID)
						<< SoundWaveIsAlivePing.Timestamp(FPlatformTime::Cycles64())
						<< SoundWaveIsAlivePing.PlayOrder(WaveInstance->GetPlayOrder())
						<< SoundWaveIsAlivePing.ActiveSoundPlayOrder(WaveInstance->ActiveSound->GetPlayOrder())
						<< SoundWaveIsAlivePing.AssetPath(*SoundWaveObjectPtr->GetPathName())
						<< SoundWaveIsAlivePing.SoundClassName(*SoundWaveBaseAssetName)
						<< SoundWaveIsAlivePing.ActorLabel(*OwnerActorLabel)
						<< SoundWaveIsAlivePing.ActorIconName(*Audio::Trace::Util::GetOwnerActorIconName(*WaveInstance->ActiveSound).ToString())
						<< SoundWaveIsAlivePing.SourceBusID(SourceBusID)
						<< SoundWaveIsAlivePing.AudioComponentName(*Audio::Trace::Util::GetAudioComponentName(*WaveInstance->ActiveSound));
				}
			}
		}
	}
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

	bool FMixerSource::ComputeMonoChannelMap(Audio::FAlignedFloatBuffer& OutChannelMap)
	{
		if (IsUsingObjectBasedSpatialization())
		{
			if (WaveInstance->SpatializationMethod != ESoundSpatializationAlgorithm::SPATIALIZATION_HRTF && !bEditorWarnedChangedSpatialization)
			{
				bEditorWarnedChangedSpatialization = true;
				UE_LOGF(LogAudioMixer, Warning, "Changing the spatialization method on a playing sound is not supported (WaveInstance: %ls)", *WaveInstance->WaveData->GetFullName());
			}

			// Treat the source as if it is a 2D stereo source:
			return ComputeStereoChannelMap(OutChannelMap);
		}
		else if (WaveInstance->GetUseSpatialization() && (!FMath::IsNearlyEqual(WaveInstance->AbsoluteAzimuth, PreviousAzimuth, 0.01f) || MixerSourceVoice->NeedsSpeakerMap()))
		{
			// Don't need to compute the source channel map if the absolute azimuth hasn't changed much
			PreviousAzimuth = WaveInstance->AbsoluteAzimuth;
			OutChannelMap.Reset();
			int32 NumOutputChannels = MixerDevice->GetNumDeviceChannels();

			if (WaveInstance->NonSpatializedRadiusMode == ENonSpatializedRadiusSpeakerMapMode::OmniDirectional)
			{
				float DefaultOmniAmount = 1.0f / NumOutputChannels;
				MixerDevice->Get3DChannelMap(NumOutputChannels, WaveInstance, WaveInstance->AbsoluteAzimuth, SpatializationParams.NonSpatializedAmount, nullptr, DefaultOmniAmount, OutChannelMap);
			}
			else if (WaveInstance->NonSpatializedRadiusMode == ENonSpatializedRadiusSpeakerMapMode::Direct2D)
			{
				// Create some omni maps for left and right channels, note we're 
				// taking into account mono upmix method
				auto CreateOmniMap = [this]() -> TMap<EAudioMixerChannel::Type, float>
				{
					EMonoChannelUpmixMethod MonoUpmixMethod = MixerDevice->GetMonoChannelUpmixMethod();
					TMap<EAudioMixerChannel::Type, float> OmniMap;

					if (MonoUpmixMethod == EMonoChannelUpmixMethod::FullVolume)
					{
						OmniMap.Add(EAudioMixerChannel::FrontLeft, Audio::MonoUpmixFullVolume);
						OmniMap.Add(EAudioMixerChannel::FrontRight, Audio::MonoUpmixFullVolume);
					}
					else if (MonoUpmixMethod == EMonoChannelUpmixMethod::EqualPower)
					{
						OmniMap.Add(EAudioMixerChannel::FrontLeft, Audio::MonoUpmixEqualPower);
						OmniMap.Add(EAudioMixerChannel::FrontRight, Audio::MonoUpmixEqualPower);
					}
					else
					{
						check(MonoUpmixMethod == EMonoChannelUpmixMethod::Linear);
						OmniMap.Add(EAudioMixerChannel::FrontLeft, Audio::MonoUpmixLinear);
						OmniMap.Add(EAudioMixerChannel::FrontRight, Audio::MonoUpmixLinear);
					}
					return OmniMap;
				};

				static const TMap<EAudioMixerChannel::Type, float> OmniMap = CreateOmniMap();
				MixerDevice->Get3DChannelMap(NumOutputChannels, WaveInstance, WaveInstance->AbsoluteAzimuth, SpatializationParams.NonSpatializedAmount, &OmniMap, 0.0f, OutChannelMap);
			}
			else if (WaveInstance->NonSpatializedRadiusMode == ENonSpatializedRadiusSpeakerMapMode::Surround2D)
			{
				// Create some omni maps for left and right channels, note we're 
				// taking into account mono upmix method
				auto CreateOmniMap = [this, &NumOutputChannels]() -> TMap<EAudioMixerChannel::Type, float>
					{
						EMonoChannelUpmixMethod MonoUpmixMethod = MixerDevice->GetMonoChannelUpmixMethod();

						TMap<EAudioMixerChannel::Type, float> OmniMap;

						if (MonoUpmixMethod == EMonoChannelUpmixMethod::FullVolume)
						{
							OmniMap.Add(EAudioMixerChannel::FrontLeft, Audio::MonoUpmixFullVolume);
							OmniMap.Add(EAudioMixerChannel::FrontRight, Audio::MonoUpmixFullVolume);
							if (NumOutputChannels == 8)
							{
								OmniMap.Add(EAudioMixerChannel::BackLeft, Audio::MonoUpmixFullVolume);
								OmniMap.Add(EAudioMixerChannel::BackRight, Audio::MonoUpmixFullVolume);
							}
							else if (NumOutputChannels == 6)
							{
								OmniMap.Add(EAudioMixerChannel::SideLeft, Audio::MonoUpmixFullVolume);
								OmniMap.Add(EAudioMixerChannel::SideRight, Audio::MonoUpmixFullVolume);
							}
						}
						else if (MonoUpmixMethod == EMonoChannelUpmixMethod::EqualPower)
						{
							OmniMap.Add(EAudioMixerChannel::FrontLeft, Audio::MonoUpmixEqualPower);
							OmniMap.Add(EAudioMixerChannel::FrontRight, Audio::MonoUpmixEqualPower);
							if (NumOutputChannels == 8)
							{
								OmniMap.Add(EAudioMixerChannel::BackLeft, Audio::MonoUpmixEqualPower);
								OmniMap.Add(EAudioMixerChannel::BackRight, Audio::MonoUpmixEqualPower);
							}
							else
							{
								OmniMap.Add(EAudioMixerChannel::SideLeft, Audio::MonoUpmixEqualPower);
								OmniMap.Add(EAudioMixerChannel::SideRight, Audio::MonoUpmixEqualPower);
							}
						}
						else
						{
							check(MonoUpmixMethod == EMonoChannelUpmixMethod::Linear);
							OmniMap.Add(EAudioMixerChannel::FrontLeft, Audio::MonoUpmixLinear);
							OmniMap.Add(EAudioMixerChannel::FrontRight, Audio::MonoUpmixLinear);
							if (NumOutputChannels == 8)
							{
								OmniMap.Add(EAudioMixerChannel::BackLeft, Audio::MonoUpmixLinear);
								OmniMap.Add(EAudioMixerChannel::BackRight, Audio::MonoUpmixLinear);
							}
							else if (NumOutputChannels == 6)
							{
								OmniMap.Add(EAudioMixerChannel::SideLeft, Audio::MonoUpmixLinear);
								OmniMap.Add(EAudioMixerChannel::SideRight, Audio::MonoUpmixLinear);
							}
						}
						return OmniMap;
					};

				static const TMap<EAudioMixerChannel::Type, float> OmniMap = CreateOmniMap();
				MixerDevice->Get3DChannelMap(NumOutputChannels, WaveInstance, WaveInstance->AbsoluteAzimuth, SpatializationParams.NonSpatializedAmount, &OmniMap, 0.0f, OutChannelMap);
			}
			return true;
		}
		else if (!OutChannelMap.Num() || (IsSpatializationCVarEnabled() != bPrevAllowedSpatializationSetting))
		{
			// Only need to compute the 2D channel map once
			MixerDevice->Get2DChannelMap(bIsVorbis, 1, WaveInstance->bCenterChannelOnly, OutChannelMap);
			return true;
		}

		// Return false means the channel map hasn't changed
		return false;
	}

	bool FMixerSource::ComputeStereoChannelMap(Audio::FAlignedFloatBuffer& OutChannelMap)
	{
		// Only recalculate positional data if the source has moved a significant amount:
		if (WaveInstance->GetUseSpatialization() && (!FMath::IsNearlyEqual(WaveInstance->AbsoluteAzimuth, PreviousAzimuth, 0.01f) || MixerSourceVoice->NeedsSpeakerMap()))
		{
			// Make sure our stereo emitter positions are updated relative to the sound emitter position
			if (GetNumChannels() == 2)
			{
				UpdateStereoEmitterPositions();
			}

			// Check whether voice is currently using 
			if (!IsUsingObjectBasedSpatialization())
			{
				float AzimuthOffset = 0.0f;

				float LeftAzimuth = 90.0f;
				float RightAzimuth = 270.0f;

				const float DistanceToUse = UseListenerOverrideForSpreadCVar ? WaveInstance->ListenerToSoundDistance : WaveInstance->ListenerToSoundDistanceForPanning;

				if (DistanceToUse > KINDA_SMALL_NUMBER)
				{
					AzimuthOffset = FMath::Atan(0.5f * WaveInstance->StereoSpread / DistanceToUse);
					AzimuthOffset = FMath::RadiansToDegrees(AzimuthOffset);

					LeftAzimuth = WaveInstance->AbsoluteAzimuth - AzimuthOffset;
					if (LeftAzimuth < 0.0f)
					{
						LeftAzimuth += 360.0f;
					}

					RightAzimuth = WaveInstance->AbsoluteAzimuth + AzimuthOffset;
					if (RightAzimuth > 360.0f)
					{
						RightAzimuth -= 360.0f;
					}
				}

				// Reset the channel map, the stereo spatialization channel mapping calls below will append their mappings
				OutChannelMap.Reset();

				int32 NumOutputChannels = MixerDevice->GetNumDeviceChannels();

				if (WaveInstance->NonSpatializedRadiusMode == ENonSpatializedRadiusSpeakerMapMode::OmniDirectional)
				{
					float DefaultOmniAmount = 1.0f / NumOutputChannels;
					MixerDevice->Get3DChannelMap(NumOutputChannels, WaveInstance, LeftAzimuth, SpatializationParams.NonSpatializedAmount, nullptr, DefaultOmniAmount, OutChannelMap);
					MixerDevice->Get3DChannelMap(NumOutputChannels, WaveInstance, RightAzimuth, SpatializationParams.NonSpatializedAmount, nullptr, DefaultOmniAmount, OutChannelMap);
				}
				else if (WaveInstance->NonSpatializedRadiusMode == ENonSpatializedRadiusSpeakerMapMode::Direct2D)
				{
					// Create some omni maps for left and right channels
					auto CreateLeftOmniMap = []() -> TMap<EAudioMixerChannel::Type, float>
					{
						TMap<EAudioMixerChannel::Type, float> LeftOmniMap;
						LeftOmniMap.Add(EAudioMixerChannel::FrontLeft, 1.0f);
						return LeftOmniMap;
					};

					auto CreateRightOmniMap = []() -> TMap<EAudioMixerChannel::Type, float>
					{
						TMap<EAudioMixerChannel::Type, float> RightOmniMap;
						RightOmniMap.Add(EAudioMixerChannel::FrontRight, 1.0f);
						return RightOmniMap;
					};

					static const TMap<EAudioMixerChannel::Type, float> LeftOmniMap = CreateLeftOmniMap();
					static const TMap<EAudioMixerChannel::Type, float> RightOmniMap = CreateRightOmniMap();

					MixerDevice->Get3DChannelMap(NumOutputChannels, WaveInstance, LeftAzimuth, SpatializationParams.NonSpatializedAmount, &LeftOmniMap, 0.0f, OutChannelMap);
					MixerDevice->Get3DChannelMap(NumOutputChannels, WaveInstance, RightAzimuth, SpatializationParams.NonSpatializedAmount, &RightOmniMap, 0.0f, OutChannelMap);
				}
				else
				{
					// If we are in 5.1, we need to use the side-channel speakers
					// If we are outputting stereo, omni-blend to a 5.1 output. This will get downmixed to stereo as a fallback.
					if (NumOutputChannels == 2 || NumOutputChannels == 6)
					{
						// Create some omni maps for left and right channels
						auto CreateLeftOmniMap = []() -> TMap<EAudioMixerChannel::Type, float>
						{
							TMap<EAudioMixerChannel::Type, float> LeftOmniMap;
							LeftOmniMap.Add(EAudioMixerChannel::FrontLeft, 1.0f);
							LeftOmniMap.Add(EAudioMixerChannel::SideLeft, 1.0f);

							return LeftOmniMap;
						};

						auto CreateRightOmniMap = []() -> TMap<EAudioMixerChannel::Type, float>
						{
							TMap<EAudioMixerChannel::Type, float> RightOmniMap;
							RightOmniMap.Add(EAudioMixerChannel::FrontRight, 1.0f);
							RightOmniMap.Add(EAudioMixerChannel::SideRight, 1.0f);

							return RightOmniMap;
						};

						static const TMap<EAudioMixerChannel::Type, float> LeftOmniMap = CreateLeftOmniMap();
						static const TMap<EAudioMixerChannel::Type, float> RightOmniMap = CreateRightOmniMap();

						MixerDevice->Get3DChannelMap(NumOutputChannels, WaveInstance, LeftAzimuth, SpatializationParams.NonSpatializedAmount, &LeftOmniMap, 0.0f, OutChannelMap);
						MixerDevice->Get3DChannelMap(NumOutputChannels, WaveInstance, RightAzimuth, SpatializationParams.NonSpatializedAmount, &RightOmniMap, 0.0f, OutChannelMap);

					}
					// If we are in 7.1 we need to use the back-channel speakers
					else if (NumOutputChannels == 8)
					{
						// Create some omni maps for left and right channels
						auto CreateLeftOmniMap = []() -> TMap<EAudioMixerChannel::Type, float>
						{
							TMap<EAudioMixerChannel::Type, float> LeftOmniMap;
							LeftOmniMap.Add(EAudioMixerChannel::FrontLeft, 1.0f);
							LeftOmniMap.Add(EAudioMixerChannel::BackLeft, 1.0f);

							return LeftOmniMap;
						};

						auto CreateRightOmniMap = []() -> TMap<EAudioMixerChannel::Type, float>
						{
							TMap<EAudioMixerChannel::Type, float> RightOmniMap;
							RightOmniMap.Add(EAudioMixerChannel::FrontRight, 1.0f);
							RightOmniMap.Add(EAudioMixerChannel::BackRight, 1.0f);

							return RightOmniMap;
						};

						static const TMap<EAudioMixerChannel::Type, float> LeftOmniMap = CreateLeftOmniMap();
						static const TMap<EAudioMixerChannel::Type, float> RightOmniMap = CreateRightOmniMap();

						MixerDevice->Get3DChannelMap(NumOutputChannels, WaveInstance, LeftAzimuth, SpatializationParams.NonSpatializedAmount, &LeftOmniMap, 0.0f, OutChannelMap);
						MixerDevice->Get3DChannelMap(NumOutputChannels, WaveInstance, RightAzimuth, SpatializationParams.NonSpatializedAmount, &RightOmniMap, 0.0f, OutChannelMap);
					}
				}		

				return true;
			}
		}

		if (!OutChannelMap.Num() || (IsSpatializationCVarEnabled() != bPrevAllowedSpatializationSetting))
		{
			MixerDevice->Get2DChannelMap(bIsVorbis, 2, WaveInstance->bCenterChannelOnly, OutChannelMap);
			return true;
		}

		return false;
	}

	bool FMixerSource::ComputeChannelMap(const int32 NumSourceChannels, Audio::FAlignedFloatBuffer& OutChannelMap)
	{
		if (NumSourceChannels == 1)
		{
			return ComputeMonoChannelMap(OutChannelMap);
		}
		else if (NumSourceChannels == 2)
		{
			return ComputeStereoChannelMap(OutChannelMap);
		}
		else if (!OutChannelMap.Num())
		{
			MixerDevice->Get2DChannelMap(bIsVorbis, NumSourceChannels, WaveInstance->bCenterChannelOnly, OutChannelMap);
			return true;
		}
		return false;
	}

	bool FMixerSource::UseObjectBasedSpatialization() const
	{
		return (GetNumChannels() <= MixerDevice->GetCurrentSpatializationPluginInterfaceInfo().MaxChannelsSupportedBySpatializationPlugin &&
				AudioDevice->IsSpatializationPluginEnabled() &&
				WaveInstance->SpatializationMethod == ESoundSpatializationAlgorithm::SPATIALIZATION_HRTF);
	}

	bool FMixerSource::IsUsingObjectBasedSpatialization() const
	{
		bool bIsUsingObjectBaseSpatialization = UseObjectBasedSpatialization();

		if (MixerSourceVoice)
		{
			// If it is currently playing, check whether it actively uses HRTF spatializer.
			// HRTF spatialization cannot be altered on currently playing source. So this handles
			// the case where the source was initialized without HRTF spatialization before HRTF
			// spatialization is enabled. 
			bool bDefaultIfNoSourceId = true;
			bIsUsingObjectBaseSpatialization &= MixerSourceVoice->IsUsingHRTFSpatializer(bDefaultIfNoSourceId);
		}
		return bIsUsingObjectBaseSpatialization;
	}

	bool FMixerSource::UseSpatializationPlugin() const
	{
		return (GetNumChannels() <= MixerDevice->GetCurrentSpatializationPluginInterfaceInfo().MaxChannelsSupportedBySpatializationPlugin) &&  
			AudioDevice->IsSpatializationPluginEnabled() &&
			WaveInstance->SpatializationPluginSettings != nullptr;
	}

	bool FMixerSource::UseOcclusionPlugin() const
	{
		return (GetNumChannels() == 1 || GetNumChannels() == 2) && 
			AudioDevice->IsOcclusionPluginEnabled() &&
			WaveInstance->OcclusionPluginSettings != nullptr;
	}

	bool FMixerSource::UseReverbPlugin() const
	{
		return (GetNumChannels() == 1 || GetNumChannels() == 2) && 
			AudioDevice->IsReverbPluginEnabled() &&
			WaveInstance->ReverbPluginSettings != nullptr;
	}

	bool FMixerSource::UseSourceDataOverridePlugin() const
	{
		return (GetNumChannels() == 1 || GetNumChannels() == 2) && 
			AudioDevice->IsSourceDataOverridePluginEnabled() &&
			WaveInstance->SourceDataOverridePluginSettings != nullptr;
	}
}
