// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AudioMixerBuffer.h"
#include "AudioMixerSourceManager.h"
#include "ProfilingDebugging/TraceAuxiliary.h"

namespace Audio
{
	class FMixerDevice;
	class FMixerSourceVoice;
	class FMixerSource;
	class FMixerBuffer;
	class ISourceListener;

	/** State to track initialization stages. */
	enum class EMixerSourceInitializationState : uint8
	{
		NotInitialized,
		Initializing,
		Initialized
	};

	/** 
	 * FMixerSource
	 * Class which implements a sound source object for the audio mixer module.
	 */
	class FMixerSource :	public FSoundSource, 
							public ISourceListener
	{
	public:
		/** Constructor. */
		FMixerSource(FAudioDevice* InAudioDevice);

		/** Destructor. */
		~FMixerSource();

		//~ Begin FSoundSource Interface
		virtual bool Init(FWaveInstance* InWaveInstance) override;
		virtual void Update() override;
		virtual bool PrepareForInitialization(FWaveInstance* InWaveInstance) override;
		virtual bool IsPreparedToInit() override;
		virtual bool IsInitialized() const override;
		virtual void Play() override;
		virtual void Stop() override;
		virtual void StopNow() override;
		virtual bool IsStopping() override { return bIsStopping; }
		virtual void Pause() override;
		virtual bool IsFinished() override;
		virtual float GetPlaybackPercent() const override;
		virtual int64 GetNumFramesPlayed() const override;
		virtual float GetEnvelopeValue() const override;
		virtual float GetRelativeRenderCost() const override;
		//~ End FSoundSource Interface

		//~ Begin ISourceListener
		virtual void OnBeginGenerate() override;
		virtual void OnDone() override;
		virtual void OnEffectTailsDone() override;
		virtual void OnLoopEnd() override { bLoopCallback = true; };
		//~ End ISourceListener

	private:

		/** Initializes the bus sends. */
		void SetupBusData(TArray<FInitAudioBusSend>* OutAudioBusSends = nullptr, bool bEnableBusSends = true);

		/** Frees any resources for this sound source. */
		void FreeResources();

		/** Updates the pitch parameter set from the game thread. */
		void UpdatePitch();
		
		/** Updates the volume parameter set from the game thread. */
		void UpdateVolume();

		/** Gets updated spatialization information for the voice. */
		void UpdateSpatialization();

		/** Updates and source effect on this voice. */
		void UpdateEffects();

		/** Updates the Modulation Routing settings on this voice. */
		void UpdateModulation();

		/** Updates source bus send levels based on game data. */
		void UpdateSourceBusSends();

		/** Updates the channel map of the sound if its a 3d sound.*/
		void UpdateChannelMaps();

		/** Updates the relative render cost estimate of the playing sound source. */
		void UpdateRelativeRenderCost();

#if ENABLE_AUDIO_DEBUG
		void UpdateCPUCoreUtilization();
#endif // ENABLE_AUDIO_DEBUG

#if UE_AUDIO_PROFILERTRACE_ENABLED
		void SendSoundWaveStartTraceMessage() const;
		void SendSoundWaveStopTraceMessage() const;
		void UpdateIsAlivePingTraceMessage() const;
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

		/** Computes the mono-channel map. */
		bool ComputeMonoChannelMap(Audio::FAlignedFloatBuffer& OutChannelMap);

		/** Computes the stereo-channel map. */
		bool ComputeStereoChannelMap(Audio::FAlignedFloatBuffer& OutChannelMap);

		/** Compute the channel map based on the number of output and source channels. */
		bool ComputeChannelMap(const int32 NumSourceChannels, Audio::FAlignedFloatBuffer& OutChannelMap);

		/** Whether or not we should create the source voice with the HRTF spatializer. */
		bool UseObjectBasedSpatialization() const;
		
		/** Whether or not existing or new sources will use the HRTF spatializer. */
		bool IsUsingObjectBasedSpatialization() const;

		/** Whether or not to use the spatialization plugin. */
		bool UseSpatializationPlugin() const;

		/** Whether or not to use the occlusion plugin. */
		bool UseOcclusionPlugin() const;

		/** Whether or not to use the reverb plugin. */
		bool UseReverbPlugin() const;

		/** Whether or not to use the source data override plugin */
		bool UseSourceDataOverridePlugin() const;

		/** Gets an accumulated volume value based on the Modulation Destination data of the WaveInstance's submix and all of the submix's ancestors */
		float GetInheritedSubmixVolumeModulation() const;

	private:
		void UpdateSubmixSendLevels(const FSoundSubmixSendInfoBase& InSendInfo, EMixerSourceSubmixSendStage InSendStage, TSet<FMixerSubmixWeakPtr>& OutTouchedSubmixes);

		bool Is3D(const FSoundBuffer* SoundBuffer) const;
		
		FMixerDevice* MixerDevice;
		FMixerBuffer* MixerBuffer;
		TSharedPtr<FMixerSourceBuffer, ESPMode::ThreadSafe> MixerSourceBuffer;
		FMixerSourceVoice* MixerSourceVoice;
		IAudioLinkFactory::FAudioLinkSourcePushedSharedPtr AudioLink;
		FMixerSubmixWeakPtr PreviousSubmixResolved;
		TObjectKey<USoundSubmixBase> PrevousSubmix;

		// These modulators are obtained from the submix and used only on binaural assets
		bool bBypassingSubmixModulation;

		uint32 bPreviousBusEnablement;
		uint32 bPreviousBaseSubmixEnablement;

		// This holds data copied from FSoundSourceBusSendInfo when a new sound starts playing
		// so that distance-based level control can be calculated during rendering
		struct FDynamicBusSendInfo
		{
			float SendLevel = 0.0f;
			uint32 BusId = 0;
#if ENABLE_AUDIO_DEBUG
			FString BusName;
#endif // ENABLE_AUDIO_DEBUG
			ESourceBusSendLevelControlMethod BusSendLevelControlMethod = ESourceBusSendLevelControlMethod::Manual;
			EBusSendType BusSendType = EBusSendType::PreEffect;
			float MinSendLevel = 0.0f;
			float MaxSendLevel = 0.0f;
			float MinSendDistance = 0.0f;
			float MaxSendDistance = 0.0f;
			FRuntimeFloatCurve CustomSendLevelCurve;
			bool bIsInit = true;
			FSendFilterSettings Filter;
			// Set when filter params change via the bHasNewBusSends path so the dynamic
			// tick loop re-sends the update even if send level didn't change.
			bool bFilterStateChanged = false;
		};

		// Mapping of channel map types to channel maps. Determined by what submixes this source sends its audio to.
		Audio::FAlignedFloatBuffer ChannelMap;
		FRWLock ChannelMapLock;

		float PreviousAzimuth;
		mutable float PreviousPlaybackPercent;

		FSpatializationParams SpatializationParams;

		EMixerSourceInitializationState InitializationState;

		FThreadSafeBool bPlayedCachedBuffer;
		FThreadSafeBool bPlaying;
		FThreadSafeBool bIsStopping;
		FThreadSafeBool bLoopCallback;
		FThreadSafeBool bIsDone;
		FThreadSafeBool bIsEffectTailsDone;
		FThreadSafeBool bIsPlayingEffectTails;
		FThreadSafeBool bFreeAsyncTask;

		// Array of copied FSoundSourceBusSendInfo data for all the bus sends this
		// source may need to live-update during its lifespan
		TArray<FDynamicBusSendInfo> DynamicBusSendInfos;

		// An array of submixes from previous update. Allows us to clear out submix sends if they are no longer being sent.
		TSet<FMixerSubmixWeakPtr> PreviousSubmixSends;
		
		// Whether or not we're currently releasing our resources. Prevents recycling the source until release is finished.
		FThreadSafeBool bIsReleasing;

		uint32 bEditorWarnedChangedSpatialization : 1;
		uint32 bIs3D : 1;
		uint32 bDebugMode : 1;
		uint32 bIsVorbis : 1;
		uint32 bIsStoppingVoicesEnabled : 1;
		uint32 bSendingAudioToBuses : 1;
		uint32 bPrevAllowedSpatializationSetting : 1;
	};
}

inline const TCHAR* LexToString(const Audio::EMixerSourceInitializationState State)
{
	switch(State)
	{
	case Audio::EMixerSourceInitializationState::Initialized: return TEXT("Initialized");
	case Audio::EMixerSourceInitializationState::NotInitialized: return TEXT("NotInitialized");
	case Audio::EMixerSourceInitializationState::Initializing: return TEXT("Initializing");
	default: break;
	}
	return TEXT("Unknown");
}
