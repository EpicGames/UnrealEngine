// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioMixerSourceBuffer.h"
#include "AudioMixerSourceDecode.h"
#include "ContentStreaming.h"
#include "AudioDecompress.h"
#include "Misc/ScopeTryLock.h"
#include "DSP/FloatArrayMath.h"

namespace Audio
{
	// Data needed for a procedural audio task (rendering on the audio render thread)
	struct FProceduralAudioTaskData
	{
		ISoundGeneratorPtr SoundGenerator;
		float* AudioData = nullptr;
		int32 NumSamples = 0;
	};

	// Results from procedural audio task
	struct FProceduralAudioTaskResults
	{
		int32 NumSamplesWritten = 0;
		bool bIsFinished = false;
		float RelativeRenderCost = 1.f;

#if ENABLE_AUDIO_DEBUG
		double CPUDuration = 0.0;
#endif // if ENABLE_AUDIO_DEBUG
	};

	bool FRawPCMDataBuffer::GetNextBuffer(TArrayView<float> OutSourceBufferPtr, const uint32 NumSampleToGet)
	{
		// TODO: support loop counts
		float* OutBufferPtr = OutSourceBufferPtr.GetData();
		int16* DataPtr = (int16*)Data;

		if (LoopCount == Audio::LOOP_FOREVER)
		{
			bool bIsFinishedOrLooped = false;
			for (uint32 Sample = 0; Sample < NumSampleToGet; ++Sample)
			{
				OutBufferPtr[Sample] = DataPtr[CurrentSample++] / 32768.0f;

				// Loop around if we're looping
				if (CurrentSample >= NumSamples)
				{
					CurrentSample = 0;
					bIsFinishedOrLooped = true;
				}
			}
			return bIsFinishedOrLooped;
		}
		else if (CurrentSample < NumSamples)
		{
			uint32 Sample = 0;
			while (Sample < NumSampleToGet && CurrentSample < NumSamples)
			{
				OutBufferPtr[Sample++] = (float)DataPtr[CurrentSample++] / 32768.0f;
			}

			// Zero out the rest of the buffer
			FMemory::Memzero(&OutBufferPtr[Sample], (NumSampleToGet - Sample) * sizeof(float));
		}
		else
		{
			FMemory::Memzero(OutBufferPtr, NumSampleToGet * sizeof(float));
		}

		// If the current sample is greater or equal to num samples we hit the end of the buffer
		return CurrentSample >= NumSamples;
	}

	TSharedPtr<FMixerSourceBuffer, ESPMode::ThreadSafe> FMixerSourceBuffer::Create(FMixerSourceBufferInitArgs& InArgs, TArray<FAudioParameter>&& InDefaultParams)
	{
		LLM_SCOPE(ELLMTag::AudioMixer);

		check(InArgs.SoundWave && InArgs.Buffer);

		// Fail if the Wave has been flagged to contain an error
		if (InArgs.SoundWave->HasError())
		{
			UE_LOGF(LogAudioMixer, VeryVerbose, "FMixerSourceBuffer::Create failed as '%ls' is flagged as containing errors", *InArgs.SoundWave->GetName());
			return {};
		}

		TSharedPtr<FMixerSourceBuffer, ESPMode::ThreadSafe> NewSourceBuffer = MakeShareable(new FMixerSourceBuffer(InArgs, MoveTemp(InDefaultParams)));

		return NewSourceBuffer;
	}

	FMixerSourceBuffer::FMixerSourceBuffer(FMixerSourceBufferInitArgs& InArgs, TArray<FAudioParameter>&& InDefaultParams)
		: NumBuffersQueued(0)
		, CurrentBuffer(0)
		, SoundWave(InArgs.SoundWave)
		, AsyncRealtimeAudioTask(nullptr)
		, DecompressionState(nullptr)
		, LoopingMode(InArgs.LoopingMode)
		, NumChannels(InArgs.Buffer->NumChannels)
		, BufferType(InArgs.Buffer->GetType())
		, NumPrecacheFrames(InArgs.SoundWave->NumPrecacheFrames)
		, AuioDeviceID(InArgs.AudioDeviceID)
		, InstanceID(InArgs.InstanceID)
		, WaveName(InArgs.SoundWave->GetFName())
#if ENABLE_AUDIO_DEBUG
		, SampleRate(InArgs.SampleRate)
#endif // ENABLE_AUDIO_DEBUG
		, bInitialized(false)
		, bBufferFinished(false)
		, bPlayedCachedBuffer(false)
		, bIsSeeking(InArgs.bIsSeeking)
		, bLoopCallback(false)
		, bProcedural(InArgs.SoundWave->bProcedural)
		, bIsBus(InArgs.SoundWave->bIsSourceBus)
		, bForceSyncDecode(InArgs.bForceSyncDecode)
		, bHasError(false)
	{
		// TODO: remove the need to do this here. 1) remove need for decoders to depend on USoundWave and 2) remove need for procedural sounds to use USoundWaveProcedural
		InArgs.SoundWave->AddPlayingSource(this);

		// Retrieve a sound generator if this is a procedural sound wave
		if (bProcedural)
		{
			FSoundGeneratorInitParams InitParams;
			InitParams.AudioDeviceID = InArgs.AudioDeviceID;
			InitParams.AudioComponentId = InArgs.AudioComponentID;
			InitParams.SampleRate = InArgs.SampleRate;
			InitParams.AudioMixerNumOutputFrames = InArgs.AudioMixerNumOutputFrames;
			InitParams.NumChannels = NumChannels;
			InitParams.NumFramesPerCallback = MONO_PCM_BUFFER_SAMPLES;
			InitParams.InstanceID = InArgs.InstanceID;
			InitParams.bIsPreviewSound = InArgs.bIsPreviewSound;
			InitParams.StartTime = InArgs.StartTime;

			SoundGenerator = InArgs.SoundWave->CreateSoundGenerator(InitParams, MoveTemp(InDefaultParams));

			// In the case of procedural audio generation, the mixer source buffer will never "loop" -- i.e. when it's done, it's done
			LoopingMode = LOOP_Never;
		}

		// Only allocate one buffer to render into for procedural sources
		const int32 NumBuffers = bProcedural ? 1 : Audio::MAX_BUFFERS_QUEUED;

		SourceVoiceBuffers.Reserve(NumBuffers);
		for (int32 BufferIndex = 0; BufferIndex < NumBuffers; ++BufferIndex)
		{
			SourceVoiceBuffers.Add(MakeShared<FAlignedFloatBuffer, ESPMode::ThreadSafe>());
		}
	}

	FMixerSourceBuffer::~FMixerSourceBuffer()
	{
		// GC methods may get called from the game thread during the destructor
		// These methods will trylock and early exit if we have this lock
		FScopeLock Lock(&SoundWaveCritSec);

		// OnEndGenerate calls EnsureTaskFinishes,
		// which will make sure we have completed our async realtime task before deleting the decompression state
		OnEndGenerate();

		// Clean up decompression state after things have been finished using it
		DeleteDecoder();

		if (SoundWave)
		{
			SoundWave->RemovePlayingSource(this);
		}
	}

	void FMixerSourceBuffer::SetDecoder(ICompressedAudioInfo* InCompressedAudioInfo)
	{
		if (DecompressionState == nullptr)
		{
			DecompressionState = InCompressedAudioInfo;
		}
	}

	void FMixerSourceBuffer::SetPCMData(const FRawPCMDataBuffer& InPCMDataBuffer)
	{
		check(BufferType == EBufferType::PCM || BufferType == EBufferType::PCMPreview);
		RawPCMDataBuffer = InPCMDataBuffer;
	}

	void FMixerSourceBuffer::SetCachedRealtimeFirstBuffers(TArray<uint8>&& InPrecachedBuffers)
	{
		CachedRealtimeFirstBuffer = MoveTemp(InPrecachedBuffers);
	}

	bool FMixerSourceBuffer::Init()
	{
		// We have successfully initialized which means our SoundWave has been flagged as bIsActive
		// GC can run between PreInit and Init so when cleaning up FMixerSourceBuffer, we don't want to touch SoundWave unless bInitailized is true.
		// SoundWave->bIsSoundActive will prevent GC until it is released in audio render thread
		bInitialized = true;

		switch (BufferType)
		{
		case EBufferType::PCM:
		case EBufferType::PCMPreview:
			SubmitInitialPCMBuffers();
			break;

		case EBufferType::PCMRealTime:
		case EBufferType::Streaming:
			if (!bProcedural)
			{
				SubmitInitialRealtimeBuffers();
			}
			break;

		case EBufferType::Invalid:
			break;
		}

		return true;
	}

	void FMixerSourceBuffer::OnBufferEnd()
	{
		if (bProcedural)
		{
			return;
		}

		FScopeTryLock Lock(&SoundWaveCritSec);

		// If the buffer is flagged as complete and there's nothing queued remaining.
		const bool bBufferCompleted = (NumBuffersQueued == 0 && bBufferFinished);

		// If we're procedural we must have a procedural SoundWave pointer to continue.
		const bool bProceduralStateBad = (bProcedural && !SoundWave);

		// If we're non-procedural and we don't have a decoder, bail. This can happen when the wave is GC'd.
		// The Decoder and SoundWave is deleted on the GameThread via FMixerSourceBuffer::OnBeginDestroy
		// Although this is bad state it's not an error, so just bail here.
		const bool bDecompressionStateBad = (!bProcedural && DecompressionState == nullptr);

		if (!Lock.IsLocked() || bBufferCompleted || bProceduralStateBad || bDecompressionStateBad || bHasError)
		{
			return;
		}

		ProcessRealTimeSource();
	}

	int32 FMixerSourceBuffer::GetNumBuffersQueued() const
	{
		FScopeTryLock Lock(&SoundWaveCritSec);
		if (Lock.IsLocked())
		{
			return NumBuffersQueued;
		}

		return 0;
	}

	TSharedPtr<FAlignedFloatBuffer, ESPMode::ThreadSafe> FMixerSourceBuffer::GetNextBuffer()
	{
		FScopeTryLock Lock(&SoundWaveCritSec);
		if (!Lock.IsLocked())
		{
			return nullptr;
		}

		if (bProcedural)
		{
			// Do rendering immediately.
			FProceduralAudioTaskData NewTaskData;

			// Make sure we actually have something to render
			check(SoundGenerator.IsValid() || (SoundWave && SoundWave->bProcedural));

			const int32 MaxSamples = SoundGenerator.IsValid() ? SoundGenerator->GetDesiredNumSamplesToRenderPerCallback() : MONO_PCM_BUFFER_SAMPLES * NumChannels;
			check(CurrentBuffer == 0);
			SourceVoiceBuffers[0]->SetNumUninitialized(MaxSamples, EAllowShrinking::No);

			NewTaskData.SoundGenerator = SoundGenerator;
			NewTaskData.AudioData = SourceVoiceBuffers[0]->GetData();
			NewTaskData.NumSamples = MaxSamples;

			FProceduralAudioTaskResults Results;
			DoProceduralRendering(NewTaskData, Results);
			FinishProceduralRendering(Results);
			bBufferFinished = Results.bIsFinished;

			return SourceVoiceBuffers[0];
		}

		TSharedPtr<FAlignedFloatBuffer, ESPMode::ThreadSafe> NewBufferPtr;
		if (BufferQueue.Dequeue(NewBufferPtr))
		{
			--NumBuffersQueued;
		}
		return NewBufferPtr;
	}

	void FMixerSourceBuffer::DoProceduralRendering(const FProceduralAudioTaskData& ProceduralTaskData, FProceduralAudioTaskResults& ProceduralResult)
	{
#if ENABLE_AUDIO_DEBUG
		FScopeDecodeTimer Timer(&ProceduralResult.CPUDuration);
#endif // if ENABLE_AUDIO_DEBUG
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FAsyncDecodeWorker_Procedural);
		if (ProceduralTaskData.SoundGenerator.IsValid())
		{
			// Generators are responsible to zero memory in case they can't generate the requested amount of samples
			ProceduralResult.NumSamplesWritten = ProceduralTaskData.SoundGenerator->GetNextBuffer(ProceduralTaskData.AudioData, ProceduralTaskData.NumSamples);
			ProceduralResult.bIsFinished = ProceduralTaskData.SoundGenerator->IsFinished();
			ProceduralResult.RelativeRenderCost = ProceduralTaskData.SoundGenerator->GetRelativeRenderCost();
		}
		else
		{
			// Make sure we've been flagged as active
			if (!SoundWave || !SoundWave->IsGeneratingAudio())
			{
				// Act as if we generated audio, but return silence.
				FMemory::Memzero(ProceduralTaskData.AudioData, ProceduralTaskData.NumSamples * sizeof(float));
				ProceduralResult.NumSamplesWritten = ProceduralTaskData.NumSamples;
				return;
			}

			// If we're not a float format, we need to convert the format to float
			const EAudioMixerStreamDataFormat::Type FormatType = SoundWave->GetGeneratedPCMDataFormat();
			if (FormatType != EAudioMixerStreamDataFormat::Float)
			{
				check(FormatType == EAudioMixerStreamDataFormat::Int16);

				int32 ByteSize = NumChannels * ProceduralTaskData.NumSamples * sizeof(int16);

				TArray<uint8> DecodeBuffer;
				DecodeBuffer.AddUninitialized(ByteSize);

				const int32 NumBytesWritten = SoundWave->GeneratePCMData(DecodeBuffer.GetData(), ProceduralTaskData.NumSamples);

				check(NumBytesWritten <= ByteSize);

				ProceduralResult.NumSamplesWritten = NumBytesWritten / sizeof(int16);
				Audio::ArrayPcm16ToFloat(MakeArrayView((int16*)DecodeBuffer.GetData(), ProceduralResult.NumSamplesWritten)
					, MakeArrayView(ProceduralTaskData.AudioData, ProceduralResult.NumSamplesWritten));
			}
			else
			{
				const int32 NumBytesWritten = SoundWave->GeneratePCMData((uint8*)ProceduralTaskData.AudioData, ProceduralTaskData.NumSamples);
				ProceduralResult.NumSamplesWritten = NumBytesWritten / sizeof(float);
			}
		}
	}

	void FMixerSourceBuffer::FinishProceduralRendering(const FProceduralAudioTaskResults& TaskResult)
	{
		// When doing procedural rendering, should only ever use buffer 0
		check(!bProcedural || CurrentBuffer == 0);

		SourceVoiceBuffers[CurrentBuffer]->SetNum(TaskResult.NumSamplesWritten, EAllowShrinking::No);

#if ENABLE_AUDIO_DEBUG
		double AudioDuration = static_cast<double>(TaskResult.NumSamplesWritten) / static_cast<double>(FMath::Max(1, NumChannels * SampleRate));
		UpdateCPUCoreUtilization(TaskResult.CPUDuration, AudioDuration);
#endif // ENABLE_AUDIO_DEBUG

		// Set the render cost encountered during the last render
		SetRelativeRenderCost(TaskResult.RelativeRenderCost);

		if (bProcedural)
		{
			ConnectToBuses();
		}
	}

	void FMixerSourceBuffer::SubmitInitialPCMBuffers()
	{
		CurrentBuffer = 0;

		RawPCMDataBuffer.NumSamples = RawPCMDataBuffer.DataSize / sizeof(int16);
		RawPCMDataBuffer.CurrentSample = 0;

		// Only submit data if we've successfully loaded it
		if (!RawPCMDataBuffer.Data || !RawPCMDataBuffer.DataSize)
		{
			return;
		}

		RawPCMDataBuffer.LoopCount = (LoopingMode != LOOP_Never) ? Audio::LOOP_FOREVER : 0;

		// Submit the first two format-converted chunks to the source voice
		const uint32 NumSamplesPerBuffer = MONO_PCM_BUFFER_SAMPLES * NumChannels;
		int16* RawPCMBufferDataPtr = (int16*)RawPCMDataBuffer.Data;

		// Prepare the buffer for the PCM submission
		SourceVoiceBuffers[0]->Reset(NumSamplesPerBuffer);
		SourceVoiceBuffers[0]->AddZeroed(NumSamplesPerBuffer);

		RawPCMDataBuffer.GetNextBuffer(*SourceVoiceBuffers[0], NumSamplesPerBuffer);

		SubmitBuffer(SourceVoiceBuffers[0]);

		CurrentBuffer = 1;
	}

	void FMixerSourceBuffer::SubmitInitialRealtimeBuffers()
	{
		static_assert(PLATFORM_NUM_AUDIODECOMPRESSION_PRECACHE_BUFFERS <= 2 && PLATFORM_NUM_AUDIODECOMPRESSION_PRECACHE_BUFFERS >= 0, "Unsupported number of precache buffers.");
		check(!bProcedural);

		CurrentBuffer = 0;

		bPlayedCachedBuffer = false;
		if (!bIsSeeking && CachedRealtimeFirstBuffer.Num() > 0)
		{
			bPlayedCachedBuffer = true;

			const uint32 NumSamples = NumPrecacheFrames * NumChannels;
			const uint32 BufferSize = NumSamples * sizeof(int16);

			// Format convert the first cached buffers
#if (PLATFORM_NUM_AUDIODECOMPRESSION_PRECACHE_BUFFERS == 2)
			{
				// Prepare the precache buffer memory
				for (int32 i = 0; i < 2; ++i)
				{
					SourceVoiceBuffers[i]->Reset();
					SourceVoiceBuffers[i]->AddZeroed(NumSamples);
				}

				int16* CachedBufferPtr0 = (int16*)CachedRealtimeFirstBuffer.GetData();
				int16* CachedBufferPtr1 = (int16*)(CachedRealtimeFirstBuffer.GetData() + BufferSize);
				float* AudioData0 = SourceVoiceBuffers[0]->GetData();
				float* AudioData1 = SourceVoiceBuffers[1]->GetData();

				Audio::ArrayPcm16ToFloat(MakeArrayView(CachedBufferPtr0, NumSamples), MakeArrayView(AudioData0, NumSamples));
				Audio::ArrayPcm16ToFloat(MakeArrayView(CachedBufferPtr1, NumSamples), MakeArrayView(AudioData1, NumSamples));

				// Submit the already decoded and cached audio buffers
				SubmitBuffer(SourceVoiceBuffers[0]);
				SubmitBuffer(SourceVoiceBuffers[1]);

				CurrentBuffer = 2;
			}
#elif (PLATFORM_NUM_AUDIODECOMPRESSION_PRECACHE_BUFFERS == 1)
			{
				SourceVoiceBuffers[0]->AudioData.Reset();
				SourceVoiceBuffers[0]->AudioData.AddZeroed(NumSamples);

				int16* CachedBufferPtr0 = (int16*)CachedRealtimeFirstBuffer.GetData();
				float* AudioData0 = SourceVoiceBuffers[0]->AudioData.GetData();
				Audio::ArrayPcm16ToFloat(MakeArrayView(CachedBufferPtr0, NumSamples), MakeArrayView(AudioData0, NumSamples));

				// Submit the already decoded and cached audio buffers
				SubmitBuffer(SourceVoiceBuffers[0]);

				CurrentBuffer = 1;
			}
#endif
		}
		else if (!bIsBus)
		{
			ProcessRealTimeSource();
		}
	}

	bool FMixerSourceBuffer::ReadMoreRealtimeData(ICompressedAudioInfo* InDecoder, const int32 BufferIndex, EBufferReadMode BufferReadMode)
	{
		check(!bProcedural);
		const int32 MaxSamples = MONO_PCM_BUFFER_SAMPLES * NumChannels;

		SourceVoiceBuffers[BufferIndex]->Reset();
		SourceVoiceBuffers[BufferIndex]->AddUninitialized(MaxSamples);

		if (BufferType != EBufferType::PCMRealTime && BufferType != EBufferType::Streaming)
		{
			check(RawPCMDataBuffer.Data != nullptr);

			// Read the next raw PCM buffer into the source buffer index. This converts raw PCM to float.
			return RawPCMDataBuffer.GetNextBuffer(*SourceVoiceBuffers[BufferIndex], MaxSamples);
		}

		// Handle the case that the decoder has an error and can't continue.
		if (InDecoder && InDecoder->HasError())
		{
			FMemory::Memzero(SourceVoiceBuffers[BufferIndex]->GetData(), MaxSamples * sizeof(float));

			FScopeTryLock Lock(&SoundWaveCritSec);
			if (Lock.IsLocked() && SoundWave)
			{
				SoundWave->SetError(TEXT("ICompressedAudioInfo::HasError() flagged on the Decoder"));
			}

			bHasError = true;
			bBufferFinished = true;
			return false;
		}

		check(InDecoder != nullptr);

		FDecodeAudioTaskData NewTaskData;
		NewTaskData.AudioData = SourceVoiceBuffers[BufferIndex]->GetData();
		NewTaskData.DecompressionState = InDecoder;
		NewTaskData.BufferType = BufferType;
		NewTaskData.NumChannels = NumChannels;
		NewTaskData.bLoopingMode = LoopingMode != LOOP_Never;
		NewTaskData.bSkipFirstBuffer = (BufferReadMode == EBufferReadMode::AsynchronousSkipFirstFrame);
		NewTaskData.NumFramesToDecode = MONO_PCM_BUFFER_SAMPLES;
		NewTaskData.NumPrecacheFrames = NumPrecacheFrames;
		NewTaskData.bForceSyncDecode = bForceSyncDecode;

		AsyncTaskStartTimeInCycles = FPlatformTime::Cycles64();
		FScopeLock Lock(&DecodeTaskCritSec);
		check(!AsyncRealtimeAudioTask);
		AsyncRealtimeAudioTask = CreateAudioTask(AuioDeviceID, NewTaskData);

		return false;
	}

	void FMixerSourceBuffer::SubmitRealTimeSourceData(const bool bInIsFinishedOrLooped)
	{
		// Have we reached the end of the sound
		if (bInIsFinishedOrLooped)
		{
			switch (LoopingMode)
			{
			case LOOP_Never:
				// Play out any queued buffers - once there are no buffers left, the state check at the beginning of IsFinished will fire
				bBufferFinished = true;
				break;

			case LOOP_WithNotification:
				// If we have just looped, and we are looping, send notification
				// This will trigger a WaveInstance->NotifyFinished() in the FXAudio2SoundSournce::IsFinished() function on main thread.
				bLoopCallback = true;
				break;

			case LOOP_Forever:
				// Let the sound loop indefinitely
				break;
			}
		}

		if (SourceVoiceBuffers[CurrentBuffer]->Num() > 0)
		{
			SubmitBuffer(SourceVoiceBuffers[CurrentBuffer]);
		}
	}

	void FMixerSourceBuffer::ProcessRealTimeSource()
	{
		FScopeLock Lock(&DecodeTaskCritSec);

		// Finish current decoding task
		if (AsyncRealtimeAudioTask)
		{
			AsyncRealtimeAudioTask->EnsureCompletion();

			bool bIsFinishedOrLooped = false;

			check(AsyncRealtimeAudioTask->GetType() == EAudioTaskType::Decode);
			{
				FDecodeAudioTaskResults TaskResult;
				AsyncRealtimeAudioTask->GetResult(TaskResult);
				bIsFinishedOrLooped = TaskResult.bIsFinishedOrLooped;

				SourceVoiceBuffers[CurrentBuffer]->SetNum(TaskResult.NumSamplesWritten, EAllowShrinking::No);
#if ENABLE_AUDIO_DEBUG
				double AudioDuration = static_cast<double>(TaskResult.NumSamplesWritten) / static_cast<double>(FMath::Max(1, NumChannels * SampleRate));
				UpdateCPUCoreUtilization(TaskResult.CPUDuration, AudioDuration);
#endif // ENABLE_AUDIO_DEBUG
			}

			delete AsyncRealtimeAudioTask;
			AsyncRealtimeAudioTask = nullptr;
			AsyncTaskStartTimeInCycles = 0;

			SubmitRealTimeSourceData(bIsFinishedOrLooped);
		}

		ConnectToBuses();

		// Start a new decoding task
		if (!AsyncRealtimeAudioTask && !bProcedural)
		{
			// Update the buffer index
			if (++CurrentBuffer > 2)
			{
				CurrentBuffer = 0;
			}

			EBufferReadMode DataReadMode;
			if (bPlayedCachedBuffer)
			{
				bPlayedCachedBuffer = false;
				DataReadMode = EBufferReadMode::AsynchronousSkipFirstFrame;
			}
			else
			{
				DataReadMode = EBufferReadMode::Asynchronous;
			}

			const bool bIsFinishedOrLooped = ReadMoreRealtimeData(DecompressionState, CurrentBuffer, DataReadMode);

			// If this was a synchronous read, then immediately write it
			if (AsyncRealtimeAudioTask == nullptr && !bHasError)
			{
				SubmitRealTimeSourceData(bIsFinishedOrLooped);
			}
		}

		if (bProcedural)
		{
			check(CurrentBuffer == 0);
			SubmitBuffer(SourceVoiceBuffers[CurrentBuffer]);
		}
	}

	void FMixerSourceBuffer::ConnectToBuses()
	{
		if (FAudioDeviceManager* ADM = FAudioDeviceManager::Get())
		{
			if (FAudioDevice* AudioDevice = ADM->GetAudioDeviceRaw(AuioDeviceID))
			{
				UAudioBusSubsystem* AudioBusSubsystem = AudioDevice->GetSubsystem<UAudioBusSubsystem>();
				check(AudioBusSubsystem);
				AudioBusSubsystem->ConnectPatches(InstanceID);
			}
		}
	}

	void FMixerSourceBuffer::SubmitBuffer(TSharedPtr<FAlignedFloatBuffer, ESPMode::ThreadSafe> InSourceVoiceBuffer)
	{
		NumBuffersQueued++;
		BufferQueue.Enqueue(InSourceVoiceBuffer);
	}

	bool FMixerSourceBuffer::IsEndOfAudio() const
	{
		return bProcedural ? bBufferFinished : GetNumBuffersQueued() == 0;
	}

	void FMixerSourceBuffer::DeleteDecoder()
	{
		// Clean up decompression state after things have been finished using it
		if (DecompressionState)
		{
			delete DecompressionState;
			DecompressionState = nullptr;
		}
	}

	bool FMixerSourceBuffer::OnBeginDestroy(USoundWave* /*Wave*/)
	{
		FScopeTryLock Lock(&SoundWaveCritSec);

		// if we don't have the lock, it means we are in ~FMixerSourceBuffer() on another thread
		if (Lock.IsLocked() && SoundWave)
		{
			EnsureAsyncTaskFinishes();
			DeleteDecoder();
			ClearWave();
			return true;
		}

		return false;
	}

	bool FMixerSourceBuffer::OnIsReadyForFinishDestroy(USoundWave* /*Wave*/) const
	{
		return false;
	}

	void FMixerSourceBuffer::OnFinishDestroy(USoundWave* /*Wave*/)
	{
		EnsureAsyncTaskFinishes();
		FScopeTryLock Lock(&SoundWaveCritSec);

		// if we don't have the lock, it means we are in ~FMixerSourceBuffer() on another thread
		if (Lock.IsLocked() && SoundWave)
		{
			DeleteDecoder();
			ClearWave();
		}
	}

	bool FMixerSourceBuffer::IsAsyncTaskInProgress() const
	{
		FScopeLock Lock(&DecodeTaskCritSec);
		return AsyncRealtimeAudioTask != nullptr;
	}

	bool FMixerSourceBuffer::IsAsyncTaskDone() const
	{
		FScopeLock Lock(&DecodeTaskCritSec);
		if (AsyncRealtimeAudioTask)
		{
			return AsyncRealtimeAudioTask->IsDone();
		}
		return true;
	}

	uint64 FMixerSourceBuffer::GetInstanceID() const
	{
		return InstanceID;
	}

	float FMixerSourceBuffer::GetRelativeRenderCost() const
	{
		return RelativeRenderCost.load(std::memory_order_relaxed);
	}

	void FMixerSourceBuffer::SetRelativeRenderCost(float InRelativeRenderCost)
	{
		RelativeRenderCost.store(InRelativeRenderCost, std::memory_order_relaxed);
	}

#if ENABLE_AUDIO_DEBUG
	double FMixerSourceBuffer::GetCPUCoreUtilization() const
	{
		return CPUCoreUtilization.load(std::memory_order_relaxed);
	}

	void FMixerSourceBuffer::UpdateCPUCoreUtilization(double InCPUTime, double InAudioTime)
	{
		constexpr double AnalysisTime = 1.0;

		if (InAudioTime > 0.0)
		{
			double NewUtilization = InCPUTime / InAudioTime;

			// Determine smoothing coefficients based upon duration of audio being rendered.
			const double DigitalCutoff = 1.0 / FMath::Max(1., AnalysisTime / InAudioTime);
			const double SmoothingBeta = FMath::Clamp(FMath::Exp(-UE_PI * DigitalCutoff), 0.0, 1.0 - UE_DOUBLE_SMALL_NUMBER);

			double PriorUtilization = CPUCoreUtilization.load(std::memory_order_relaxed);

			// Smooth value if utilization has been initialized.
			if (PriorUtilization > 0.0)
			{
				NewUtilization = (1.0 - SmoothingBeta) * NewUtilization + SmoothingBeta * PriorUtilization;
			}
			CPUCoreUtilization.store(NewUtilization, std::memory_order_relaxed);
		}
	}
#endif // ENABLE_AUDIO_DEBUG

	void FMixerSourceBuffer::GetDiagnosticState(FDiagnosticState& OutState)
	{
		// Query without a lock!
		OutState.bInFlight = AsyncRealtimeAudioTask != nullptr;
		OutState.WaveName = WaveName;
		OutState.bProcedural = bProcedural;
		OutState.RunTimeInSecs = OutState.bInFlight ?
			FPlatformTime::ToSeconds(FPlatformTime::Cycles64() - this->AsyncTaskStartTimeInCycles) :
			0.f;
	}

	void FMixerSourceBuffer::EnsureAsyncTaskFinishes()
	{
		FScopeLock Lock(&DecodeTaskCritSec);
		if (AsyncRealtimeAudioTask)
		{
			AsyncRealtimeAudioTask->CancelTask();

			delete AsyncRealtimeAudioTask;
			AsyncRealtimeAudioTask = nullptr;
		}
	}

	void FMixerSourceBuffer::OnBeginGenerate()
	{
		FScopeTryLock Lock(&SoundWaveCritSec);
		if (!Lock.IsLocked())
		{
			return;
		}

		if (SoundGenerator.IsValid())
		{
			SoundGenerator->OnBeginGenerate();
		}
		else
		{
			if (SoundWave && bProcedural)
			{
				check(SoundWave && SoundWave->bProcedural);
				SoundWave->OnBeginGenerate();
			}
		}
	}

	void FMixerSourceBuffer::OnEndGenerate()
	{
		// Make sure the async task finishes!
		EnsureAsyncTaskFinishes();

		FScopeTryLock Lock(&SoundWaveCritSec);
		if (!Lock.IsLocked())
		{
			return;
		}

		if (SoundGenerator.IsValid())
		{
			SoundGenerator->OnEndGenerate();
			if (SoundWave)
			{
				SoundWave->OnEndGenerate(SoundGenerator);
			}

			if (FAudioDeviceManager* ADM = FAudioDeviceManager::Get())
			{
				if (FAudioDevice* AudioDevice = ADM->GetAudioDeviceRaw(AuioDeviceID))
				{
					UAudioBusSubsystem* AudioBusSubsystem = AudioDevice->GetSubsystem<UAudioBusSubsystem>();
					check(AudioBusSubsystem);
					AudioBusSubsystem->RemoveSound(InstanceID);
				}
			}
		}
		else
		{
			// Only need to call OnEndGenerate and access SoundWave here if we successfully initialized
			if (SoundWave && bInitialized && bProcedural)
			{
				check(SoundWave && SoundWave->bProcedural);
				SoundWave->OnEndGenerate();
			}
		}
	}

}
