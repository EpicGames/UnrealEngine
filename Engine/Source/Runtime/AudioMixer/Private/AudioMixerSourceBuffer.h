// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AudioMixerBuffer.h"
#include "AudioMixerSourceManager.h"
#include "Sound/SoundWave.h"
#include "Sound/SoundGenerator.h"

namespace Audio
{
	static const int32 MAX_BUFFERS_QUEUED = 3;
	static const int32 LOOP_FOREVER = -1;

	struct FProceduralAudioTaskData;
	struct FProceduralAudioTaskResults;

	struct FRawPCMDataBuffer
	{
		uint8* Data;
		uint32 DataSize;
		int32 LoopCount;
		uint32 CurrentSample;
		uint32 NumSamples;

		bool GetNextBuffer(TArrayView<float> OutSourceBufferPtr, const uint32 NumSampleToGet);

		FRawPCMDataBuffer()
			: Data(nullptr)
			, DataSize(0)
			, LoopCount(0)
			, CurrentSample(0)
			, NumSamples(0)
		{}
	};

	/** Enum describing the data-read mode of an audio buffer. */
	enum class EBufferReadMode : uint8
	{
		/** Read the next buffer asynchronously. */
		Asynchronous,

		/** Read the next buffer asynchronously but skip the first chunk of audio. */
		AsynchronousSkipFirstFrame
	};

	using FMixerSourceBufferPtr = TSharedPtr<class FMixerSourceBuffer, ESPMode::ThreadSafe>;

	struct FMixerSourceBufferInitArgs
	{
		FDeviceId AudioDeviceID = 0;
		uint64 AudioComponentID = 0;
		uint64 InstanceID = 0;
		int32 SampleRate = 0;
		int32 AudioMixerNumOutputFrames = 0;
		FMixerBuffer* Buffer = nullptr;
		USoundWave* SoundWave = nullptr;
		ELoopingMode LoopingMode = ELoopingMode::LOOP_Never;
		float StartTime = 0.0f;
		bool bIsSeeking = false;
		bool bForceSyncDecode = false;
		bool bIsPreviewSound = false;
	};

	/** Class which handles decoding audio for a particular source buffer. */
	class FMixerSourceBuffer : public ISoundWaveClient
	{
	public:
		static FMixerSourceBufferPtr Create(FMixerSourceBufferInitArgs& InArgs, TArray<FAudioParameter>&& InDefaultParams=TArray<FAudioParameter>());

		~FMixerSourceBuffer();

		bool Init();

		// Sets the decoder to use for realtime async decoding
		void SetDecoder(ICompressedAudioInfo* InCompressedAudioInfo);

		// Sets the raw PCM data buffer to use for the source buffer
		void SetPCMData(const FRawPCMDataBuffer& InPCMDataBuffer);

		// Sets the precached buffers
		void SetCachedRealtimeFirstBuffers(TArray<uint8>&& InPrecachedBuffer);

		// Called by source manager when needing more buffers
		void OnBufferEnd();

		// Return the number of buffers enqueued on the mixer source buffer
		int32 GetNumBuffersQueued() const;

		// Returns the next enqueued buffer, returns nullptr if no buffers enqueued
		TSharedPtr<FAlignedFloatBuffer, ESPMode::ThreadSafe> GetNextBuffer();

		// Returns if buffer looped
		bool DidBufferLoop() const { return bLoopCallback; }

		// Returns true if buffer finished
		bool DidBufferFinish() const { return bBufferFinished; }

		// Returns true if the last buffer has been read
		bool IsEndOfAudio() const;

		// Called to start an async task to read more data
		bool ReadMoreRealtimeData(ICompressedAudioInfo* InDecoder, int32 BufferIndex, EBufferReadMode BufferReadMode);

		// Returns true if async task is in progress
		bool IsAsyncTaskInProgress() const;

		// Returns true if the async task is done
		bool IsAsyncTaskDone() const;

		// Get the unique instance ID (transmitter ID) for the source
		uint64 GetInstanceID() const;

		// Returns some diagnostic state
		struct FDiagnosticState
		{
			FName WaveName;
			float RunTimeInSecs=0.f;
			bool bInFlight=false;
			bool bProcedural=false;
		};
		void GetDiagnosticState(FDiagnosticState& OutState);

		// Ensures the async task finishes
		void EnsureAsyncTaskFinishes();

		// Begin and end generation on the audio render thread (audio mixer only)
		void OnBeginGenerate();
		void OnEndGenerate();
		void ClearWave() { SoundWave = nullptr; }

#if ENABLE_AUDIO_DEBUG
		double GetCPUCoreUtilization() const;
#endif // ENABLE_AUDIO_DEBUG

		// Returns the runtime render cost
		float GetRelativeRenderCost() const;

		// Note: can be called either directly on the rendering thread or from a background decode task.
		void DoProceduralRendering(const FProceduralAudioTaskData& ProceduralTaskData, FProceduralAudioTaskResults& ProceduralResult);
		void FinishProceduralRendering(const FProceduralAudioTaskResults& TaskResult);

	private:
		FMixerSourceBuffer(FMixerSourceBufferInitArgs& InArgs, TArray<FAudioParameter>&& InDefaultParams);

		void SubmitInitialPCMBuffers();
		void SubmitInitialRealtimeBuffers();
		void SubmitRealTimeSourceData(const bool bFinishedOrLooped);
		void ProcessRealTimeSource();
		void SubmitBuffer(TSharedPtr<FAlignedFloatBuffer, ESPMode::ThreadSafe> InSourceVoiceBuffer);
		void DeleteDecoder();
		void ConnectToBuses();

		int32 NumBuffersQueued;
		FRawPCMDataBuffer RawPCMDataBuffer;

		TArray<TSharedPtr<FAlignedFloatBuffer, ESPMode::ThreadSafe>> SourceVoiceBuffers;
		TQueue<TSharedPtr<FAlignedFloatBuffer, ESPMode::ThreadSafe>> BufferQueue;
		int32 CurrentBuffer;
		// SoundWaves are only set for procedural sound waves
		USoundWave* SoundWave;
		ISoundGeneratorPtr SoundGenerator;
		IAudioTask* AsyncRealtimeAudioTask;
		ICompressedAudioInfo* DecompressionState;
		ELoopingMode LoopingMode;
		int32 NumChannels;
		Audio::EBufferType::Type BufferType;
		int32 NumPrecacheFrames;
		Audio::FDeviceId AuioDeviceID;
		uint64 InstanceID;
		TArray<uint8> CachedRealtimeFirstBuffer;
		FName WaveName;
		uint64 AsyncTaskStartTimeInCycles=0;

#if ENABLE_AUDIO_DEBUG
		int32 SampleRate = 0;
		std::atomic<double> CPUCoreUtilization = 0.0;
		void UpdateCPUCoreUtilization(double InCPUTime, double InAudioTime);
#endif // ENABLE_AUDIO_DEBUG

		std::atomic<float> RelativeRenderCost = 1.0f;
		void SetRelativeRenderCost(float InRelativeRenderCost);

		mutable FCriticalSection SoundWaveCritSec;
		mutable FCriticalSection DecodeTaskCritSec;

		uint32 bInitialized : 1;
		uint32 bBufferFinished : 1;
		uint32 bPlayedCachedBuffer : 1;
		uint32 bIsSeeking : 1;
		uint32 bLoopCallback : 1;
		uint32 bProcedural : 1;
		uint32 bIsBus : 1;
		uint32 bForceSyncDecode : 1;
		uint32 bHasError : 1;
		
		virtual bool OnBeginDestroy(class USoundWave* Wave) override;
		virtual bool OnIsReadyForFinishDestroy(class USoundWave* Wave) const override;
		virtual void OnFinishDestroy(class USoundWave* Wave) override;
	};
}
