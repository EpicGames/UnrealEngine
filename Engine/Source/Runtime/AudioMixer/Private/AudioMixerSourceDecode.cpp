// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioMixerSourceDecode.h"
#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "AudioMixer.h"
#include "HAL/RunnableThread.h"
#include "AudioMixerBuffer.h"
#include "Async/Async.h"
#include "AudioDecompress.h"
#include "DSP/FloatArrayMath.h"

static int32 ForceSyncAudioDecodesCvar = 0;
FAutoConsoleVariableRef CVarForceSyncAudioDecodes(
	TEXT("au.ForceSyncAudioDecodes"),
	ForceSyncAudioDecodesCvar,
	TEXT("Disables using async tasks for processing sources.\n")
	TEXT("0: Not Disabled, 1: Disabled"),
	ECVF_Default);

namespace Audio
{

class FAsyncDecodeWorker : public FNonAbandonableTask
{
public:
	FAsyncDecodeWorker(const FHeaderParseAudioTaskData& InTaskData)
		: HeaderParseAudioData(InTaskData)
		, TaskType(EAudioTaskType::Header)
		, bIsDone(false)
	{
	}

	FAsyncDecodeWorker(const FDecodeAudioTaskData& InTaskData)
		: DecodeTaskData(InTaskData)
		, TaskType(EAudioTaskType::Decode)
		, bIsDone(false)
	{
	}

	~FAsyncDecodeWorker()
	{
	}

	void DoWork()
	{
		FScopedFTZFloatMode FTZ;

		switch (TaskType)
		{
		case EAudioTaskType::Header:
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FAsyncDecodeWorker_Header);
			HeaderParseAudioData.MixerBuffer->ReadCompressedInfo(HeaderParseAudioData.SoundWave);
		}
		break;

		case EAudioTaskType::Decode:
		{
#if ENABLE_AUDIO_DEBUG
			FScopeDecodeTimer Timer(&DecodeResult.CPUDuration);
#endif // if ENABLE_AUDIO_DEBUG
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FAsyncDecodeWorker_Decode);
			int32 NumChannels = DecodeTaskData.NumChannels;
			int32 ByteSize = NumChannels * DecodeTaskData.NumFramesToDecode * sizeof(int16);

			// Create a buffer to decode into that's of the appropriate size
			TArray<uint8> DecodeBuffer;
			DecodeBuffer.AddZeroed(ByteSize);

#if PLATFORM_NUM_AUDIODECOMPRESSION_PRECACHE_BUFFERS
			// skip the first buffers if we've already decoded them during Precache:
			if (DecodeTaskData.bSkipFirstBuffer)
			{
				const int32 kPCMBufferSize = NumChannels * DecodeTaskData.NumPrecacheFrames * sizeof(int16);
				int32 NumBytesStreamed = kPCMBufferSize;
				if (DecodeTaskData.BufferType == EBufferType::Streaming)
				{
					for (int32 NumberOfBuffersToSkip = 0; NumberOfBuffersToSkip < PLATFORM_NUM_AUDIODECOMPRESSION_PRECACHE_BUFFERS; NumberOfBuffersToSkip++)
					{
						DecodeTaskData.DecompressionState->StreamCompressedData(DecodeBuffer.GetData(), DecodeTaskData.bLoopingMode, kPCMBufferSize, NumBytesStreamed);
					}
				}
				else
				{
					for (int32 NumberOfBuffersToSkip = 0; NumberOfBuffersToSkip < PLATFORM_NUM_AUDIODECOMPRESSION_PRECACHE_BUFFERS; NumberOfBuffersToSkip++)
					{
						DecodeTaskData.DecompressionState->ReadCompressedData(DecodeBuffer.GetData(), DecodeTaskData.bLoopingMode, kPCMBufferSize);
					}
				}
			}
#endif

			const int32 kPCMBufferSize = NumChannels * DecodeTaskData.NumFramesToDecode * sizeof(int16);
			int32 NumBytesStreamed = kPCMBufferSize;
			if (DecodeTaskData.BufferType == EBufferType::Streaming)
			{
				DecodeResult.bIsFinishedOrLooped = DecodeTaskData.DecompressionState->StreamCompressedData(DecodeBuffer.GetData(), DecodeTaskData.bLoopingMode, kPCMBufferSize, NumBytesStreamed);
			}
			else
			{
				DecodeResult.bIsFinishedOrLooped = DecodeTaskData.DecompressionState->ReadCompressedData(DecodeBuffer.GetData(), DecodeTaskData.bLoopingMode, kPCMBufferSize);
			}

			const int32 NumSamplesStreamed = NumBytesStreamed / sizeof(int16);

			DecodeResult.NumSamplesWritten = NumSamplesStreamed;

			// Convert the decoded PCM data into a float buffer while still in the async task
			Audio::ArrayPcm16ToFloat(
				MakeArrayView((int16*)DecodeBuffer.GetData(), NumSamplesStreamed)
				, MakeArrayView(DecodeTaskData.AudioData, NumSamplesStreamed));
		}
		break;
		}
		bIsDone = true;
	}

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FAsyncDecodeWorker, STATGROUP_ThreadPoolAsyncTasks);
	}

	FHeaderParseAudioTaskData HeaderParseAudioData;
	FDecodeAudioTaskData DecodeTaskData;
	FDecodeAudioTaskResults DecodeResult;
	EAudioTaskType TaskType;
	FThreadSafeBool bIsDone;
};

class FDecodeHandleBase : public IAudioTask
{
public:
	FDecodeHandleBase()
		: Task(nullptr)
	{}

	virtual ~FDecodeHandleBase()
	{
		if (Task)
		{
			Task->EnsureCompletion(/*bIsLatencySensitive =*/ true);
			delete Task;
		}
	}

	virtual bool IsDone() const override
	{
		if (Task)
		{
			return Task->IsDone();
		}
		return true;
	}

	virtual void EnsureCompletion() override
	{
		if (Task)
		{
			Task->EnsureCompletion(/*bIsLatencySensitive =*/ true);
		}
	}

	virtual void CancelTask() override
	{
		if (Task)
		{
			// If Cancel returns false, it means we weren't able to cancel. So lets then fallback to ensure complete.
			if (!Task->Cancel())
			{
				Task->EnsureCompletion(/*bIsLatencySensitive =*/ true);
			}
		}
	}

protected:

	FAsyncTask<FAsyncDecodeWorker>* Task;
};

class FHeaderDecodeHandle : public FDecodeHandleBase
{
public:
	FHeaderDecodeHandle(const FHeaderParseAudioTaskData& InJobData)
	{
		Task = new FAsyncTask<FAsyncDecodeWorker>(InJobData);
		if (ForceSyncAudioDecodesCvar)
		{
			Task->StartSynchronousTask();
			return;
		}

		Task->StartBackgroundTask();
	}

	virtual EAudioTaskType GetType() const override
	{
		return EAudioTaskType::Header;
	}
};

class FDecodeHandle : public FDecodeHandleBase
{
public:
	FDecodeHandle(const FDecodeAudioTaskData& InJobData)
	{
		Task = new FAsyncTask<FAsyncDecodeWorker>(InJobData);
		if (ForceSyncAudioDecodesCvar || InJobData.bForceSyncDecode)
		{
			Task->StartSynchronousTask();
			return;
		}

		const bool bUseBackground = ShouldUseBackgroundPoolFor_FAsyncRealtimeAudioTask();
		Task->StartBackgroundTask(bUseBackground ? GBackgroundPriorityThreadPool : GThreadPool);
	}

	virtual EAudioTaskType GetType() const override
	{
		return EAudioTaskType::Decode;
	}

	virtual void GetResult(FDecodeAudioTaskResults& OutResult) override
	{
		Task->EnsureCompletion();
		const FAsyncDecodeWorker& DecodeWorker = Task->GetTask();
		OutResult = DecodeWorker.DecodeResult;
	}
};

IAudioTask* CreateAudioTask(Audio::FDeviceId InDeviceId, const FHeaderParseAudioTaskData& InJobData)
{
	return new FHeaderDecodeHandle(InJobData);
}

IAudioTask* CreateAudioTask(Audio::FDeviceId InDeviceId, const FDecodeAudioTaskData& InJobData)
{
	return new FDecodeHandle(InJobData);
}

}
