// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Queue.h"
#include "AudioDecompress.h"
#include "AudioMixerBuffer.h"

class USoundWave;

namespace Audio
{
	class FMixerBuffer;

	// Data needed for a decode audio task
	struct FDecodeAudioTaskData
	{
		// A pointer to a buffer of audio which will be decoded to
		float* AudioData;

		// Decompression state for decoder
		ICompressedAudioInfo* DecompressionState;

		// The buffer type for the decoder
		Audio::EBufferType::Type BufferType;

		// Number of channels of the decoder
		int32 NumChannels;

		// The number of frames which are precached
		int32 NumPrecacheFrames;

		// The number of frames to decode
		int32 NumFramesToDecode;

		// Whether or not this sound is intending to be looped
		bool bLoopingMode;

		// Whether or not to skip the first buffer
		bool bSkipFirstBuffer;

		// Force this decoding operation to occur synchronously,
		// regardless of the value of au.ForceSyncAudioDecodes. (used by time synth)
		bool bForceSyncDecode;

		FDecodeAudioTaskData()
			: AudioData(nullptr)
			, DecompressionState(nullptr)
			, BufferType(Audio::EBufferType::Invalid)
			, NumChannels(0)
			, NumPrecacheFrames(0)
			, NumFramesToDecode(0)
			, bLoopingMode(false)
			, bSkipFirstBuffer(false)
			, bForceSyncDecode(false)
		{}
	};

	// Data needed for a header parse audio task
	struct FHeaderParseAudioTaskData
	{
		// The mixer buffer object which results will be written to
		FMixerBuffer* MixerBuffer;

		// The sound wave object which contains the encoded file
		USoundWave* SoundWave;

		FHeaderParseAudioTaskData()
			: MixerBuffer(nullptr)
			, SoundWave(nullptr)
		{}
	};

	// Results from decode audio task
	struct FDecodeAudioTaskResults
	{

		// Whether or not the audio buffer looped
		bool bIsFinishedOrLooped;
		int32 NumSamplesWritten = 0;

#if ENABLE_AUDIO_DEBUG
		double CPUDuration = 0;
#endif // if ENABLE_AUDIO_DEBUG

		FDecodeAudioTaskResults()
			: bIsFinishedOrLooped(false)
		{}
	};

	// The types of audio tasks
	enum class EAudioTaskType
	{
		// The job is a header decode job
		Header,

		// The job is a decode job
		Decode,

		// The job is invalid (or unknown)
		Invalid,
	};

	// Handle to an in-flight decode job. Can be queried and used on any thread.
	class IAudioTask
	{
	public:
		virtual ~IAudioTask() = default;

		// Queries if the decode job has finished.
		virtual bool IsDone() const = 0;

		// Returns the job type of the handle.
		virtual EAudioTaskType GetType() const = 0;

		// Ensures the completion of the decode operation.
		virtual void EnsureCompletion() = 0;

		// Cancel the decode operation
		virtual void CancelTask() = 0;

		// Returns the result of a decode job
		virtual void GetResult(FDecodeAudioTaskResults& OutResult) {};
	};

	// Creates a task to decode a decoded file header
	IAudioTask* CreateAudioTask(Audio::FDeviceId InDeviceId, const FHeaderParseAudioTaskData& InJobData);

	// Creates a task to decode a chunk of audio
	IAudioTask* CreateAudioTask(Audio::FDeviceId InDeviceId, const FDecodeAudioTaskData& InJobData);

#if ENABLE_AUDIO_DEBUG
	struct FScopeDecodeTimer
	{
		FScopeDecodeTimer(double* OutResultSeconds)
			: Result(OutResultSeconds)
		{
			StartCycle = FPlatformTime::Cycles64();
		}
		~FScopeDecodeTimer()
		{
			uint64 EndCycle = FPlatformTime::Cycles64();
			if (Result)
			{
				*Result = static_cast<double>(EndCycle - StartCycle) * FPlatformTime::GetSecondsPerCycle64();
			}
		}

		double* Result = nullptr;
		uint64 StartCycle = 0;
	};
#endif // if ENABLE_AUDIO_DEBUG

}
