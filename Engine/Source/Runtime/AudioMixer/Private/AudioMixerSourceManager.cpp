// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioMixerSourceManager.h"

#include "Audio/AudioTimingLog.h"
#include "AudioDefines.h"
#include "AudioMixer.h"
#include "AudioMixerCVars.h"
#include "AudioMixerDevice.h"
#include "AudioMixerSourceBuffer.h"
#include "AudioMixerSourceVoice.h"
#include "AudioMixerSubmix.h"
#include "AudioMixerTrace.h"
#include "AudioRenderScheduler.h"
#include "AudioThread.h"
#include "DSP/FloatArrayMath.h"
#include "DSP/Dsp.h"
#include "IAudioExtensionPlugin.h"
#include "Sound/SoundModulationDestination.h"
#include "SoundFieldRendering.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Async/Async.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "HAL/PlatformStackWalk.h"
#include "Stats/Stats.h"
#include "Tasks/Task.h"
#include "Trace/Trace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioMixerSourceManager)

#if WITH_AUDIO_MIXER_THREAD_COMMAND_DEBUG
	#define AUDIO_MIXER_THREAD_COMMAND_STRING(X) (X)
#else //WITH_AUDIO_MIXER_THREAD_COMMAND_DEBUG
	#define AUDIO_MIXER_THREAD_COMMAND_STRING(X) ("")
#endif //WITH_AUDIO_MIXER_THREAD_COMMAND_DEBUG

// Link to "Audio" profiling category
CSV_DECLARE_CATEGORY_MODULE_EXTERN(AUDIOMIXERCORE_API, Audio);

static int32 DisableParallelSourceProcessingCvar = 0;
FAutoConsoleVariableRef CVarDisableParallelSourceProcessing(
	TEXT("au.DisableParallelSourceProcessing"),
	DisableParallelSourceProcessingCvar,
	TEXT("Disables using  tasks for processing sources.\n")
	TEXT("0: Not Disabled, 1: Disabled"),
	ECVF_Default);

static int32 DisableFilteringCvar = 0;
FAutoConsoleVariableRef CVarDisableFiltering(
	TEXT("au.DisableFiltering"),
	DisableFilteringCvar,
	TEXT("Disables using the per-source lowpass and highpass filter.\n")
	TEXT("0: Not Disabled, 1: Disabled"),
	ECVF_Default);

static int32 DisableHPFilteringCvar = 0;
FAutoConsoleVariableRef CVarDisableHPFiltering(
	TEXT("au.DisableHPFiltering"),
	DisableHPFilteringCvar,
	TEXT("Disables using the per-source highpass filter.\n")
	TEXT("0: Not Disabled, 1: Disabled"),
	ECVF_Default);

static int32 DisableEnvelopeFollowingCvar = 0;
FAutoConsoleVariableRef CVarDisableEnvelopeFollowing(
	TEXT("au.DisableEnvelopeFollowing"),
	DisableEnvelopeFollowingCvar,
	TEXT("Disables using the envlope follower for source envelope tracking.\n")
	TEXT("0: Not Disabled, 1: Disabled"),
	ECVF_Default);

static int32 DisableSourceEffectsCvar = 0;
FAutoConsoleVariableRef CVarDisableSourceEffects(
	TEXT("au.DisableSourceEffects"),
	DisableSourceEffectsCvar,
	TEXT("Disables using any source effects.\n")
	TEXT("0: Not Disabled, 1: Disabled"),
	ECVF_Default);

static int32 DisableDistanceAttenuationCvar = 0;
FAutoConsoleVariableRef CVarDisableDistanceAttenuation(
	TEXT("au.DisableDistanceAttenuation"),
	DisableDistanceAttenuationCvar,
	TEXT("Disables using any Distance Attenuation.\n")
	TEXT("0: Not Disabled, 1: Disabled"),
	ECVF_Default);

static int32 BypassAudioPluginsCvar = 0;
FAutoConsoleVariableRef CVarBypassAudioPlugins(
	TEXT("au.BypassAudioPlugins"),
	BypassAudioPluginsCvar,
	TEXT("Bypasses any audio plugin processing.\n")
	TEXT("0: Not Disabled, 1: Disabled"),
	ECVF_Default);

static int32 FlushCommandBufferOnTimeoutCvar = 0;
FAutoConsoleVariableRef CVarFlushCommandBufferOnTimeout(
	TEXT("au.FlushCommandBufferOnTimeout"),
	FlushCommandBufferOnTimeoutCvar,
	TEXT("When set to 1, flushes audio render thread synchronously when our fence has timed out.\n")
	TEXT("0: Not Disabled, 1: Disabled"),
	ECVF_Default);

static int32 CommandBufferFlushWaitTimeMsCvar = 1000;
FAutoConsoleVariableRef CVarCommandBufferFlushWaitTimeMs(
	TEXT("au.CommandBufferFlushWaitTimeMs"),
	CommandBufferFlushWaitTimeMsCvar,
	TEXT("How long to wait for the command buffer flush to complete.\n"),
	ECVF_Default);

static int32 CommandBufferMaxSizeInMbCvar = 10;
FAutoConsoleVariableRef CVarCommandBufferMaxSizeMb(
	TEXT("au.CommandBufferMaxSizeInMb"),
	CommandBufferMaxSizeInMbCvar,
	TEXT("How big to allow the command buffer to grow before ignoring more commands"),
	ECVF_Default);

static int32 CommandBufferInitialCapacityCvar = 500;
FAutoConsoleVariableRef CVarCommandBufferInitialCapacity(
	TEXT("au.CommandBufferInitialCapacity"),
	CommandBufferInitialCapacityCvar,
	TEXT("How many elements to initialize the command buffer capacity with"),
	ECVF_Default);

static float AudioCommandExecTimeMsWarningThresholdCvar = 500.f;
FAutoConsoleVariableRef CVarAudioCommandExecTimeMsWarningThreshold(
	TEXT("au.AudioThreadCommand.ExecutionTimeWarningThresholdInMs"),
	AudioCommandExecTimeMsWarningThresholdCvar,
	TEXT("If a command took longer to execute than this number (in milliseconds) then we log a warning"),
	ECVF_Default);

static int32 LogEveryAudioThreadCommandCvar = 0;
FAutoConsoleVariableRef LogEveryAudioThreadCommand(
	TEXT("au.AudioThreadCommand.LogEveryExecution"),
	LogEveryAudioThreadCommandCvar,
	TEXT("Extremely verbose logging of each Audio Thread command caller and it's execution time"),
	ECVF_Default);

static int32 LogCmdQueueWhenNumberReachedCVar = 0;
FAutoConsoleVariableRef CVarLogCmdQueueWhenNumberReached(
	TEXT("au.debug.LogCmdQueueWhenReached"),
	LogCmdQueueWhenNumberReachedCVar,
	TEXT("When number of commands in queue is reached, it will dump the command list to the log."),
	ECVF_Cheat);

static int32 NumCmdsConsideredFullCVar = 1000;
FAutoConsoleVariableRef CVarNumCmdsConsideredFullCVar(
	TEXT("au.NumCmdsConsideredFullCVar"),
	NumCmdsConsideredFullCVar,
	TEXT("Num of commands in the queue is considered full"),
	ECVF_Default);

static float FloatCompareMinScaleCVar = 0.001f;
FAutoConsoleVariableRef CVarFloatCompareMinScale(
	TEXT("au.FloatCompareMinScale"),
	FloatCompareMinScaleCVar,
	TEXT("Minimum parameter float compare tolerance"),
	ECVF_Default);

static float FloatCompareMaxScaleCVar = 0.1f;
FAutoConsoleVariableRef CVarFloatCompareMaxScale(
	TEXT("au.FloatCompareMaxScale"),
	FloatCompareMaxScaleCVar,
	TEXT("Maximum parameter float compare tolerance"),
	ECVF_Default);


static bool bDumpCallstackOnStallDiagnostics = true;
FAutoConsoleVariableRef CVarDumpCallstackOnStallDiagnostics(
	TEXT("au.DumpCallstackOnStallDiagnostics"),
	bDumpCallstackOnStallDiagnostics,
	TEXT("Whether to dump the callstack to the log as part of stall diagnostics (default = true)"),
	ECVF_Default);

// +/- 4 Octaves (default)
static float MaxModulationPitchRangeFreqCVar = 16.0f;
static float MinModulationPitchRangeFreqCVar = 0.0625f;
static FAutoConsoleCommand GModulationSetMaxPitchRange(
	TEXT("au.Modulation.SetPitchRange"),
	TEXT("Sets max final modulation range of pitch (in semitones). Default: 96 semitones (+/- 4 octaves)"),
	FConsoleCommandWithArgsDelegate::CreateStatic(
		[](const TArray<FString>& Args)
		{
			if (Args.Num() < 1)
			{
				UE_LOGF(LogAudioMixer, Error, "Failed to set max modulation pitch range: Range not provided");
				return;
			}

			const float Range = FCString::Atof(*Args[0]);
			MaxModulationPitchRangeFreqCVar = Audio::GetFrequencyMultiplier(Range * 0.5f);
			MinModulationPitchRangeFreqCVar = Audio::GetFrequencyMultiplier(Range * -0.5f);
		}
	)
);


#define ENVELOPE_TAIL_THRESHOLD (1.58489e-5f) // -96 dB

#define VALIDATE_SOURCE_MIXER_STATE 1

#if AUDIO_MIXER_ENABLE_DEBUG_MODE

// Macro which checks if the source id is in debug mode, avoids having a bunch of #ifdefs in code
#define AUDIO_MIXER_DEBUG_LOG(SourceId, Format, ...)																							\
	if (SourceInfos[SourceId].bIsDebugMode)																													\
	{																																			\
		FString CustomMessage = FString::Printf(Format, ##__VA_ARGS__);																			\
		FString LogMessage = FString::Printf(TEXT("<Debug Sound Log> [Id=%d][Name=%s]: %s"), SourceId, *SourceInfos[SourceId].GetDebugName(), *CustomMessage);	\
		UE_LOGF(LogAudioMixer, Log, "%ls", *LogMessage);																								\
	}

#else

#define AUDIO_MIXER_DEBUG_LOG(SourceId, Message)

#endif

// Global count for _all_ queued commands (across all devices).
TRACE_DECLARE_ATOMIC_INT_COUNTER(AudioMixerSourceManager_TotalQdCmds, TEXT("AudioSourceManager Qd Cmds"));

// Define profiling for source manager. 
DEFINE_STAT(STAT_AudioMixerHRTF);
DEFINE_STAT(STAT_AudioMixerSourceBuffers);
DEFINE_STAT(STAT_AudioMixerSourceEffectBuffers);
DEFINE_STAT(STAT_AudioMixerSourceManagerUpdate);
DEFINE_STAT(STAT_AudioMixerSourceOutputBuffers);

#if UE_AUDIO_PROFILERTRACE_ENABLED
// Mixer Source messages
UE_TRACE_EVENT_BEGIN(Audio, SoundVolume)
	UE_TRACE_EVENT_FIELD(uint32, DeviceId)
	UE_TRACE_EVENT_FIELD(uint64, Timestamp)
	UE_TRACE_EVENT_FIELD(uint32, PlayOrder)
	UE_TRACE_EVENT_FIELD(uint32, ActiveSoundPlayOrder)
	UE_TRACE_EVENT_FIELD(float, Volume)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Audio, SoundDistanceAttenuation)
	UE_TRACE_EVENT_FIELD(uint32, DeviceId)
	UE_TRACE_EVENT_FIELD(uint64, Timestamp)
	UE_TRACE_EVENT_FIELD(uint32, PlayOrder)
	UE_TRACE_EVENT_FIELD(uint32, ActiveSoundPlayOrder)
	UE_TRACE_EVENT_FIELD(float, DistanceAttenuation)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Audio, SoundPitch)
	UE_TRACE_EVENT_FIELD(uint32, DeviceId)
	UE_TRACE_EVENT_FIELD(uint64, Timestamp)
	UE_TRACE_EVENT_FIELD(uint32, PlayOrder)
	UE_TRACE_EVENT_FIELD(uint32, ActiveSoundPlayOrder)
	UE_TRACE_EVENT_FIELD(float, Pitch)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Audio, SoundSourceFilters)
	UE_TRACE_EVENT_FIELD(uint32, DeviceId)
	UE_TRACE_EVENT_FIELD(uint64, Timestamp)
	UE_TRACE_EVENT_FIELD(uint32, PlayOrder)
	UE_TRACE_EVENT_FIELD(uint32, ActiveSoundPlayOrder)
	UE_TRACE_EVENT_FIELD(float, LPFFrequency)
	UE_TRACE_EVENT_FIELD(float, HPFFrequency)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Audio, SoundSourceEnvelope)
	UE_TRACE_EVENT_FIELD(uint32, DeviceId)
	UE_TRACE_EVENT_FIELD(uint64, Timestamp)
	UE_TRACE_EVENT_FIELD(uint32, PlayOrder)
	UE_TRACE_EVENT_FIELD(uint32, ActiveSoundPlayOrder)
	UE_TRACE_EVENT_FIELD(float, Envelope)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Audio, SoundToSubmixSend)
	UE_TRACE_EVENT_FIELD(uint32, DeviceId)
	UE_TRACE_EVENT_FIELD(uint64, Timestamp)
	UE_TRACE_EVENT_FIELD(uint32, SenderID)
	UE_TRACE_EVENT_FIELD(uint32[], ReceiverIDs)
	UE_TRACE_EVENT_FIELD(float[], SendLevels)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Audio, SoundToBusSend)
	UE_TRACE_EVENT_FIELD(uint32, DeviceId)
	UE_TRACE_EVENT_FIELD(uint64, Timestamp)
	UE_TRACE_EVENT_FIELD(uint32, SenderID)
	UE_TRACE_EVENT_FIELD(uint32[], ReceiverIDs)
	UE_TRACE_EVENT_FIELD(float[], SendLevels)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Audio, BusPatchWriterConnected)
	UE_TRACE_EVENT_FIELD(uint32, DeviceId)
	UE_TRACE_EVENT_FIELD(uint64, Timestamp)
	UE_TRACE_EVENT_FIELD(uint32, SenderID)
	UE_TRACE_EVENT_FIELD(uint32[], ReceiverIDs)
	UE_TRACE_EVENT_FIELD(float[], SendLevels)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Audio, BusPatchReaderConnected)
	UE_TRACE_EVENT_FIELD(uint32, DeviceId)
	UE_TRACE_EVENT_FIELD(uint64, Timestamp)
	UE_TRACE_EVENT_FIELD(uint32, SenderID)
	UE_TRACE_EVENT_FIELD(uint32[], ReceiverIDs)
	UE_TRACE_EVENT_FIELD(float[], SendLevels)
UE_TRACE_EVENT_END()

// Audio Bus messages
UE_TRACE_EVENT_BEGIN(Audio, AudioBusStart)
	UE_TRACE_EVENT_FIELD(uint32, DeviceId)
	UE_TRACE_EVENT_FIELD(uint32, AudioBusId)
	UE_TRACE_EVENT_FIELD(uint64, Timestamp)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Name)
	UE_TRACE_EVENT_FIELD(int32, NumChannels)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Audio, AudioBusIsAlivePing)
	UE_TRACE_EVENT_FIELD(uint32, DeviceId)
	UE_TRACE_EVENT_FIELD(uint32, AudioBusId)
	UE_TRACE_EVENT_FIELD(uint64, Timestamp)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Name)
	UE_TRACE_EVENT_FIELD(int32, NumChannels)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Audio, AudioBusStop)
	UE_TRACE_EVENT_FIELD(uint32, DeviceId)
	UE_TRACE_EVENT_FIELD(uint32, AudioBusId)
	UE_TRACE_EVENT_FIELD(uint64, Timestamp)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Audio, AudioBusHasActivity)
	UE_TRACE_EVENT_FIELD(uint32, DeviceId)
	UE_TRACE_EVENT_FIELD(uint32, AudioBusId)
	UE_TRACE_EVENT_FIELD(uint64, Timestamp)
	UE_TRACE_EVENT_FIELD(bool, HasActivity)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Audio, AudioBusEnvelopeFollowerEnabled)
	UE_TRACE_EVENT_FIELD(uint32, DeviceId)
	UE_TRACE_EVENT_FIELD(uint32, AudioBusId)
	UE_TRACE_EVENT_FIELD(uint64, Timestamp)
	UE_TRACE_EVENT_FIELD(bool, EnvelopeFollowerEnabled)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Audio, AudioBusEnvelopeValues)
	UE_TRACE_EVENT_FIELD(uint32, DeviceId)
	UE_TRACE_EVENT_FIELD(uint32, AudioBusId)
	UE_TRACE_EVENT_FIELD(uint64, Timestamp)
	UE_TRACE_EVENT_FIELD(float[], EnvelopeValues)
UE_TRACE_EVENT_END()
#endif // UE_AUDIO_PROFILERTRACE_ENABLED


#ifndef CASE_ENUM_TO_TEXT
	#define CASE_ENUM_TO_TEXT(TXT) case TXT: return TEXT(#TXT);
#endif

const TCHAR* LexToString(ESourceManagerRenderThreadPhase InPhase)
{
	switch(InPhase)
	{
		FOREACH_ENUM_ESOURCEMANAGERRENDERTHREADPHASE(CASE_ENUM_TO_TEXT)
	} 
	return TEXT("Unknown");
}

namespace Audio
{
	int32 GetCommandBufferInitialCapacity()
	{
		return FMath::Clamp(CommandBufferInitialCapacityCvar, 0, 10000);
	}

#if UE_AUDIO_PROFILERTRACE_ENABLED
	void TraceAudioBusStartMessage(const uint32 InDeviceId, const uint32 InObjectId, const FString& InAudioBusName, const int32 InNumChannels)
	{
		const bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AudioChannel);
		if (bChannelEnabled)
		{
			UE_TRACE_LOG(Audio, AudioBusStart, AudioChannel)
				<< AudioBusStart.DeviceId(InDeviceId)
				<< AudioBusStart.AudioBusId(InObjectId)
				<< AudioBusStart.Timestamp(FPlatformTime::Cycles64())
				<< AudioBusStart.Name(InAudioBusName.IsEmpty() ? *FString::Printf(TEXT("%d"), InObjectId) : *InAudioBusName)
				<< AudioBusStart.NumChannels(InNumChannels);
		}
	}

	void TraceAudioBusIsAlivePingMessage(const uint32 InDeviceId, const uint32 InObjectId, const FString& InAudioBusName, const int32 InNumChannels)
	{
		const bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AudioChannel);
		if (bChannelEnabled)
		{
			UE_TRACE_LOG(Audio, AudioBusIsAlivePing, AudioChannel)
				<< AudioBusIsAlivePing.DeviceId(InDeviceId)
				<< AudioBusIsAlivePing.AudioBusId(InObjectId)
				<< AudioBusIsAlivePing.Timestamp(FPlatformTime::Cycles64())
				<< AudioBusIsAlivePing.Name(InAudioBusName.IsEmpty() ? *FString::Printf(TEXT("%d"), InObjectId) : *InAudioBusName)
				<< AudioBusIsAlivePing.NumChannels(InNumChannels);
		}
	}

	void TraceAudioBusStopMessage(const uint32 InDeviceId, const uint32 InObjectId)
	{
		const bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AudioChannel);
		if (bChannelEnabled)
		{
			UE_TRACE_LOG(Audio, AudioBusStop, AudioChannel)
				<< AudioBusStop.DeviceId(InDeviceId)
				<< AudioBusStop.AudioBusId(InObjectId)
				<< AudioBusStop.Timestamp(FPlatformTime::Cycles64());
		}
	}

	void TraceBusPatchWriterConnectedMessage(const uint32 InDeviceId, const uint32 InPlayOrder, const uint32 InBusObjectId)
	{
		const bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AudioChannel);
		if (!bChannelEnabled)
		{
			return;
		}

		constexpr float SendLevel = 1.0f;
		UE_TRACE_LOG(Audio, BusPatchWriterConnected, AudioChannel)
			<< BusPatchWriterConnected.DeviceId(InDeviceId)
			<< BusPatchWriterConnected.Timestamp(FPlatformTime::Cycles64())
			<< BusPatchWriterConnected.SenderID(InPlayOrder)
			<< BusPatchWriterConnected.ReceiverIDs(&InBusObjectId, 1)
			<< BusPatchWriterConnected.SendLevels(&SendLevel, 1);
	}

	void TraceBusPatchReaderConnectedMessage(const uint32 InDeviceId, const uint32 InBusObjectId, const uint32 InPlayOrder)
	{
		const bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AudioChannel);
		if (!bChannelEnabled)
		{
			return;
		}

		constexpr float SendLevel = 1.0f;
		UE_TRACE_LOG(Audio, BusPatchReaderConnected, AudioChannel)
			<< BusPatchReaderConnected.DeviceId(InDeviceId)
			<< BusPatchReaderConnected.Timestamp(FPlatformTime::Cycles64())
			<< BusPatchReaderConnected.SenderID(InBusObjectId)
			<< BusPatchReaderConnected.ReceiverIDs(&InPlayOrder, 1)
			<< BusPatchReaderConnected.SendLevels(&SendLevel, 1);
	}
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

	/*************************************************************************
	* FMixerSourceManager
	**************************************************************************/

	FMixerSourceManager::FMixerSourceManager(FMixerDevice* InMixerDevice)
		: MixerDevice(InMixerDevice)
		, NumActiveSources(0)
		, NumTotalSources(0)
		, NumLegacySources(0)
		, NumGeneratorSources(0)
		, NumOutputFrames(0)
		, NumOutputSamples(0)
		, bInitialized(false)
		, bUsingSpatializationPlugin(false)
		, bUsingSourceDataOverridePlugin(false)
	{
		// Get a manual resetable event
		const bool bIsManualReset = true;
		CommandsProcessedEvent = FPlatformProcess::GetSynchEventFromPool(bIsManualReset);
		check(CommandsProcessedEvent != nullptr);

		// Immediately trigger the command processed in case a flush happens before the audio thread swaps command buffers
		CommandsProcessedEvent->Trigger();

		// reserve the first buffer with the initial capacity
		CommandBuffers[0].SourceCommandQueue.Reserve(GetCommandBufferInitialCapacity());
	}

	FMixerSourceManager::~FMixerSourceManager()
	{
#if UE_AUDIO_PROFILERTRACE_ENABLED
		FTraceAuxiliary::OnTraceStarted.RemoveAll(this);
		FTraceAuxiliary::OnTraceStopped.RemoveAll(this);
		UnregisterAudioBusEnvelopeFollowerCVars();
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

		FPlatformProcess::ReturnSynchEventToPool(CommandsProcessedEvent);
	}

#if UE_AUDIO_PROFILERTRACE_ENABLED
	void FMixerSourceManager::OnTraceStarted(FTraceAuxiliary::EConnectionType TraceType, const FString& TraceDestination)
	{
		for (const auto& [AudioBusKey, AudioBus] : AudioBuses)
		{
			if (AudioBus.IsValid())
			{
 				TraceAudioBusStartMessage(MixerDevice->DeviceID, AudioBusKey.ObjectId, AudioBus->BusName, AudioBus->GetNumChannels());
			}
		}
	}

	void FMixerSourceManager::OnTraceStopped(FTraceAuxiliary::EConnectionType TraceType, const FString& TraceDestination)
	{
		FAudioThread::RunCommandOnAudioThread([this]()
		{
			AudioMixerThreadCommand([this]()
			{
				const bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AudioMixerChannel);

				for (auto& [AudioBusKey, AudioBus] : AudioBuses)
				{
					if (AudioBus.IsValid() && AudioBus->bIsEnvelopeFollowing && AudioBusesWithAutoEnvelopeFollowers.Contains(AudioBusKey.ObjectId))
					{
						const bool bStopped = AudioBus->StopEnvelopeFollower();
						if (bChannelEnabled && bStopped)
						{
							UE_TRACE_LOG(Audio, AudioBusEnvelopeFollowerEnabled, AudioMixerChannel)
								<< AudioBusEnvelopeFollowerEnabled.DeviceId(MixerDevice->DeviceID)
								<< AudioBusEnvelopeFollowerEnabled.AudioBusId(AudioBusKey.ObjectId)
								<< AudioBusEnvelopeFollowerEnabled.Timestamp(FPlatformTime::Cycles64())
								<< AudioBusEnvelopeFollowerEnabled.EnvelopeFollowerEnabled(false);
						}
					}
				}

				AudioBusesWithAutoEnvelopeFollowers.Reset();
			}, AUDIO_MIXER_THREAD_COMMAND_STRING("OnTraceStopped()"));
		});
	}

	void FMixerSourceManager::RegisterAudioBusEnvelopeFollowerCVars()
	{
		// StartAudioBusEnvelopeFollowerCVar
		StartAudioBusEnvelopeFollowerCVar = IConsoleManager::Get().RegisterConsoleVariable(
			*FString::Printf(TEXT("au.StartAudioBusEnvelopeFollower_AD%d"), MixerDevice->DeviceID),
			FString(),
			TEXT("Starts an audio bus envelope follower (args= audio bus id)\n"),
			ECVF_Default
		);

		if (StartAudioBusEnvelopeFollowerCVar)
		{
			StartAudioBusEnvelopeFollowerCVar->SetOnChangedCallback(FConsoleVariableDelegate::CreateRaw(this, &FMixerSourceManager::OnStartAudioBusEnvelopeFollowerCVarChanged));
		}

		// StopAudioBusEnvelopeFollowerCVar
		StopAudioBusEnvelopeFollowerCVar = IConsoleManager::Get().RegisterConsoleVariable(
			*FString::Printf(TEXT("au.StopAudioBusEnvelopeFollower_AD%d"), MixerDevice->DeviceID),
			FString(),
			TEXT("Stops an audio bus envelope follower (args= audio bus id)\n"),
			ECVF_Default
		);

		if (StopAudioBusEnvelopeFollowerCVar)
		{
			StopAudioBusEnvelopeFollowerCVar->SetOnChangedCallback(FConsoleVariableDelegate::CreateRaw(this, &FMixerSourceManager::OnStopAudioBusEnvelopeFollowerCVarChanged));
		}
	}

	void FMixerSourceManager::UnregisterAudioBusEnvelopeFollowerCVars()
	{
		IConsoleManager::Get().UnregisterConsoleObject(StartAudioBusEnvelopeFollowerCVar);
		IConsoleManager::Get().UnregisterConsoleObject(StopAudioBusEnvelopeFollowerCVar);
	}

	void FMixerSourceManager::OnStartAudioBusEnvelopeFollowerCVarChanged(IConsoleVariable* InStartAudioBusEnvelopeFollowerCVar)
	{
		ensure(IsInGameThread());

		if (InStartAudioBusEnvelopeFollowerCVar)
		{
			uint32 AudioBusId = INDEX_NONE;
			LexFromString(AudioBusId, *InStartAudioBusEnvelopeFollowerCVar->GetString());

			FAudioThread::RunCommandOnAudioThread([this, AudioBusId]()
			{
				AudioMixerThreadCommand([this, AudioBusId]()
				{
					StartAudioBusEnvelopeFollowerFromCVar(AudioBusId);
				}, AUDIO_MIXER_THREAD_COMMAND_STRING("OnStartAudioBusEnvelopeFollowerCVarChanged"));
			});
		}
	}

	void FMixerSourceManager::OnStopAudioBusEnvelopeFollowerCVarChanged(IConsoleVariable* InStopAudioBusEnvelopeFollowerCVar)
	{
		ensure(IsInGameThread());

		if (InStopAudioBusEnvelopeFollowerCVar)
		{
			uint32 AudioBusId = INDEX_NONE;
			LexFromString(AudioBusId, *InStopAudioBusEnvelopeFollowerCVar->GetString());

			FAudioThread::RunCommandOnAudioThread([this, AudioBusId]()
			{
				AudioMixerThreadCommand([this, AudioBusId]()
				{
					StopAudioBusEnvelopeFollowerFromCVar(AudioBusId);
				}, AUDIO_MIXER_THREAD_COMMAND_STRING("OnStopAudioBusEnvelopeFollowerCVarChanged"));
			});
		}
	}

	void FMixerSourceManager::StartAudioBusEnvelopeFollowerFromCVar(const uint32 InAudioBusId)
	{
		AUDIO_MIXER_CHECK_AUDIO_PLAT_THREAD(MixerDevice);

		constexpr float AttackTime  = 10.0f;
		constexpr float ReleaseTime = 500.0f;

		const bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AudioMixerChannel);

		for (auto& [AudioBusKey, AudioBus] : AudioBuses)
		{
			if (AudioBusKey.ObjectId == InAudioBusId && AudioBus.IsValid())
			{
				const bool bStarted = AudioBus->StartEnvelopeFollower(AttackTime, ReleaseTime, MixerDevice->GetSampleRate());

				if (bChannelEnabled && bStarted)
				{
					UE_TRACE_LOG(Audio, AudioBusEnvelopeFollowerEnabled, AudioMixerChannel)
						<< AudioBusEnvelopeFollowerEnabled.DeviceId(MixerDevice->DeviceID)
						<< AudioBusEnvelopeFollowerEnabled.AudioBusId(AudioBusKey.ObjectId)
						<< AudioBusEnvelopeFollowerEnabled.Timestamp(FPlatformTime::Cycles64())
						<< AudioBusEnvelopeFollowerEnabled.EnvelopeFollowerEnabled(true);
				}

				break;
			}
		}
	}

	void FMixerSourceManager::StopAudioBusEnvelopeFollowerFromCVar(const uint32 InAudioBusId)
	{
		AUDIO_MIXER_CHECK_AUDIO_PLAT_THREAD(MixerDevice);

		const bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AudioMixerChannel);

		for (auto& [AudioBusKey, AudioBus] : AudioBuses)
		{
			if (AudioBusKey.ObjectId == InAudioBusId && AudioBus.IsValid())
			{
				const bool bStopped = AudioBus->StopEnvelopeFollower();

				if (bChannelEnabled && bStopped)
				{
					UE_TRACE_LOG(Audio, AudioBusEnvelopeFollowerEnabled, AudioMixerChannel)
						<< AudioBusEnvelopeFollowerEnabled.DeviceId(MixerDevice->DeviceID)
						<< AudioBusEnvelopeFollowerEnabled.AudioBusId(AudioBusKey.ObjectId)
						<< AudioBusEnvelopeFollowerEnabled.Timestamp(FPlatformTime::Cycles64())
						<< AudioBusEnvelopeFollowerEnabled.EnvelopeFollowerEnabled(false);
				}

				break;
			}
		}
	}

	void FMixerSourceManager::AutomaticallyStartAudioBusEnvelopeFollower(const FAudioBusKey& InAudioBusKey, const TSharedPtr<FMixerAudioBus>& InAudioBus)
	{
		if (!AudioBusesWithAutoEnvelopeFollowers.Contains(InAudioBusKey.ObjectId))
		{
			constexpr float AttackTime = 10.0f;
			constexpr float ReleaseTime = 500.0f;

			AudioBusesWithAutoEnvelopeFollowers.Add(InAudioBusKey.ObjectId);

			const bool bStarted = InAudioBus->StartEnvelopeFollower(AttackTime, ReleaseTime, MixerDevice->GetSampleRate());
			if (bStarted)
			{
				UE_TRACE_LOG(Audio, AudioBusEnvelopeFollowerEnabled, AudioMixerChannel)
					<< AudioBusEnvelopeFollowerEnabled.DeviceId(MixerDevice->DeviceID)
					<< AudioBusEnvelopeFollowerEnabled.AudioBusId(InAudioBusKey.ObjectId)
					<< AudioBusEnvelopeFollowerEnabled.Timestamp(FPlatformTime::Cycles64())
					<< AudioBusEnvelopeFollowerEnabled.EnvelopeFollowerEnabled(true);
			}
		}
	}

	void FMixerSourceManager::StopAutoAudioBusEnvelopeFollower(const uint32 InAudioBusId, FMixerAudioBus& InAudioBus)
	{
		if (AudioBusesWithAutoEnvelopeFollowers.Remove(InAudioBusId) && InAudioBus.bIsEnvelopeFollowing)
		{
			const bool bStopped = InAudioBus.StopEnvelopeFollower();
			const bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AudioMixerChannel);
			if (bStopped && bChannelEnabled)
			{
				UE_TRACE_LOG(Audio, AudioBusEnvelopeFollowerEnabled, AudioMixerChannel)
					<< AudioBusEnvelopeFollowerEnabled.DeviceId(MixerDevice->DeviceID)
					<< AudioBusEnvelopeFollowerEnabled.AudioBusId(InAudioBusId)
					<< AudioBusEnvelopeFollowerEnabled.Timestamp(FPlatformTime::Cycles64())
					<< AudioBusEnvelopeFollowerEnabled.EnvelopeFollowerEnabled(false);
			}
		}
	}

	void FMixerSourceManager::TraceSourceToSubmixSendMessage(const FSourceInfo& SourceInfo) const
	{
		const bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AudioMixerChannel);

		if (!bChannelEnabled || SourceInfo.SubmixSends.IsEmpty())
		{
			return;
		}

		TArray<uint32, TInlineAllocator<32>> SendReceiverIDs;
		TArray<float, TInlineAllocator<32>> SendLevels;

		SendReceiverIDs.Reserve(SourceInfo.SubmixSends.Num());
		SendLevels.Reserve(SourceInfo.SubmixSends.Num());

		for (const Audio::FMixerSourceSubmixSend& SubmixSend : SourceInfo.SubmixSends)
		{
			const FMixerSubmixPtr SubmixPtr = SubmixSend.Submix.Pin();
			if (SubmixPtr.IsValid())
			{
				SendReceiverIDs.Add(GetTypeHash(SubmixPtr->GetPathName()));
				SendLevels.Add(SubmixSend.SendLevel);
			}
		}

		if (!SendReceiverIDs.IsEmpty() && ensure(SendReceiverIDs.Num() == SendLevels.Num()))
		{
			UE_TRACE_LOG(Audio, SoundToSubmixSend, AudioMixerChannel)
				<< SoundToSubmixSend.DeviceId(MixerDevice->DeviceID)
				<< SoundToSubmixSend.Timestamp(FPlatformTime::Cycles64())
				<< SoundToSubmixSend.SenderID(SourceInfo.PlayOrder)
				<< SoundToSubmixSend.ReceiverIDs(SendReceiverIDs.GetData(), SendReceiverIDs.Num())
				<< SoundToSubmixSend.SendLevels(SendLevels.GetData(), SendLevels.Num());
		}
	}

	void FMixerSourceManager::TraceSourceToBusSendMessage(const uint32 SourceId, const FSourceInfo& SourceInfo) const
	{
		const bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AudioMixerChannel);
		if (!bChannelEnabled)
		{
			return;
		}

		TArray<uint32, TInlineAllocator<32>> BusReceiverIDs;
		TArray<float, TInlineAllocator<32>> BusSendLevels;

		for (int32 AudioBusSendType = 0; AudioBusSendType < (int32)EBusSendType::Count; ++AudioBusSendType)
		{
			for (uint32 AudioBusId : SourceInfo.AudioBusSends[AudioBusSendType])
			{
				const TSharedPtr<FMixerAudioBus>* AudioBusPtr = AudioBuses.Find(AudioBusId);
				if (AudioBusPtr && AudioBusPtr->IsValid())
				{
					FAudioBusSend* SourceToBusSendInfo = (*AudioBusPtr)->AudioBusSends[AudioBusSendType].FindByPredicate([SourceId](const FAudioBusSend& BusSendInfo)
					{
						return BusSendInfo.SourceId == SourceId;
					});

					if (SourceToBusSendInfo)
					{
						BusReceiverIDs.Add(AudioBusId);
						BusSendLevels.Add(SourceToBusSendInfo->SendLevel);
					}
				}
			}
		}

		if (!BusReceiverIDs.IsEmpty() && ensure(BusReceiverIDs.Num() == BusSendLevels.Num()))
		{
			UE_TRACE_LOG(Audio, SoundToBusSend, AudioMixerChannel)
				<< SoundToBusSend.DeviceId(MixerDevice->DeviceID)
				<< SoundToBusSend.Timestamp(FPlatformTime::Cycles64())
				<< SoundToBusSend.SenderID(SourceInfo.PlayOrder)
				<< SoundToBusSend.ReceiverIDs(BusReceiverIDs.GetData(), BusReceiverIDs.Num())
				<< SoundToBusSend.SendLevels(BusSendLevels.GetData(), BusSendLevels.Num());
		}
	}

	uint32 FMixerSourceManager::ResolvePlayOrder(const FPendingAudioBusConnection& Connection) const
	{
		uint32 PlayOrder = Connection.WaveInstancePlayOrder;
		if (PlayOrder == INDEX_NONE)
		{
			const uint32* FoundPlayOrder = InstanceIdToPlayOrderMap.Find(Connection.InstanceID);
			if (FoundPlayOrder)
			{
				PlayOrder = *FoundPlayOrder;
			}
		}

		return PlayOrder;
	}
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

	void FMixerSourceManager::Init(const FSourceManagerInitParams& InitParams)
	{
		AUDIO_MIXER_CHECK(InitParams.NumSources > 0);

		if (bInitialized || !MixerDevice)
		{
			return;
		}

		AUDIO_MIXER_CHECK(MixerDevice->GetSampleRate() > 0);

		NumLegacySources = InitParams.NumSources;
		NumGeneratorSources = InitParams.NumGeneratorSources;
		NumTotalSources = NumLegacySources + NumGeneratorSources;

		NumOutputFrames = MixerDevice->PlatformSettings.CallbackBufferFrameSize;
		NumOutputSamples = NumOutputFrames * MixerDevice->GetNumDeviceChannels();
		
		GeneratorSourceState.AddDefaulted(NumGeneratorSources);

		MixerSources.Init(nullptr, NumLegacySources);

		// Populate output sources array with default data
		SourceSubmixOutputBuffers.Reset(NumTotalSources);
		for (int32 Index = 0; Index < NumTotalSources; Index++)
		{
			SourceSubmixOutputBuffers.Emplace(MixerDevice, 2, MixerDevice->GetNumDeviceChannels(), NumOutputFrames);
		}

		SourceInfos.AddDefaulted(NumTotalSources);

		for (int32 i = 0; i < NumTotalSources; ++i)
		{
			FSourceInfo& SourceInfo = SourceInfos[i];

			SourceInfo.SourceManager = this;
			SourceInfo.MixerSourceBuffer = nullptr;

			SourceInfo.VolumeSourceStart = -1.0f;
			SourceInfo.VolumeSourceDestination = -1.0f;
			SourceInfo.VolumeFadeSlope = 0.0f;
			SourceInfo.VolumeFadeStart = 0.0f;
			SourceInfo.VolumeFadeFramePosition = 0;
			SourceInfo.VolumeFadeNumFrames = 0;

			SourceInfo.DistanceAttenuationSourceStart = -1.0f;
			SourceInfo.DistanceAttenuationSourceDestination = -1.0f;

			SourceInfo.LowPassFreq = MAX_FILTER_FREQUENCY;
			SourceInfo.HighPassFreq = MIN_FILTER_FREQUENCY;

			SourceInfo.SourceListener = nullptr;
			SourceInfo.CurrentPCMBuffer = nullptr;
			SourceInfo.CurrentFrameIndex = 0;
			SourceInfo.NumFramesPlayed = 0;
			SourceInfo.SubmixSends.Reset();
			SourceInfo.AudioBusId = INDEX_NONE;
			SourceInfo.SourceBusDurationFrames = INDEX_NONE;
		
			SourceInfo.AudioBusSends[(int32)EBusSendType::PreEffect].Reset();
			SourceInfo.AudioBusSends[(int32)EBusSendType::PostEffect].Reset();
			SourceInfo.AudioBusSends[(int32)EBusSendType::PostAttenuation].Reset();

			SourceInfo.SourceEffectChainId = INDEX_NONE;

			Audio::FInlineEnvelopeFollowerInitParams EnvelopeFollowerInitParams;
			EnvelopeFollowerInitParams.SampleRate = MixerDevice->SampleRate;
			EnvelopeFollowerInitParams.AttackTimeMsec = 10.f;
			EnvelopeFollowerInitParams.ReleaseTimeMsec = 100.f;
			EnvelopeFollowerInitParams.Mode = EPeakMode::Peak;
			SourceInfo.SourceEnvelopeFollower = Audio::FInlineEnvelopeFollower(EnvelopeFollowerInitParams);

			SourceInfo.SourceEnvelopeValue = 0.0f;
			SourceInfo.bEffectTailsDone = false;
		
			SourceInfo.ResetModulators(MixerDevice->DeviceID);

			SourceInfo.bIs3D = false;
			SourceInfo.bIsCenterChannelOnly = false;
			SourceInfo.bIsActive = false;
			SourceInfo.bIsPlaying = false;
			SourceInfo.bIsPaused = false;
			SourceInfo.bIsPausedForQuantization = false;
			SourceInfo.bDelayLineSet = false;
			SourceInfo.bIsStopping = false;
			SourceInfo.bIsDone = false;
			SourceInfo.bIsLastBuffer = false;
			SourceInfo.bIsBusy = false;
			SourceInfo.bUseHRTFSpatializer = false;
			SourceInfo.bUseOcclusionPlugin = false;
			SourceInfo.bUseReverbPlugin = false;
			SourceInfo.bHasStarted = false;
			SourceInfo.bEnableBusSends = false;
			SourceInfo.bEnableBaseSubmix = false;
			SourceInfo.bEnableSubmixSends = false;
			SourceInfo.bHasAttenuationSubmixSends = false;
			SourceInfo.bIsVorbis = false;
			SourceInfo.bHasPreDistanceAttenuationSend = false;
			SourceInfo.bModFiltersUpdated = false;
			SourceInfo.bResampling = false;

#if AUDIO_MIXER_ENABLE_DEBUG_MODE
			SourceInfo.bIsDebugMode = false;
#endif // AUDIO_MIXER_ENABLE_DEBUG_MODE

			SourceInfo.NumInputChannels = 0;
			SourceInfo.NumPostEffectChannels = 0;
			SourceInfo.NumInputFrames = 0;
		}
		
		GameThreadInfo.bIsBusy.AddDefaulted(NumLegacySources);
		GameThreadInfo.bNeedsSpeakerMap.AddDefaulted(NumLegacySources);
		GameThreadInfo.bIsDebugMode.AddDefaulted(NumLegacySources);
		GameThreadInfo.bIsUsingHRTFSpatializer.AddDefaulted(NumLegacySources);
#if ENABLE_AUDIO_DEBUG
		GameThreadInfo.CPUCoreUtilization.AddZeroed(NumLegacySources);
#endif // if ENABLE_AUDIO_DEBUG

		GameThreadInfo.RelativeRenderCost.Reset(NumLegacySources);
		GameThreadInfo.FreeSourceIndices.Reset(NumLegacySources);
		GameThreadInfo.ModulationVolume.Reset(NumLegacySources);
		for (int32 i = NumLegacySources - 1; i >= 0; --i)
		{
			GameThreadInfo.RelativeRenderCost.Add(1.0f);
			GameThreadInfo.FreeSourceIndices.Add(i);
			GameThreadInfo.ModulationVolume.Add(1.0f);
		}

		// Initialize the source buffer memory usage to max source scratch buffers (num frames times max source channels)
		for (int32 SourceId = 0; SourceId < NumTotalSources; ++SourceId)
		{
			FSourceInfo& SourceInfo = SourceInfos[SourceId];

			SourceInfo.SourceBuffer.Reset(NumOutputFrames * 8);
			SourceInfo.PreDistanceAttenuationBuffer.Reset(NumOutputFrames * 8);
			SourceInfo.SourceEffectScratchBuffer.Reset(NumOutputFrames * 8);
			SourceInfo.AudioPluginOutputData.AudioBuffer.Reset(NumOutputFrames * 2);
		}

		// Cache the spatialization plugin
		bUsingSpatializationPlugin = false;
		SpatialInterfaceInfo = MixerDevice->GetCurrentSpatializationPluginInterfaceInfo();
		const auto& SpatializationPlugin = SpatialInterfaceInfo.SpatializationPlugin;
		if (SpatialInterfaceInfo.SpatializationPlugin.IsValid())
		{
			bUsingSpatializationPlugin = true;
		}
		// Cache the source data override plugin
		SourceDataOverridePlugin = MixerDevice->SourceDataOverridePluginInterface;
		if (SourceDataOverridePlugin.IsValid())
		{
			bUsingSourceDataOverridePlugin = true;
		}

		// Spam command queue with nops.
		static FAutoConsoleCommand SpamNopsCmd(
			TEXT("au.AudioThreadCommand.SpamCommandQueue"),
			TEXT(""),
			FConsoleCommandDelegate::CreateLambda([this]() 
			{				
				struct FSpamPayload
				{
					uint8 JunkBytes[1024];
				} Payload;
				for (int32 i = 0; i < 65536; ++i)
				{
					AudioMixerThreadCommand([Payload] {}, AUDIO_MIXER_THREAD_COMMAND_STRING("SpamNopsCmd() -- Console command"));
				}
			})
		);


		// submit a command that has an endless loop
		static FAutoConsoleCommand SpamEndlessCmd(
			TEXT("au.AudioThreadCommand.ChokeCommandQueue"),
			TEXT(""),
			FConsoleCommandDelegate::CreateLambda([this]()
			{
				AudioMixerThreadCommand([] {while(true){}}, AUDIO_MIXER_THREAD_COMMAND_STRING("ChokeCommandQueue() -- Console command"));
			})
		);

		// submit a MPSC command that has an endless loop
		static FAutoConsoleCommand SpamEndlessCmdMPSC(
			TEXT("au.AudioThreadCommand.ChokeMPSCCommandQueue"),
			TEXT(""),
			FConsoleCommandDelegate::CreateLambda([this]()
				{
					AudioMixerThreadMPSCCommand([] {while (true) {}}, AUDIO_MIXER_THREAD_COMMAND_STRING("ChokeMPSCCommandQueue() -- Console command"));
				})
		);

		// Test stall diagnostics.
		static FAutoConsoleCommand StallDiagnostics(
			TEXT("au.AudioSourceManager.HangDiagnostics"),
			TEXT(""),
			FConsoleCommandDelegate::CreateLambda([this]() { DoStallDiagnostics(); })
		);

		bInitialized = true;
		bPumpQueue = false;

#if UE_AUDIO_PROFILERTRACE_ENABLED
		RegisterAudioBusEnvelopeFollowerCVars();
		FTraceAuxiliary::OnTraceStarted.AddRaw(this, &FMixerSourceManager::OnTraceStarted);
		FTraceAuxiliary::OnTraceStopped.AddRaw(this, &FMixerSourceManager::OnTraceStopped);
#endif // UE_AUDIO_PROFILERTRACE_ENABLED
	}

	void FMixerSourceManager::Update(bool bTimedOut)
	{
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

#if VALIDATE_SOURCE_MIXER_STATE
		for (int32 i = 0; i < NumLegacySources; ++i)
		{
			if (!GameThreadInfo.bIsBusy[i])
			{
				// Make sure that our bIsFree and FreeSourceIndices are correct
				AUDIO_MIXER_CHECK(GameThreadInfo.FreeSourceIndices.Contains(i) == true);
			}
		}
#endif

		if (FPlatformProcess::SupportsMultithreading())
		{
			// If the command was triggered, then we want to do a swap of command buffers
			if (CommandsProcessedEvent->Wait(0))
			{
				int32 CurrentGameIndex = !RenderThreadCommandBufferIndex.GetValue();

				// This flags the audio render thread to be able to pump the next batch of commands
				// And will allow the audio thread to write to a new command slot
				const int32 NextIndex = (CurrentGameIndex + 1) & 1;

				FCommands& NextCommandBuffer = CommandBuffers[NextIndex];

				// Make sure we've actually emptied the command queue from the render thread before writing to it
				if (FlushCommandBufferOnTimeoutCvar && NextCommandBuffer.SourceCommandQueue.Num() != 0)
				{
					UE_LOGF(LogAudioMixer, Warning, "Audio render callback stopped. Flushing %d commands.", NextCommandBuffer.SourceCommandQueue.Num());

					// Pop and execute all the commands that came since last update tick
					for (int32 Id = 0; Id < NextCommandBuffer.SourceCommandQueue.Num(); ++Id)
					{
						FAudioMixerThreadCommand AudioCommand = NextCommandBuffer.SourceCommandQueue[Id];

						AudioCommand();
						NumCommands.Decrement();
						TRACE_COUNTER_DECREMENT(AudioMixerSourceManager_TotalQdCmds);
					}

					NextCommandBuffer.SourceCommandQueue.Reset();
				}

				// Here we ensure that we block for any pending calls to AudioMixerThreadCommand.
				FScopeLock ScopeLock(&CommandBufferIndexCriticalSection);
				RenderThreadCommandBufferIndex.Set(CurrentGameIndex);

				CommandsProcessedEvent->Reset();
			}
		}
		else
		{
			int32 CurrentRenderIndex = RenderThreadCommandBufferIndex.GetValue();
			int32 CurrentGameIndex = !RenderThreadCommandBufferIndex.GetValue();
			check(CurrentGameIndex == 0 || CurrentGameIndex == 1);
			check(CurrentRenderIndex == 0 || CurrentRenderIndex == 1);

			// If these values are the same, that means the audio render thread has finished the last buffer queue so is ready for the next block
			if (CurrentRenderIndex == CurrentGameIndex)
			{
				// This flags the audio render thread to be able to pump the next batch of commands
				// And will allow the audio thread to write to a new command slot
				const int32 NextIndex = !CurrentGameIndex;

				// Make sure we've actually emptied the command queue from the render thread before writing to it
				if (CommandBuffers[NextIndex].SourceCommandQueue.Num() != 0)
				{
					UE_LOGF(LogAudioMixer, Warning, "Source command queue not empty: %d", CommandBuffers[NextIndex].SourceCommandQueue.Num());
				}
				bPumpQueue = true;
			}
		}

	}

	void FMixerSourceManager::ReleaseSource(const int32 SourceId)
	{
		AUDIO_MIXER_CHECK_AUDIO_PLAT_THREAD(MixerDevice);

		AUDIO_MIXER_CHECK(SourceId < NumTotalSources);
		AUDIO_MIXER_CHECK(bInitialized);

		if (SourceId < NumLegacySources && MixerSources[SourceId] == nullptr)
		{
			UE_LOGF(LogAudioMixer, Warning, "Ignoring double release of SourceId: %i", SourceId);
			return;
		}

		AUDIO_MIXER_DEBUG_LOG(SourceId, TEXT("Is releasing"));
		
		FSourceInfo& SourceInfo = SourceInfos[SourceId];

		if (UseRenderScheduler())
		{
			check(SourceId >= NumLegacySources || SourceInfo.MixerSourceBuffer.IsValid());
			GetRenderScheduler().RemoveStep(FAudioRenderStepId::FromWaveInstancePlayOrder(SourceInfo.PlayOrder));
		}

#if AUDIO_MIXER_ENABLE_DEBUG_MODE
		if (SourceInfo.bIsDebugMode)
		{
			DebugSoloSources.Remove(SourceId);
		}
#endif
		// Remove from list of active bus or source ids depending on what type of source this is
		if (SourceInfo.AudioBusId != INDEX_NONE)
		{
			// Remove this bus from the registry of bus instances
			TSharedPtr<FMixerAudioBus> AudioBusPtr = AudioBuses.FindRef(SourceInfo.AudioBusId);
			if (AudioBusPtr.IsValid())
			{
				// If this audio bus was automatically created via source bus playback, this this audio bus can be removed
				if (AudioBusPtr->RemoveInstanceId(SourceId, SourceInfo.PlayOrder))
				{
#if UE_AUDIO_PROFILERTRACE_ENABLED
					TraceAudioBusStopMessage(MixerDevice->DeviceID, SourceInfo.AudioBusId);
					StopAutoAudioBusEnvelopeFollower(SourceInfo.AudioBusId, *AudioBusPtr);
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

					// Only automatic buses will be getting removed here. Otherwise they need to be manually removed from the source manager.
					ensure(AudioBusPtr->IsAutomatic());
					AudioBuses.Remove(SourceInfo.AudioBusId);
				}
			}
		}

		// Remove this source's send list from the bus data registry
		for (int32 AudioBusSendType = 0; AudioBusSendType < (int32)EBusSendType::Count; ++AudioBusSendType)
		{
			for (uint32 AudioBusId : SourceInfo.AudioBusSends[AudioBusSendType])
			{
				// we should have a bus registration entry still since the send hasn't been cleaned up yet
				TSharedPtr<FMixerAudioBus> AudioBusPtr = AudioBuses.FindRef(AudioBusId);
				if (AudioBusPtr.IsValid())
				{
					if (AudioBusPtr->RemoveSend((EBusSendType)AudioBusSendType, SourceId))
					{
#if UE_AUDIO_PROFILERTRACE_ENABLED
						TraceAudioBusStopMessage(MixerDevice->DeviceID, AudioBusId);
						StopAutoAudioBusEnvelopeFollower(AudioBusId, *AudioBusPtr);
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

						ensure(AudioBusPtr->IsAutomatic());
						AudioBuses.Remove(AudioBusId);
					}
				}
			}

			SourceInfo.AudioBusSends[AudioBusSendType].Reset();
		}

		SourceInfo.AudioBusId = INDEX_NONE;
		SourceInfo.SourceBusDurationFrames = INDEX_NONE;

		// Free the mixer source buffer data
		if (SourceInfo.MixerSourceBuffer.IsValid())
		{
#if UE_AUDIO_PROFILERTRACE_ENABLED
			InstanceIdToPlayOrderMap.Remove(SourceInfo.MixerSourceBuffer->GetInstanceID());
			ActiveBusPatchWriters.Remove(SourceInfo.PlayOrder);
			ActiveBusPatchReaders.Remove(SourceInfo.PlayOrder);
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

			if (UseRenderScheduler())
			{
				RemoveBusPatchDependencies(SourceInfo.MixerSourceBuffer->GetInstanceID());
			}
			PendingSourceBuffers.Add(SourceInfo.MixerSourceBuffer);
			SourceInfo.MixerSourceBuffer = nullptr;
		}

		SourceInfo.SourceListener = nullptr;

		// Remove the mixer source from its submix sends
		for (FMixerSourceSubmixSend& SubmixSendItem : SourceInfo.SubmixSends)
		{
			FMixerSubmixPtr SubmixPtr = SubmixSendItem.Submix.Pin();
			if (SubmixPtr.IsValid())
			{
				SubmixPtr->RemoveSourceById(SourceId);
			}
		}
		SourceInfo.SubmixSends.Reset();

		// Notify plugin effects
		if (SourceInfo.bUseHRTFSpatializer)
		{
			AUDIO_MIXER_CHECK(bUsingSpatializationPlugin);
			LLM_SCOPE(ELLMTag::AudioMixerPlugins);
			SpatialInterfaceInfo.SpatializationPlugin->OnReleaseSource(SourceId);
		}

		if (SourceInfo.bUseOcclusionPlugin)
		{
			MixerDevice->OcclusionInterface->OnReleaseSource(SourceId);
		}

		if (SourceInfo.bUseReverbPlugin)
		{
			MixerDevice->ReverbPluginInterface->OnReleaseSource(SourceId);
		}

		if (SourceInfo.AudioLink)
		{
			SourceInfo.AudioLink->OnSourceReleased(SourceId);
			SourceInfo.AudioLink.Reset();
		}

		// Delete the source effects
		SourceInfo.SourceEffectChainId = INDEX_NONE;
		ResetSourceEffectChain(SourceId);

		SourceInfo.SourceEnvelopeFollower.Reset();
		SourceInfo.bEffectTailsDone = true;

		// Release the source voice back to the mixer device. This is pooled.
		if (SourceId < NumLegacySources)
		{
			MixerDevice->ReleaseMixerSourceVoice(MixerSources[SourceId]);
			MixerSources[SourceId] = nullptr;
		}

		// Reset all state and data
		SourceInfo.PitchSourceParam.Init();
		SourceInfo.VolumeSourceStart = -1.0f;
		SourceInfo.VolumeSourceDestination = -1.0f;
		SourceInfo.VolumeFadeSlope = 0.0f;
		SourceInfo.VolumeFadeStart = 0.0f;
		SourceInfo.VolumeFadeFramePosition = 0;
		SourceInfo.VolumeFadeNumFrames = 0;

		SourceInfo.DistanceAttenuationSourceStart = -1.0f;
		SourceInfo.DistanceAttenuationSourceDestination = -1.0f;

		SourceInfo.LowPassFreq = MAX_FILTER_FREQUENCY;
		SourceInfo.HighPassFreq = MIN_FILTER_FREQUENCY;

		if (SourceInfo.SourceBufferListener)
		{
			SourceInfo.SourceBufferListener->OnSourceReleased(SourceId);
			SourceInfo.SourceBufferListener.Reset();
		}

		SourceInfo.ResetModulators(MixerDevice->DeviceID);

		SourceInfo.LowPassFilter.Reset();
		SourceInfo.HighPassFilter.Reset();
		SourceInfo.CurrentPCMBuffer = nullptr;
		SourceInfo.SourceBuffer.Reset();
		SourceInfo.PreDistanceAttenuationBuffer.Reset();
		SourceInfo.PostDistanceAttenuationBuffer.Empty();
		SourceInfo.SourceEffectScratchBuffer.Reset();
		SourceInfo.AudioPluginOutputData.AudioBuffer.Reset();
		SourceInfo.CurrentFrameIndex = 0;
		SourceInfo.NumFramesPlayed = 0;
		SourceInfo.bIs3D = false;
		SourceInfo.bIsCenterChannelOnly = false;
		SourceInfo.bIsActive = false;
		SourceInfo.bIsPlaying = false;
		SourceInfo.bIsDone = true;
		SourceInfo.bIsLastBuffer = false;
		SourceInfo.bIsPaused = false;
		SourceInfo.bIsPausedForQuantization = false;
		SourceInfo.bDelayLineSet = false;
		SourceInfo.bIsStopping = false;
		SourceInfo.bIsBusy = false;
		SourceInfo.bUseHRTFSpatializer = false;
		SourceInfo.bIsExternalSend = false;
		SourceInfo.bUseOcclusionPlugin = false;
		SourceInfo.bUseReverbPlugin = false;
		SourceInfo.bHasStarted = false;
		SourceInfo.bEnableBusSends = false;
		SourceInfo.bEnableBaseSubmix = false;
		SourceInfo.bEnableSubmixSends = false;
		SourceInfo.bHasAttenuationSubmixSends = false;
		SourceInfo.bHasPreDistanceAttenuationSend = false;
		SourceInfo.bModFiltersUpdated = false;

		SourceInfo.AudioComponentID = 0;
		SourceInfo.PlayOrder = INDEX_NONE;

		SourceInfo.QuantizedCommandHandle.Reset();

#if AUDIO_MIXER_ENABLE_DEBUG_MODE
		SourceInfo.bIsDebugMode = false;
		SourceInfo.DebugName = FString();
#endif //AUDIO_MIXER_ENABLE_DEBUG_MODE

		SourceInfo.NumInputChannels = 0;
		SourceInfo.NumPostEffectChannels = 0;

		if (SourceId < NumLegacySources)
		{
			GameThreadInfo.bNeedsSpeakerMap[SourceId] = false;
		}
	}

	void FMixerSourceManager::BuildSourceEffectChain(const int32 SourceId, FSoundEffectSourceInitData& InitData, const TArray<FSourceEffectChainEntry>& InSourceEffectChain, TArray<TSoundEffectSourcePtr>& OutSourceEffects)
	{
		// Create new source effects. The memory will be owned by the source manager.
		FScopeLock ScopeLock(&EffectChainMutationCriticalSection);
		for (const FSourceEffectChainEntry& ChainEntry : InSourceEffectChain)
		{
			// Presets can have null entries
			if (!ChainEntry.Preset)
			{
				continue;
			}

			// Get this source effect presets unique id so instances can identify their originating preset object
			const uint32 PresetUniqueId = ChainEntry.Preset->GetUniqueID();
			InitData.ParentPresetUniqueId = PresetUniqueId;

			TSoundEffectSourcePtr NewEffect = USoundEffectPreset::CreateInstance<FSoundEffectSourceInitData, FSoundEffectSource>(InitData, *ChainEntry.Preset);
			NewEffect->SetEnabled(!ChainEntry.bBypass);

			OutSourceEffects.Add(NewEffect);
		}
	}

	void FMixerSourceManager::ResetSourceEffectChain(const int32 SourceId)
	{
		FScopeLock ScopeLock(&EffectChainMutationCriticalSection);
		{
			FSourceInfo& SourceInfo = SourceInfos[SourceId];

			// Unregister these source effect instances from their owning USoundEffectInstance on the audio thread.
			// Have to pass to Game Thread prior to processing on AudioThread to avoid race condition with GC.
			// (RunCommandOnAudioThread is not safe to call from any thread other than the GameThread).
			if (!SourceInfo.SourceEffects.IsEmpty())
			{
				AsyncTask(ENamedThreads::GameThread, [GTSourceEffects = MoveTemp(SourceInfo.SourceEffects)]() mutable
				{
					FAudioThread::RunCommandOnAudioThread([ATSourceEffects = MoveTemp(GTSourceEffects)]() mutable
					{
						for (const TSoundEffectSourcePtr& EffectPtr : ATSourceEffects)
						{
							USoundEffectPreset::UnregisterInstance(EffectPtr);
						}
					});
				});

				SourceInfo.SourceEffects.Reset();
			}
			SourceInfo.SourceEffectPresets.Reset();
		}
	}

	bool FMixerSourceManager::GetFreeSourceId(int32& OutSourceId)
	{
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		if (GameThreadInfo.FreeSourceIndices.Num())
		{
			OutSourceId = GameThreadInfo.FreeSourceIndices.Pop();

			AUDIO_MIXER_CHECK(OutSourceId < NumLegacySources);
			AUDIO_MIXER_CHECK(!GameThreadInfo.bIsBusy[OutSourceId]);

			AUDIO_MIXER_CHECK(!GameThreadInfo.bIsDebugMode[OutSourceId]);
			AUDIO_MIXER_CHECK(NumActiveSources < NumLegacySources);
			++NumActiveSources;

			GameThreadInfo.bIsBusy[OutSourceId] = true;
			return true;
		}
		AUDIO_MIXER_CHECK(false);
		return false;
	}

	int32 FMixerSourceManager::GetNumActiveSources() const
	{
		return NumActiveSources;
	}

	int32 FMixerSourceManager::GetNumActiveAudioBuses() const
	{
		return AudioBuses.Num();
	}

	void FMixerSourceManager::InitSource(const int32 SourceId, const FMixerSourceVoiceInitParams& InitParams)
	{
		AUDIO_MIXER_CHECK(SourceId < NumLegacySources);
		AUDIO_MIXER_CHECK(GameThreadInfo.bIsBusy[SourceId]);
		AUDIO_MIXER_CHECK(!GameThreadInfo.bIsDebugMode[SourceId]);
		AUDIO_MIXER_CHECK(InitParams.SourceListener != nullptr);
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

#if AUDIO_MIXER_ENABLE_DEBUG_MODE
		GameThreadInfo.bIsDebugMode[SourceId] = InitParams.bIsDebugMode;
#endif 

		// Make sure we flag that this source needs a speaker map to at least get one
		GameThreadInfo.bNeedsSpeakerMap[SourceId] = true;

		GameThreadInfo.bIsUsingHRTFSpatializer[SourceId] = InitParams.bUseHRTFSpatialization;

		// Need to build source effect instances on the audio thread
		FSoundEffectSourceInitData InitData;
		InitData.SampleRate = MixerDevice->SampleRate;
		InitData.NumSourceChannels = InitParams.NumInputChannels;
		InitData.AudioClock = MixerDevice->GetAudioTime();
		InitData.AudioDeviceId = MixerDevice->DeviceID;

		TArray<TSoundEffectSourcePtr> SourceEffectChain;
		BuildSourceEffectChain(SourceId, InitData, InitParams.SourceEffectChain, SourceEffectChain);

		FModulationDestination VolumeMod;
		VolumeMod.Init(MixerDevice->DeviceID, FName("Volume"), false /* bInIsBuffered */, true /* bInValueLinear */);
		VolumeMod.SetModulators(InitParams.ModulationSettings.VolumeModulationDestination.Modulators);

		FModulationDestination PitchMod;
		PitchMod.Init(MixerDevice->DeviceID, FName("Pitch"), false /* bInIsBuffered */);
		PitchMod.SetModulators(InitParams.ModulationSettings.PitchModulationDestination.Modulators);

		FModulationDestination HighpassMod;
		HighpassMod.Init(MixerDevice->DeviceID, FName("HPFCutoffFrequency"), false /* bInIsBuffered */);
		HighpassMod.SetModulators(InitParams.ModulationSettings.HighpassModulationDestination.Modulators);

		FModulationDestination LowpassMod;
		LowpassMod.Init(MixerDevice->DeviceID, FName("LPFCutoffFrequency"), false /* bInIsBuffered */);
		LowpassMod.SetModulators(InitParams.ModulationSettings.LowpassModulationDestination.Modulators);

		AudioMixerThreadCommand([
			this,
			SourceId,
			InitParams,
			VolumeModulation = MoveTemp(VolumeMod),
			HighpassModulation = MoveTemp(HighpassMod),
			LowpassModulation = MoveTemp(LowpassMod),
			PitchModulation = MoveTemp(PitchMod),
			SourceEffectChain
		]() mutable
		{
			AUDIO_MIXER_CHECK_AUDIO_PLAT_THREAD(MixerDevice);
			AUDIO_MIXER_CHECK(InitParams.SourceVoice != nullptr);

			FSourceInfo& SourceInfo = SourceInfos[SourceId];

			// Initialize the mixer source buffer decoder with the given mixer buffer
			SourceInfo.MixerSourceBuffer = InitParams.MixerSourceBuffer;
			AUDIO_MIXER_CHECK(SourceInfo.MixerSourceBuffer.IsValid());
			SourceInfo.MixerSourceBuffer->Init();
			SourceInfo.MixerSourceBuffer->OnBeginGenerate();

			SourceInfo.bIs3D = InitParams.bIs3D;
			SourceInfo.bIsPlaying = false;
			SourceInfo.bIsPaused = false;
			SourceInfo.bIsPausedForQuantization = false;
			SourceInfo.bDelayLineSet = false;
			SourceInfo.bIsStopping = false;
			SourceInfo.bIsActive = true;
			SourceInfo.bIsBusy = true;
			SourceInfo.bIsDone = false;
			SourceInfo.bIsLastBuffer = false;
			SourceInfo.bUseHRTFSpatializer = InitParams.bUseHRTFSpatialization;
			SourceInfo.bIsExternalSend = InitParams.bIsExternalSend;
			SourceInfo.bIsVorbis = InitParams.bIsVorbis;
			SourceInfo.PlayOrder = InitParams.PlayOrder;
			SourceInfo.ActiveSoundPlayOrder = InitParams.ActiveSoundPlayOrder;
			SourceInfo.AudioComponentID = InitParams.AudioComponentID;
			SourceInfo.bIsSoundfield = InitParams.bIsSoundfield;
			SourceInfo.bResampling = false;

#if UE_AUDIO_PROFILERTRACE_ENABLED
			if (SourceInfo.MixerSourceBuffer.IsValid())
			{
				InstanceIdToPlayOrderMap.Add(SourceInfo.MixerSourceBuffer->GetInstanceID(), SourceInfo.PlayOrder);
			}
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

			// Call initialization from the render thread so anything wanting to do any initialization here can do so (e.g. procedural sound waves)
			SourceInfo.SourceListener = InitParams.SourceListener;
			SourceInfo.SourceListener->OnBeginGenerate();

			SourceInfo.NumInputChannels = InitParams.NumInputChannels;
			SourceInfo.NumInputFrames = InitParams.NumInputFrames;

			SourceInfo.Resampler.Reset(SourceInfo.NumInputChannels);

			// init and zero-out buffers
			const int32 BufferSize = NumOutputFrames * InitParams.NumInputChannels;
			SourceInfo.PreEffectBuffer.Reset();
			SourceInfo.PreEffectBuffer.AddZeroed(BufferSize);

			SourceInfo.PreDistanceAttenuationBuffer.Reset();
			SourceInfo.PreDistanceAttenuationBuffer.AddZeroed(BufferSize);

			SourceInfo.PostDistanceAttenuationBuffer.Reset();
			SourceInfo.PostDistanceAttenuationBuffer.AddZeroed(BufferSize);

			// Initialize the number of per-source LPF filters based on input channels
			SourceInfo.LowPassFilter.Init(MixerDevice->SampleRate, InitParams.NumInputChannels);
			SourceInfo.HighPassFilter.Init(MixerDevice->SampleRate, InitParams.NumInputChannels);

			Audio::FInlineEnvelopeFollowerInitParams EnvelopeFollowerInitParams;
			EnvelopeFollowerInitParams.SampleRate = MixerDevice->SampleRate / NumOutputFrames;
			EnvelopeFollowerInitParams.AttackTimeMsec = (float)InitParams.EnvelopeFollowerAttackTime;
			EnvelopeFollowerInitParams.ReleaseTimeMsec = (float)InitParams.EnvelopeFollowerReleaseTime;
			EnvelopeFollowerInitParams.Mode = EPeakMode::Peak;
			SourceInfo.SourceEnvelopeFollower = Audio::FInlineEnvelopeFollower(EnvelopeFollowerInitParams);

			SourceInfo.VolumeModulation = MoveTemp(VolumeModulation);
			SourceInfo.PitchModulation = MoveTemp(PitchModulation);
			SourceInfo.LowpassModulation = MoveTemp(LowpassModulation);
			SourceInfo.HighpassModulation = MoveTemp(HighpassModulation);

			// Pass required info to clock manager
			const FQuartzQuantizedRequestData& QuantData = InitParams.QuantizedRequestData;
			if (QuantData.QuantizedCommandPtr)
			{
				if (false == MixerDevice->QuantizedEventClockManager.DoesClockExist(QuantData.ClockName))
				{
					UE_LOGF(LogAudioMixer, Warning, "Quantization Clock: '%ls' Does not exist.", *QuantData.ClockName.ToString());
					QuantData.QuantizedCommandPtr->Cancel();
				}
				else
				{
					FQuartzQuantizedCommandInitInfo QuantCommandInitInfo(QuantData, MixerDevice->GetSampleRate(), SourceId);
					SourceInfo.QuantizedCommandHandle = MixerDevice->QuantizedEventClockManager.AddCommandToClock(QuantCommandInitInfo);
				}
			}


			// Create the spatialization plugin source effect
			if (InitParams.bUseHRTFSpatialization)
			{
				AUDIO_MIXER_CHECK(bUsingSpatializationPlugin);
				LLM_SCOPE(ELLMTag::AudioMixerPlugins);

				// re-cache the spatialization plugin in case it changed
				bUsingSpatializationPlugin = false;
				SpatialInterfaceInfo = MixerDevice->GetCurrentSpatializationPluginInterfaceInfo();
				const auto& SpatializationPlugin = SpatialInterfaceInfo.SpatializationPlugin;
				if (SpatialInterfaceInfo.SpatializationPlugin.IsValid())
				{
					bUsingSpatializationPlugin = true;
				}

				SpatialInterfaceInfo.SpatializationPlugin->OnInitSource(SourceId, InitParams.AudioComponentUserID, InitParams.NumInputChannels, InitParams.SpatializationPluginSettings);
			}

			// Create the occlusion plugin source effect
			if (InitParams.OcclusionPluginSettings != nullptr)
			{
				MixerDevice->OcclusionInterface->OnInitSource(SourceId, InitParams.AudioComponentUserID, InitParams.NumInputChannels, InitParams.OcclusionPluginSettings);
				SourceInfo.bUseOcclusionPlugin = true;
			}

			// Create the reverb plugin source effect
			if (InitParams.ReverbPluginSettings != nullptr)
			{
				MixerDevice->ReverbPluginInterface->OnInitSource(SourceId, InitParams.AudioComponentUserID, InitParams.NumInputChannels, InitParams.ReverbPluginSettings);
				SourceInfo.bUseReverbPlugin = true;
			}

			if (InitParams.AudioLink.IsValid())
			{
				SourceInfo.AudioLink = InitParams.AudioLink;
			}

			// Optional Source Buffer listener.
			SourceInfo.SourceBufferListener = InitParams.SourceBufferListener;
			SourceInfo.bShouldSourceBufferListenerZeroBuffer = InitParams.bShouldSourceBufferListenerZeroBuffer;

			// Default all sounds to not consider effect chain tails when playing
			SourceInfo.bEffectTailsDone = true;

			// Which forms of routing to enable
			SourceInfo.bEnableBusSends = InitParams.bEnableBusSends;
			SourceInfo.bEnableBaseSubmix = InitParams.bEnableBaseSubmix;
			SourceInfo.bEnableSubmixSends = InitParams.bEnableSubmixSends;
			SourceInfo.bHasAttenuationSubmixSends = InitParams.bHasAttenuationSubmixSends;

			// Copy the source effect chain if the channel count is less than or equal to the number of channels supported by the effect chain
			if (InitParams.NumInputChannels <= InitParams.SourceEffectChainMaxSupportedChannels)
			{
				// If we're told to care about effect chain tails, then we're not allowed
				// to stop playing until the effect chain tails are finished
				SourceInfo.bEffectTailsDone = !InitParams.bPlayEffectChainTails;
				SourceInfo.SourceEffectChainId = InitParams.SourceEffectChainId;
				
				// Add the effect chain instances 
				SourceInfo.SourceEffects = MoveTemp(SourceEffectChain);
				
				// Add a slot entry for the preset so it can change while running. This will get sent to the running effect instance if the preset changes.
				SourceInfo.SourceEffectPresets.Add(nullptr);
				// If this is going to be a source bus, add this source id to the list of active bus ids
				if (InitParams.AudioBusId != INDEX_NONE)
				{
					// Setting this BusId will flag this source as a bus. It doesn't try to generate 
					// audio in the normal way but instead will render in a second stage, after normal source rendering.
					SourceInfo.AudioBusId = InitParams.AudioBusId;

					// Source bus duration allows us to stop a bus after a given time
					if (InitParams.SourceBusDuration != 0.0f)
					{
						SourceInfo.SourceBusDurationFrames = InitParams.SourceBusDuration * MixerDevice->GetSampleRate();
					}

					// If the bus is not registered, make a new entry. This will default to an automatic audio bus until explicitly made manual later.
					bool bAlreadyExisted = false;
					TSharedRef<FMixerAudioBus> AudioBusPtr = FindOrAddAudioBus(SourceInfo.AudioBusId, InitParams.AudioBusChannels, true, &bAlreadyExisted);
					AudioBusPtr->AddInstanceId(SourceId, SourceInfo.PlayOrder, InitParams.NumInputChannels);

#if UE_AUDIO_PROFILERTRACE_ENABLED
					if (!bAlreadyExisted)
					{
						AudioBusPtr->BusName = InitParams.AudioBusName;
						TraceAudioBusStartMessage(MixerDevice->DeviceID, InitParams.AudioBusId, InitParams.AudioBusName, InitParams.AudioBusChannels);
					}
#endif // UE_AUDIO_PROFILERTRACE_ENABLED
				}

			}

			// Iterate through source's bus sends and add this source to the bus send list
			// Note: buses can also send their audio to other buses.
			for (int32 BusSendType = 0; BusSendType < (int32)EBusSendType::Count; ++BusSendType)
			{
				for (const FInitAudioBusSend& AudioBusSend : InitParams.AudioBusSends[BusSendType])
				{
					// New struct to map which source (SourceId) is sending to the bus
					FAudioBusSend NewAudioBusSend;
					NewAudioBusSend.SourceId = SourceId;
					NewAudioBusSend.WaveInstancePlayOrder = SourceInfo.PlayOrder;
					NewAudioBusSend.SendLevel = AudioBusSend.SendLevel;

					// Get existing BusId and add the send, or create new bus registration. A newly created bus won't output audio, but
					// we register the send anyway in the event that this bus does play, we'll know to send this
					// source's audio to it.
					bool bAlreadyExisted = false;
					TSharedRef<FMixerAudioBus> AudioBusPtr = FindOrAddAudioBus(AudioBusSend.AudioBusId, AudioBusSend.BusChannels, true, &bAlreadyExisted);

					if (AudioBusSend.Filter.IsFiltering())
					{
						NewAudioBusSend.Filter = AudioBusSend.Filter;
						const int32 NumSourceChannels = FMath::Max(SourceInfo.NumInputChannels, 1);
						NewAudioBusSend.Lpf.Init(MixerDevice->SampleRate, NumSourceChannels);
						NewAudioBusSend.Hpf.Init(MixerDevice->SampleRate, NumSourceChannels);
					}

					AudioBusPtr->AddSend((EBusSendType)BusSendType, NewAudioBusSend);
#if UE_AUDIO_PROFILERTRACE_ENABLED
					if (!bAlreadyExisted)
					{
						AudioBusPtr->BusName = AudioBusSend.AudioBusName;
						TraceAudioBusStartMessage(MixerDevice->DeviceID, AudioBusSend.AudioBusId, AudioBusSend.AudioBusName, AudioBusSend.BusChannels);
					}
#endif // UE_AUDIO_PROFILERTRACE_ENABLED
					
					// Store on this source, which buses its sending its audio to
					SourceInfo.AudioBusSends[BusSendType].Add(AudioBusSend.AudioBusId);
				}
			}

			AUDIO_MIXER_CHECK(MixerSources[SourceId] == nullptr);
			MixerSources[SourceId] = InitParams.SourceVoice;

			// Loop through the source's sends and add this source to those submixes with the send info

			AUDIO_MIXER_CHECK(SourceInfo.SubmixSends.Num() == 0);

			// Initialize a new downmix data:
			check(SourceId < SourceInfos.Num());
			const int32 SourceInputChannels = (SourceInfo.bUseHRTFSpatializer && !SourceInfo.bIsExternalSend) ? 2 : SourceInfo.NumInputChannels;

			// Collect the soundfield encoding keys we need to initialize with our output buffers
			TArray<FMixerSubmixPtr> SoundfieldSubmixSends;

			for (int32 i = 0; i < InitParams.SubmixSends.Num(); ++i)
			{
				const FMixerSourceSubmixSend& MixerSubmixSend = InitParams.SubmixSends[i];

				FMixerSubmixPtr SubmixPtr = MixerSubmixSend.Submix.Pin();
				if (SubmixPtr.IsValid())
				{
					FMixerSourceSubmixSend& AddedSend = SourceInfo.SubmixSends.Add_GetRef(MixerSubmixSend);
					AddedSend.SubmixId = SubmixPtr->GetId();

					if (AddedSend.Filter.IsFiltering())
					{
						// Use device output channel count: the filtered buffer is the source device-format output
						// buffer, not the destination submix output channel count.
						const int32 NumDeviceChannels = FMath::Max(MixerDevice->GetNumDeviceChannels(), 1);
						AddedSend.Lpf.Init(MixerDevice->SampleRate, NumDeviceChannels);
						AddedSend.Hpf.Init(MixerDevice->SampleRate, NumDeviceChannels);
						
						// Snap coefficients to the initial cutoffs immediately. bIsFirstFrequencyChange (reset by
						// Init) forces CurrInterpLength=0 in StartFrequencyInterpolation, so coefficients land on
						// the target without interpolation rather than ramping from the default.
						if (AddedSend.Filter.bEnableLPF)
						{
							AddedSend.Lpf.StartFrequencyInterpolation(AddedSend.Filter.LpfCutoff, 1);
						}
						
						if (AddedSend.Filter.bEnableHPF)
						{
							AddedSend.Hpf.StartFrequencyInterpolation(AddedSend.Filter.HpfCutoff, 1);
						}
					}

					if (MixerSubmixSend.SubmixSendStage == EMixerSourceSubmixSendStage::PreDistanceAttenuation)
					{
						SourceInfo.bHasPreDistanceAttenuationSend = true;
					}

					SubmixPtr->AddOrSetSourceById(SourceId, MixerSubmixSend.SendLevel, MixerSubmixSend.SubmixSendStage);
					
					if (SubmixPtr->IsSoundfieldSubmix())
					{
						SoundfieldSubmixSends.Add(SubmixPtr);
					}
				}
			}

			// Initialize the submix output source for this source id
			FMixerSourceSubmixOutputBuffer& SourceSubmixOutputBuffer = SourceSubmixOutputBuffers[SourceId];

			FMixerSourceSubmixOutputBufferSettings SourceSubmixOutputResetSettings;
			SourceSubmixOutputResetSettings.NumOutputChannels = MixerDevice->GetDeviceOutputChannels();
			SourceSubmixOutputResetSettings.NumSourceChannels = SourceInputChannels;
			SourceSubmixOutputResetSettings.SoundfieldSubmixSends = SoundfieldSubmixSends;
			SourceSubmixOutputResetSettings.bIs3D = SourceInfo.bIs3D;
			SourceSubmixOutputResetSettings.bIsSoundfield = SourceInfo.bIsSoundfield;

			SourceSubmixOutputBuffer.Reset(SourceSubmixOutputResetSettings);

#if AUDIO_MIXER_ENABLE_DEBUG_MODE
			AUDIO_MIXER_CHECK(!SourceInfo.bIsDebugMode);
			SourceInfo.bIsDebugMode = InitParams.bIsDebugMode;
			SourceInfo.DebugName = InitParams.DebugName;
#endif 

			if (UseRenderScheduler())
			{
				GetRenderScheduler().AddStep(FAudioRenderStepId::FromWaveInstancePlayOrder(SourceInfo.PlayOrder), &SourceInfo);
			}

			AUDIO_MIXER_DEBUG_LOG(SourceId, TEXT("Is initializing"));
		});
	}

	void FMixerSourceManager::ReleaseSourceId(const int32 SourceId)
	{
		AUDIO_MIXER_CHECK(GameThreadInfo.bIsBusy[SourceId]);
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		AUDIO_MIXER_CHECK(NumActiveSources > 0);
		--NumActiveSources;

		GameThreadInfo.bIsBusy[SourceId] = false;

#if AUDIO_MIXER_ENABLE_DEBUG_MODE
		GameThreadInfo.bIsDebugMode[SourceId] = false;
#endif

#if ENABLE_AUDIO_DEBUG
		GameThreadInfo.CPUCoreUtilization[SourceId] = 0.0f;
#endif 

		GameThreadInfo.RelativeRenderCost[SourceId] = 1.0f;
		GameThreadInfo.FreeSourceIndices.Push(SourceId);

		AUDIO_MIXER_CHECK(GameThreadInfo.FreeSourceIndices.Contains(SourceId));

		AudioMixerThreadCommand([this, SourceId]()
		{
			AUDIO_MIXER_CHECK_AUDIO_PLAT_THREAD(MixerDevice);

			ReleaseSource(SourceId);
		});
	}

	void FMixerSourceManager::StartAudioBus(FAudioBusKey InAudioBusKey, int32 InNumChannels, bool bInIsAutomatic)
	{
		StartAudioBus(InAudioBusKey, FString(), InNumChannels, bInIsAutomatic);
	}

	void FMixerSourceManager::StartAudioBus(FAudioBusKey InAudioBusKey, const FString& InAudioBusName, int32 InNumChannels, bool bInIsAutomatic)
	{
		if (AudioBusKeys_AudioThread.Contains(InAudioBusKey))
		{
			return;
		}

		AudioBusKeys_AudioThread.Add(InAudioBusKey);

#if UE_AUDIO_PROFILERTRACE_ENABLED
		TraceAudioBusStartMessage(MixerDevice->DeviceID, InAudioBusKey.ObjectId, InAudioBusName, InNumChannels);
#endif

		AudioMixerThreadCommand([this, InAudioBusKey, InNumChannels, bInIsAutomatic, InAudioBusName]()
		{
			// If this audio bus id already exists, set it to not be automatic and return it
			bool bAlreadyExisted = false;
			TSharedRef<FMixerAudioBus> AudioBusPtr = FindOrAddAudioBus(InAudioBusKey, InNumChannels, bInIsAutomatic, &bAlreadyExisted);
			if (bAlreadyExisted)
			{
				// If this audio bus already existed, make sure the num channels lines up
				ensure(AudioBusPtr->GetNumChannels() == InNumChannels);
				AudioBusPtr->SetAutomatic(bInIsAutomatic);
			}

#if UE_AUDIO_PROFILERTRACE_ENABLED
			if (AudioBusPtr->BusName.IsEmpty())
			{
				AudioBusPtr->BusName = InAudioBusName;
			}
#endif // UE_AUDIO_PROFILERTRACE_ENABLED
		}, AUDIO_MIXER_THREAD_COMMAND_STRING("StartAudioBus()"));
	}

	void FMixerSourceManager::StopAudioBus(FAudioBusKey InAudioBusKey)
	{
		if (!AudioBusKeys_AudioThread.Contains(InAudioBusKey))
		{
			return;
		}

		AudioBusKeys_AudioThread.Remove(InAudioBusKey);

#if UE_AUDIO_PROFILERTRACE_ENABLED
		TraceAudioBusStopMessage(MixerDevice->DeviceID, InAudioBusKey.ObjectId);
#endif

		AudioMixerThreadCommand([this, InAudioBusKey]()
		{
			TSharedPtr<FMixerAudioBus>* AudioBusPtr = AudioBuses.Find(InAudioBusKey);
			if (AudioBusPtr)
			{
				if (!(*AudioBusPtr)->IsAutomatic())
				{
#if UE_AUDIO_PROFILERTRACE_ENABLED
					StopAutoAudioBusEnvelopeFollower(InAudioBusKey.ObjectId, **AudioBusPtr);
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

					// Immediately stop all sources which were source buses
					for (FSourceInfo& SourceInfo : SourceInfos)
					{
						if (SourceInfo.AudioBusId == InAudioBusKey.ObjectId)
						{
							SourceInfo.bIsPlaying = false;
							SourceInfo.bIsPaused = false;
							SourceInfo.bIsActive = false;
							SourceInfo.bIsStopping = false;
						}
					}
					AudioBuses.Remove(InAudioBusKey);
				}
			}
		}, AUDIO_MIXER_THREAD_COMMAND_STRING("StopAudioBus()"));
	}

	bool FMixerSourceManager::IsAudioBusActive(FAudioBusKey InAudioBusKey) const
	{
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);
		return AudioBusKeys_AudioThread.Contains(InAudioBusKey);
	}

	int32 FMixerSourceManager::GetAudioBusNumChannels(FAudioBusKey InAudioBusKey) const
	{
		AUDIO_MIXER_CHECK_AUDIO_PLAT_THREAD(MixerDevice);
		TSharedPtr<FMixerAudioBus> AudioBusPtr = AudioBuses.FindRef(InAudioBusKey);
		if (AudioBusPtr.IsValid())
		{
			return AudioBusPtr->GetNumChannels();
		}

		return 0;
	}

	void FMixerSourceManager::AddPatchOutputForAudioBus(FAudioBusKey InAudioBusKey, const FPatchOutputStrongPtr& InPatchOutputStrongPtr)
	{
		AUDIO_MIXER_CHECK_AUDIO_PLAT_THREAD(MixerDevice);
		if (MixerDevice->IsAudioRenderingThread())
		{
			TSharedPtr<FMixerAudioBus> AudioBusPtr = AudioBuses.FindRef(InAudioBusKey);
			if (AudioBusPtr.IsValid())
			{
				AudioBusPtr->AddNewPatchOutput(InPatchOutputStrongPtr);
			}
		}
		else
		{
			// Queue up the command via MPSC command queue
			AudioMixerThreadMPSCCommand([this, InAudioBusKey, InPatchOutputStrongPtr]()
			{
				AddPatchOutputForAudioBus(InAudioBusKey, InPatchOutputStrongPtr);
			}, AUDIO_MIXER_THREAD_COMMAND_STRING("AddPatchOutputForAudioBus()"));
		}
	}

	void FMixerSourceManager::AddPatchOutputForAudioBus_AudioThread(FAudioBusKey InAudioBusKey, const FPatchOutputStrongPtr& InPatchOutputStrongPtr)
	{
		AudioMixerThreadCommand([this, InAudioBusKey, NewPatchPtr = InPatchOutputStrongPtr]() mutable
		{
			AddPatchOutputForAudioBus(InAudioBusKey, NewPatchPtr);
		}, AUDIO_MIXER_THREAD_COMMAND_STRING("AddPatchOutputForAudioBus_AudioThread()"));
	}

	void FMixerSourceManager::AddPatchInputForAudioBus(FAudioBusKey InAudioBusKey, const FPatchInput& InPatchInput)
	{
		AUDIO_MIXER_CHECK_AUDIO_PLAT_THREAD(MixerDevice);
		if (MixerDevice->IsAudioRenderingThread())
		{
			TSharedPtr<FMixerAudioBus> AudioBusPtr = AudioBuses.FindRef(InAudioBusKey);
			if (AudioBusPtr.IsValid())
			{
				AudioBusPtr->AddNewPatchInput(InPatchInput);
			}
		}
		else
		{
			// Queue up the command via MPSC command queue
			AudioMixerThreadMPSCCommand([this, InAudioBusKey, InPatchInput]()
			{
				AddPatchInputForAudioBus(InAudioBusKey, InPatchInput);
			}, AUDIO_MIXER_THREAD_COMMAND_STRING("AddPatchInputForAudioBus()"));
		}
	}

	void FMixerSourceManager::AddPatchInputForAudioBus_AudioThread(FAudioBusKey InAudioBusKey, const FPatchInput& InPatchInput)
	{
		AudioMixerThreadCommand([this, InAudioBusKey, InPatchInput]()
		{
			AddPatchInputForAudioBus(InAudioBusKey, InPatchInput);
		}, AUDIO_MIXER_THREAD_COMMAND_STRING("AddPatchInputForAudioBus_AudioThread()"));
	}

	void FMixerSourceManager::Play(const int32 SourceId)
	{
		AUDIO_MIXER_CHECK(SourceId < NumLegacySources);
		AUDIO_MIXER_CHECK(GameThreadInfo.bIsBusy[SourceId]);
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		AudioMixerThreadCommand([this, SourceId]()
		{
			AUDIO_MIXER_CHECK_AUDIO_PLAT_THREAD(MixerDevice);

			FSourceInfo& SourceInfo = SourceInfos[SourceId];
			
			SourceInfo.bIsPlaying = true;
			SourceInfo.bIsPaused = false;
			SourceInfo.bIsActive = true;

			UE_CLOGF(Audio::MatchesLogFilter(*SourceInfo.GetDebugName()), LogAudioTiming, Verbose,
				"FMixerSourceManager::Play (render thread), Wave=%ls, SourceId=%d", *SourceInfo.GetDebugName(), SourceId);

			AUDIO_MIXER_DEBUG_LOG(SourceId, TEXT("Is playing"));
		}, AUDIO_MIXER_THREAD_COMMAND_STRING("Play()"));
	}

	void FMixerSourceManager::CancelQuantizedSound(const int32 SourceId)
	{
		if (!MixerDevice)
		{
			return;
		}

		// If we are in the audio rendering thread, this is being called either before
		// or after source generation, so it is safe (and preffered) to call StopInternal()
		// synchronously. 
		if (MixerDevice->IsAudioRenderingThread())
		{
			StopInternal(SourceId);

			// Verify we have a reasonable Source
			AUDIO_MIXER_CHECK(SourceId < NumLegacySources);
			SetSourceDone(SourceId);
		}
	}

	void FMixerSourceManager::Stop(const int32 SourceId)
	{
		if (!MixerDevice)
		{
			return;
		}

		AUDIO_MIXER_CHECK(SourceId < NumLegacySources);

		//Assert that we are being called from the GameThread and the
		//source isn't busy.  Then call StopInternal() in a thread command
		AUDIO_MIXER_CHECK(GameThreadInfo.bIsBusy[SourceId]);
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		AudioMixerThreadCommand([this, SourceId]()
		{
			StopInternal(SourceId);
		}, AUDIO_MIXER_THREAD_COMMAND_STRING("Stop()"));
	}

	void FMixerSourceManager::StopInternal(const int32 SourceId)
	{
		AUDIO_MIXER_CHECK_AUDIO_PLAT_THREAD(MixerDevice);

		FSourceInfo& SourceInfo = SourceInfos[SourceId];

		SourceInfo.bIsPlaying = false;
		SourceInfo.bIsPaused = false;
		SourceInfo.bIsActive = false;
		SourceInfo.bIsStopping = false;

		if (SourceInfo.bIsPausedForQuantization)
		{
			UE_LOGF(LogAudioMixer, Display, "StopInternal() cancelling command [%ls]", *SourceInfo.QuantizedCommandHandle.CommandPtr->GetCommandName().ToString());
			SourceInfo.QuantizedCommandHandle.Cancel();
			SourceInfo.bIsPausedForQuantization = false;
		}

		AUDIO_MIXER_DEBUG_LOG(SourceId, TEXT("Is immediately stopping"));
	}

	void FMixerSourceManager::StopFade(const int32 SourceId, const int32 NumFrames)
	{
		AUDIO_MIXER_CHECK(SourceId < NumLegacySources);
		AUDIO_MIXER_CHECK(GameThreadInfo.bIsBusy[SourceId]);
		AUDIO_MIXER_CHECK(NumFrames > 0);
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		AudioMixerThreadCommand([this, SourceId, NumFrames]()
		{
			AUDIO_MIXER_CHECK_AUDIO_PLAT_THREAD(MixerDevice);

			FSourceInfo& SourceInfo = SourceInfos[SourceId];

			SourceInfo.bIsPaused = false;
			SourceInfo.bIsStopping = true;

			if (SourceInfo.bIsPausedForQuantization)
			{
				// no need to fade, we haven't actually started playing
				StopInternal(SourceId);
				return;
			}
			
			// Only allow multiple of 4 fade frames and positive
			int32 NumFadeFrames = AlignArbitrary(NumFrames, 4);
			if (NumFadeFrames <= 0)
			{
				// Stop immediately if we've been given no fade frames
				SourceInfo.bIsPlaying = false;
				SourceInfo.bIsPaused = false;
				SourceInfo.bIsActive = false;
				SourceInfo.bIsStopping = false;
			}
			else
			{
				// compute the fade slope
				SourceInfo.VolumeFadeStart = SourceInfo.VolumeSourceStart;
				SourceInfo.VolumeFadeNumFrames = NumFadeFrames;
				SourceInfo.VolumeFadeSlope = -SourceInfo.VolumeSourceStart / SourceInfo.VolumeFadeNumFrames;
				SourceInfo.VolumeFadeFramePosition = 0;
			}

			AUDIO_MIXER_DEBUG_LOG(SourceId, TEXT("Is stopping with fade"));
		}, AUDIO_MIXER_THREAD_COMMAND_STRING("StopFade()"));
	}
	
	void FMixerSourceManager::Pause(const int32 SourceId)
	{
		AUDIO_MIXER_CHECK(SourceId < NumLegacySources);
		AUDIO_MIXER_CHECK(GameThreadInfo.bIsBusy[SourceId]);
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		AudioMixerThreadCommand([this, SourceId]()
		{
			AUDIO_MIXER_CHECK_AUDIO_PLAT_THREAD(MixerDevice);

			FSourceInfo& SourceInfo = SourceInfos[SourceId];

			SourceInfo.bIsPaused = true;
			SourceInfo.bIsActive = false;
		}, AUDIO_MIXER_THREAD_COMMAND_STRING("Pause()"));
	}

	void FMixerSourceManager::SetPitch(const int32 SourceId, const float Pitch)
	{
		AUDIO_MIXER_CHECK(SourceId < NumLegacySources);
		AUDIO_MIXER_CHECK(GameThreadInfo.bIsBusy[SourceId]);

		AudioMixerThreadCommand([this, SourceId, Pitch]()
		{
			AUDIO_MIXER_CHECK_AUDIO_PLAT_THREAD(MixerDevice);
			check(NumOutputFrames > 0);

			SourceInfos[SourceId].PitchSourceParam.SetValue(Pitch, NumOutputFrames);
		}, AUDIO_MIXER_THREAD_COMMAND_STRING("SetPitch()"));
	}

	void FMixerSourceManager::SetVolume(const int32 SourceId, const float Volume)
	{
		AUDIO_MIXER_CHECK(SourceId < NumLegacySources);
		AUDIO_MIXER_CHECK(GameThreadInfo.bIsBusy[SourceId]);
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		AudioMixerThreadCommand([this, SourceId, Volume]()
		{
			AUDIO_MIXER_CHECK_AUDIO_PLAT_THREAD(MixerDevice);
			check(NumOutputFrames > 0);

			FSourceInfo& SourceInfo = SourceInfos[SourceId];

			// Only set the volume if we're not stopping. Stopping sources are setting their volume to 0.0.
			if (!SourceInfo.bIsStopping)
			{
				// If we've not yet set a volume, we need to immediately set the start and destination to be the same value (to avoid an initial fade in)
				if (SourceInfos[SourceId].VolumeSourceDestination < 0.0f)
				{
					SourceInfos[SourceId].VolumeSourceStart = Volume;
				}

				SourceInfos[SourceId].VolumeSourceDestination = Volume;
			}
		}, AUDIO_MIXER_THREAD_COMMAND_STRING("SetVolume()"));
	}

	void FMixerSourceManager::SetDistanceAttenuation(const int32 SourceId, const float DistanceAttenuation)
	{
		AUDIO_MIXER_CHECK(SourceId < NumLegacySources);
		AUDIO_MIXER_CHECK(GameThreadInfo.bIsBusy[SourceId]);
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		AudioMixerThreadCommand([this, SourceId, DistanceAttenuation]()
		{
			AUDIO_MIXER_CHECK_AUDIO_PLAT_THREAD(MixerDevice);
			check(NumOutputFrames > 0);

			// If we've not yet set a distance attenuation, we need to immediately set the start and destination to be the same value (to avoid an initial fade in)
			if (SourceInfos[SourceId].DistanceAttenuationSourceDestination < 0.0f)
			{
				SourceInfos[SourceId].DistanceAttenuationSourceStart = DistanceAttenuation;
			}

			SourceInfos[SourceId].DistanceAttenuationSourceDestination = DistanceAttenuation;
		}, AUDIO_MIXER_THREAD_COMMAND_STRING("SetDistanceAttenuation()"));
	}

	void FMixerSourceManager::SetSpatializationParams(const int32 SourceId, const FSpatializationParams& InParams)
	{
		AUDIO_MIXER_CHECK(SourceId < NumLegacySources);
		AUDIO_MIXER_CHECK(GameThreadInfo.bIsBusy[SourceId]);
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		AudioMixerThreadCommand([this, SourceId, InParams]()
		{
			AUDIO_MIXER_CHECK_AUDIO_PLAT_THREAD(MixerDevice);

			SourceInfos[SourceId].SpatParams = InParams;
		}, AUDIO_MIXER_THREAD_COMMAND_STRING("SetSpatializationParams()"));
	}

	void FMixerSourceManager::SetChannelMap(const int32 SourceId, const uint32 NumInputChannels, const Audio::FAlignedFloatBuffer& ChannelMap, const bool bInIs3D, const bool bInIsCenterChannelOnly)
	{
		AUDIO_MIXER_CHECK(SourceId < NumLegacySources);
		AUDIO_MIXER_CHECK(GameThreadInfo.bIsBusy[SourceId]);
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		AudioMixerThreadCommand([this, SourceId, NumInputChannels, ChannelMap, bInIs3D, bInIsCenterChannelOnly]()
		{
			AUDIO_MIXER_CHECK_AUDIO_PLAT_THREAD(MixerDevice);

			check(NumOutputFrames > 0);

			FSourceInfo& SourceInfo = SourceInfos[SourceId];
			FMixerSourceSubmixOutputBuffer& SourceSubmixOutput = SourceSubmixOutputBuffers[SourceId];

			if (SourceSubmixOutput.GetNumSourceChannels() != NumInputChannels && !SourceInfo.bUseHRTFSpatializer)
			{
				// This means that this source has been reinitialized as a different source while this command was in flight,
				// In which case it is of no use to us. Exit.
				return;
			}

			// Set whether or not this is a 3d channel map and if its center channel only. Used for reseting channel maps on device change.
			SourceInfo.bIs3D = bInIs3D;
			SourceInfo.bIsCenterChannelOnly = bInIsCenterChannelOnly;

			bool bNeedsSpeakerMap = SourceSubmixOutput.SetChannelMap(ChannelMap, bInIsCenterChannelOnly);
			GameThreadInfo.bNeedsSpeakerMap[SourceId] = bNeedsSpeakerMap;
		}, AUDIO_MIXER_THREAD_COMMAND_STRING("SetChannelMap()"));
	}

	void FMixerSourceManager::SetLPFFrequency(const int32 SourceId, const float InLPFFrequency)
	{
		AUDIO_MIXER_CHECK(SourceId < NumLegacySources);
		AUDIO_MIXER_CHECK(GameThreadInfo.bIsBusy[SourceId]);
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		AudioMixerThreadCommand([this, SourceId, InLPFFrequency]()
		{
			AUDIO_MIXER_CHECK_AUDIO_PLAT_THREAD(MixerDevice);

			FSourceInfo& SourceInfo = SourceInfos[SourceId];

			// LowPassFreq is cached off as the version set by this setter as well as that internal to the LPF.
			// There is a second cutoff frequency cached in SourceInfo.LowpassModulation updated per buffer callback.
			// On callback, the client version may be overridden with the modulation LPF value depending on which is more aggressive.  
			SourceInfo.LowPassFreq = InLPFFrequency;
			SourceInfo.LowPassFilter.StartFrequencyInterpolation(InLPFFrequency, NumOutputFrames);
		}, AUDIO_MIXER_THREAD_COMMAND_STRING("SetLPFFrequency()"));
	}

	void FMixerSourceManager::SetHPFFrequency(const int32 SourceId, const float InHPFFrequency)
	{
		AUDIO_MIXER_CHECK(SourceId < NumLegacySources);
		AUDIO_MIXER_CHECK(GameThreadInfo.bIsBusy[SourceId]);
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		AudioMixerThreadCommand([this, SourceId, InHPFFrequency]()
		{
			AUDIO_MIXER_CHECK_AUDIO_PLAT_THREAD(MixerDevice);
			FSourceInfo& SourceInfo = SourceInfos[SourceId];

			// HighPassFreq is cached off as the version set by this setter as well as that internal to the HPF.
			// There is a second cutoff frequency cached in SourceInfo.HighpassModulation updated per buffer callback.
			// On callback, the client version may be overridden with the modulation HPF value depending on which is more aggressive.  
			SourceInfo.HighPassFreq = InHPFFrequency;
			SourceInfo.HighPassFilter.StartFrequencyInterpolation(InHPFFrequency, NumOutputFrames);
		}, AUDIO_MIXER_THREAD_COMMAND_STRING("SetHPFFrequency()"));
	}

	void FMixerSourceManager::SetModLPFFrequency(const int32 SourceId, const float InLPFFrequency)
	{
		AUDIO_MIXER_CHECK(SourceId < NumLegacySources);
		AUDIO_MIXER_CHECK(GameThreadInfo.bIsBusy[SourceId]);
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		AudioMixerThreadCommand([this, SourceId, InLPFFrequency]()
		{
			AUDIO_MIXER_CHECK_AUDIO_PLAT_THREAD(MixerDevice);

			FSourceInfo& SourceInfo = SourceInfos[SourceId];
			SourceInfo.LowpassModulationBase = InLPFFrequency;
			SourceInfo.bModFiltersUpdated = true;
		}, AUDIO_MIXER_THREAD_COMMAND_STRING("SetModLPFFrequency()"));
	}

	void FMixerSourceManager::SetModHPFFrequency(const int32 SourceId, const float InHPFFrequency)
	{
		AUDIO_MIXER_CHECK(SourceId < NumLegacySources);
		AUDIO_MIXER_CHECK(GameThreadInfo.bIsBusy[SourceId]);
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		AudioMixerThreadCommand([this, SourceId, InHPFFrequency]()
		{
			AUDIO_MIXER_CHECK_AUDIO_PLAT_THREAD(MixerDevice);

			FSourceInfo& SourceInfo = SourceInfos[SourceId];
			SourceInfo.HighpassModulationBase = InHPFFrequency;
			SourceInfo.bModFiltersUpdated = true;
		}, AUDIO_MIXER_THREAD_COMMAND_STRING("SetModHPFFrequency()"));
	}

	void FMixerSourceManager::SetModulationRouting(const int32 SourceId, FSoundModulationDefaultSettings& ModulationSettings)
	{
		FSourceInfo& SourceInfo = SourceInfos[SourceId];

		FModulationDestination VolumeMod;
		VolumeMod.Init(MixerDevice->DeviceID, FName("Volume"), false /* bInIsBuffered */, true /* bInValueLinear */);
		VolumeMod.SetModulators(ModulationSettings.VolumeModulationDestination.Modulators);

		FModulationDestination PitchMod;
		PitchMod.Init(MixerDevice->DeviceID, FName("Pitch"), false /* bInIsBuffered */);
		PitchMod.SetModulators(ModulationSettings.PitchModulationDestination.Modulators);

		FModulationDestination HighpassMod;
		HighpassMod.Init(MixerDevice->DeviceID, FName("HPFCutoffFrequency"), false /* bInIsBuffered */);
		HighpassMod.SetModulators(ModulationSettings.HighpassModulationDestination.Modulators);

		FModulationDestination LowpassMod;
		LowpassMod.Init(MixerDevice->DeviceID, FName("LPFCutoffFrequency"), false /* bInIsBuffered */);
		LowpassMod.SetModulators(ModulationSettings.LowpassModulationDestination.Modulators);


		AudioMixerThreadCommand([
			this,
				SourceId,
				VolumeModulation = MoveTemp(VolumeMod),
				HighpassModulation = MoveTemp(HighpassMod),
				LowpassModulation = MoveTemp(LowpassMod),
				PitchModulation = MoveTemp(PitchMod)
		]() mutable
			{
				FSourceInfo& SourceInfo = SourceInfos[SourceId];

				SourceInfo.VolumeModulation = MoveTemp(VolumeModulation);
				SourceInfo.PitchModulation = MoveTemp(PitchModulation);
				SourceInfo.LowpassModulation = MoveTemp(LowpassModulation);
				SourceInfo.HighpassModulation = MoveTemp(HighpassModulation);
			}, AUDIO_MIXER_THREAD_COMMAND_STRING("SetModulationRouting()")
		);
	}

	void FMixerSourceManager::SetSourceBufferListener(const int32 SourceId, FSharedISourceBufferListenerPtr& InSourceBufferListener, bool InShouldSourceBufferListenerZeroBuffer)
	{
		AUDIO_MIXER_CHECK(SourceId < NumLegacySources);
		AUDIO_MIXER_CHECK(GameThreadInfo.bIsBusy[SourceId]);
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		AudioMixerThreadCommand([this, SourceId, InSourceBufferListener, InShouldSourceBufferListenerZeroBuffer]()
		{
			AUDIO_MIXER_CHECK_AUDIO_PLAT_THREAD(MixerDevice);

			FSourceInfo& SourceInfo = SourceInfos[SourceId];
			SourceInfo.SourceBufferListener = InSourceBufferListener;
			SourceInfo.bShouldSourceBufferListenerZeroBuffer = InShouldSourceBufferListenerZeroBuffer;
		}, AUDIO_MIXER_THREAD_COMMAND_STRING("SetSourceBufferListener()"));
	}

	void FMixerSourceManager::SetModVolume(const int32 SourceId, const float InModVolume)
	{
		AUDIO_MIXER_CHECK(SourceId < NumLegacySources);
		AUDIO_MIXER_CHECK(GameThreadInfo.bIsBusy[SourceId]);
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		AudioMixerThreadCommand([this, SourceId, InModVolume]()
		{
			AUDIO_MIXER_CHECK_AUDIO_PLAT_THREAD(MixerDevice);

			FSourceInfo& SourceInfo = SourceInfos[SourceId];
			SourceInfo.VolumeModulationBase = InModVolume;
		}, AUDIO_MIXER_THREAD_COMMAND_STRING("SetModVolume()"));
	}

	void FMixerSourceManager::SetModPitch(const int32 SourceId, const float InModPitch)
	{
		AUDIO_MIXER_CHECK(SourceId < NumLegacySources);
		AUDIO_MIXER_CHECK(GameThreadInfo.bIsBusy[SourceId]);
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		AudioMixerThreadCommand([this, SourceId, InModPitch]()
		{
			AUDIO_MIXER_CHECK_AUDIO_PLAT_THREAD(MixerDevice);

			FSourceInfo& SourceInfo = SourceInfos[SourceId];
			SourceInfo.PitchModulationBase = InModPitch;
		}, AUDIO_MIXER_THREAD_COMMAND_STRING("SetModPitch()"));
	}

	void FMixerSourceManager::SetSubmixSendInfo(const int32 SourceId, const FMixerSourceSubmixSend& InSubmixSend)
	{
		AUDIO_MIXER_CHECK(SourceId < NumLegacySources);
		AUDIO_MIXER_CHECK(GameThreadInfo.bIsBusy[SourceId]);
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		AudioMixerThreadCommand([this, SourceId, InSubmixSend]()
		{
			FSourceInfo& SourceInfo = SourceInfos[SourceId];

			FMixerSubmixPtr InSubmixPtr = InSubmixSend.Submix.Pin();
			if (InSubmixPtr.IsValid())
			{
				// Determine whether submix send is new and whether any sends have 
				// a pre-distance-attenuation send.
				bool bIsNew = true;
				SourceInfo.bHasPreDistanceAttenuationSend = InSubmixSend.SubmixSendStage == EMixerSourceSubmixSendStage::PreDistanceAttenuation;

				for (FMixerSourceSubmixSend& SubmixSend : SourceInfo.SubmixSends)
				{
					FMixerSubmixPtr SubmixPtr = SubmixSend.Submix.Pin();

					if (SubmixPtr.IsValid())
					{
						if (SubmixPtr->GetId() == InSubmixPtr->GetId())
						{
							// Update existing submix send if it already exists
							SubmixSend.SendLevel = InSubmixSend.SendLevel;
							SubmixSend.SubmixSendStage = InSubmixSend.SubmixSendStage;
							const bool bWasFilteringEnabled = SubmixSend.Filter.IsFiltering();
							SubmixSend.Filter.bEnableLPF = InSubmixSend.Filter.bEnableLPF;
							SubmixSend.Filter.bEnableHPF = InSubmixSend.Filter.bEnableHPF;
							if (SubmixSend.Filter.IsFiltering())
							{
								if (!bWasFilteringEnabled)
								{
									const int32 NumDeviceChannels = FMath::Max(MixerDevice->GetNumDeviceChannels(), 1);
									SubmixSend.Lpf.Init(MixerDevice->SampleRate, NumDeviceChannels);
									SubmixSend.Hpf.Init(MixerDevice->SampleRate, NumDeviceChannels);
								}
								
								if (SubmixSend.Filter.bEnableLPF)
								{
									SubmixSend.Filter.LpfCutoff = InSubmixSend.Filter.LpfCutoff;
									SubmixSend.Lpf.StartFrequencyInterpolation(SubmixSend.Filter.LpfCutoff, 1);
								}
								
								if (SubmixSend.Filter.bEnableHPF)
								{
									SubmixSend.Filter.HpfCutoff = InSubmixSend.Filter.HpfCutoff;
									SubmixSend.Hpf.StartFrequencyInterpolation(SubmixSend.Filter.HpfCutoff, 1);
								}
							}
							
							bIsNew = false;
							if (SourceInfo.bHasPreDistanceAttenuationSend)
							{
								break;
							}
						}

						if (SubmixSend.SubmixSendStage == EMixerSourceSubmixSendStage::PreDistanceAttenuation)
						{
							SourceInfo.bHasPreDistanceAttenuationSend = true;
						}
					}
				}

				if (bIsNew)
				{
					FMixerSourceSubmixSend NewSend = InSubmixSend;
					NewSend.SubmixId = InSubmixPtr->GetId();
					if (NewSend.Filter.IsFiltering())
					{
						const int32 NumDeviceChannels = FMath::Max(MixerDevice->GetNumDeviceChannels(), 1);
						NewSend.Lpf.Init(MixerDevice->SampleRate, NumDeviceChannels);
						NewSend.Hpf.Init(MixerDevice->SampleRate, NumDeviceChannels);
						// Snap coefficients to the initial cutoffs immediately. bIsFirstFrequencyChange (reset by
						// Init) forces CurrInterpLength=0 in StartFrequencyInterpolation, so coefficients land on
						// the target without interpolation rather than ramping from the default.
						if (NewSend.Filter.bEnableLPF)
						{
							NewSend.Lpf.StartFrequencyInterpolation(NewSend.Filter.LpfCutoff, 1);
						}
						
						if (NewSend.Filter.bEnableHPF)
						{
							NewSend.Hpf.StartFrequencyInterpolation(NewSend.Filter.HpfCutoff, 1);
						}
					}
					
					SourceInfo.SubmixSends.Add(MoveTemp(NewSend));
				}
				
				// If we don't have a pre-distance attenuation send, lets zero out the buffer so the output buffer stops doing math with it.
				if (!SourceInfo.bHasPreDistanceAttenuationSend)
				{
					SourceSubmixOutputBuffers[SourceId].SetPreAttenuationSourceBuffer(nullptr);
				}

				InSubmixPtr->AddOrSetSourceById(SourceId, InSubmixSend.SendLevel, InSubmixSend.SubmixSendStage);
			}
		}, AUDIO_MIXER_THREAD_COMMAND_STRING("SetSubmixSendInfo()"));
	}

	void FMixerSourceManager::ClearSubmixSendInfo(const int32 SourceId, const FMixerSourceSubmixSend& InSubmixSend)
	{
		AUDIO_MIXER_CHECK(SourceId < NumLegacySources);
		AUDIO_MIXER_CHECK(GameThreadInfo.bIsBusy[SourceId]);
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		AudioMixerThreadCommand([this, SourceId, InSubmixSend]()
		{
			FSourceInfo& SourceInfo = SourceInfos[SourceId];

			FMixerSubmixPtr InSubmixPtr = InSubmixSend.Submix.Pin();
			if (InSubmixPtr.IsValid())
			{
				for (int32 i = SourceInfo.SubmixSends.Num() - 1; i >= 0; --i)
				{
					if (SourceInfo.SubmixSends[i].Submix == InSubmixSend.Submix)
					{
						SourceInfo.SubmixSends.RemoveAtSwap(i, EAllowShrinking::No);
					}
				}

				// Update the has predist attenuation send state
				SourceInfo.bHasPreDistanceAttenuationSend = false;
				for (FMixerSourceSubmixSend& SubmixSend : SourceInfo.SubmixSends)
				{
					FMixerSubmixPtr SubmixPtr = SubmixSend.Submix.Pin();
					if (SubmixPtr.IsValid())
					{
						if (SubmixSend.SubmixSendStage == EMixerSourceSubmixSendStage::PreDistanceAttenuation)
						{
							SourceInfo.bHasPreDistanceAttenuationSend = true;
							break;
						}
					}
				}

				// If we don't have a pre-distance attenuation send, lets zero out the buffer so the output buffer stops doing math with it.
				if (!SourceInfo.bHasPreDistanceAttenuationSend)
				{
					SourceSubmixOutputBuffers[SourceId].SetPreAttenuationSourceBuffer(nullptr);
				}

				// Now remove the source voice from the submix send list
				InSubmixPtr->RemoveSourceById(SourceId);
			}
		}, AUDIO_MIXER_THREAD_COMMAND_STRING("ClearSubmixSendInfo()"));
	}

	void FMixerSourceManager::SetBusSendInfo(const int32 SourceId, EBusSendType InAudioBusSendType, uint32 AudioBusId, float BusSendLevel, const FString& InBusName, const FSendFilterSettings& InFilter)
	{
		AUDIO_MIXER_CHECK(SourceId < NumLegacySources);
		AUDIO_MIXER_CHECK(GameThreadInfo.bIsBusy[SourceId]);
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		AudioMixerThreadCommand([this, SourceId, InAudioBusSendType, AudioBusId, BusSendLevel, InBusName, InFilter]()
		{
			FSourceInfo& SourceInfo = SourceInfos[SourceId];

			// Create mapping of source id to bus send level
			FAudioBusSend BusSend;
			BusSend.SourceId = SourceId;
			BusSend.WaveInstancePlayOrder = SourceInfo.PlayOrder;
			BusSend.SendLevel = BusSendLevel;

			// Retrieve the bus we want to send audio to. A newly created bus won't output audio, but
			// we register the send anyway in the event that this bus does play, we'll know to send this
			// source's audio to it.
			bool bAlreadyExisted = false;
			TSharedRef<FMixerAudioBus> AudioBusPtr = FindOrAddAudioBus(AudioBusId, SourceInfo.NumInputChannels, true, &bAlreadyExisted);

			BusSend.Filter = InFilter;
			if (InFilter.IsFiltering())
			{
				const int32 NumSourceChannels = FMath::Max(SourceInfo.NumInputChannels, 1);
				BusSend.Lpf.Init(MixerDevice->SampleRate, NumSourceChannels);
				BusSend.Hpf.Init(MixerDevice->SampleRate, NumSourceChannels);
			}

			AudioBusPtr->AddSend(InAudioBusSendType, BusSend);

#if UE_AUDIO_PROFILERTRACE_ENABLED
			if (!bAlreadyExisted)
			{
				AudioBusPtr->BusName = InBusName;
				TraceAudioBusStartMessage(MixerDevice->DeviceID, AudioBusId, InBusName, SourceInfo.NumInputChannels);
			}
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

			// Check to see if we need to create new bus data. If we are not playing a bus with this id, then we
			// need to create a slot for it such that when a bus does play, it'll start rendering audio from this source
			bool bExisted = false;
			for (uint32 BusId : SourceInfo.AudioBusSends[(int32)InAudioBusSendType])
			{
				if (BusId == AudioBusId)
				{
					bExisted = true;
					break;
				}
			}

			if (!bExisted)
			{
				SourceInfo.AudioBusSends[(int32)InAudioBusSendType].Add(AudioBusId);
			}
		}, AUDIO_MIXER_THREAD_COMMAND_STRING("SetBusSendInfo()"));
	}

	void FMixerSourceManager::SetListenerTransforms(const TArray<FTransform>& InListenerTransforms)
	{
		AudioMixerThreadCommand([this, InListenerTransforms]()
		{
			ListenerTransforms = InListenerTransforms;
		}, AUDIO_MIXER_THREAD_COMMAND_STRING("SetListenerTransforms()"));
	}

	const TArray<FTransform>* FMixerSourceManager::GetListenerTransforms() const
	{
		AUDIO_MIXER_CHECK_AUDIO_PLAT_THREAD(MixerDevice);
		return &ListenerTransforms;
	}

	int64 FMixerSourceManager::GetNumFramesPlayed(const int32 SourceId) const
	{
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);
		return SourceInfos[SourceId].NumFramesPlayed;
	}

	float FMixerSourceManager::GetEnvelopeValue(const int32 SourceId) const
	{
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);
		return SourceInfos[SourceId].SourceEnvelopeValue;
	}

	float FMixerSourceManager::GetVolumeModulationValue(const int32 SourceId) const
	{
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);
		AUDIO_MIXER_CHECK(SourceId < NumLegacySources); 
		return GameThreadInfo.ModulationVolume[SourceId];
	}

#if ENABLE_AUDIO_DEBUG
	double FMixerSourceManager::GetCPUCoreUtilization(const int32 SourceId) const
	{
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);
		AUDIO_MIXER_CHECK(SourceId < NumLegacySources);
		return GameThreadInfo.CPUCoreUtilization[SourceId];
	}
#endif // if ENABLE_AUDIO_DEBUG

	float FMixerSourceManager::GetRelativeRenderCost(const int32 SourceId) const
	{
		AUDIO_MIXER_CHECK(SourceId < NumLegacySources);
		return GameThreadInfo.RelativeRenderCost[SourceId];
	}

	bool FMixerSourceManager::IsUsingHRTFSpatializer(const int32 SourceId) const
	{
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);
		AUDIO_MIXER_CHECK(SourceId < NumLegacySources);
		return GameThreadInfo.bIsUsingHRTFSpatializer[SourceId];
	}

	bool FMixerSourceManager::NeedsSpeakerMap(const int32 SourceId) const
	{
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);
		AUDIO_MIXER_CHECK(SourceId < NumLegacySources);
		return GameThreadInfo.bNeedsSpeakerMap[SourceId];
	}

	void FMixerSourceManager::ComputeSourceBuffer(const bool bGenerateBuses, const int32 SourceId)
	{
		CSV_SCOPED_TIMING_STAT(Audio, SourceBuffers);
		SCOPE_CYCLE_COUNTER(STAT_AudioMixerSourceBuffers);

		FSourceInfo& SourceInfo = SourceInfos[SourceId];

		if (!SourceInfo.bIsBusy || !SourceInfo.bIsPlaying || SourceInfo.bIsPaused || SourceInfo.bIsPausedForQuantization)
		{
			return;
		}

		const bool bIsSourceBus = SourceInfo.AudioBusId != INDEX_NONE;
		if ((bGenerateBuses && !bIsSourceBus) || (!bGenerateBuses && bIsSourceBus))
		{
			return;
		}

		// Fill array with elements all at once to avoid sequential Add() operation overhead.
		const int32 NumSamples = NumOutputFrames * SourceInfo.NumInputChannels;

		UE_CLOGF(Audio::MatchesLogFilter(*SourceInfo.GetDebugName()), LogAudioTiming, Verbose,
			"ComputeSourceBuffer Name=%ls, NumOutputFrames=%d, FramesPlayed=%lld, CurrentFrameIndex=%d ",
			*SourceInfo.GetDebugName(), NumOutputFrames, SourceInfo.NumFramesPlayed, SourceInfo.CurrentFrameIndex);

		// Initialize both the pre-distance attenuation buffer and the source buffer
		SourceInfo.PreDistanceAttenuationBuffer.Reset();
		SourceInfo.PreDistanceAttenuationBuffer.AddZeroed(NumSamples);

		SourceInfo.SourceEffectScratchBuffer.Reset();
		SourceInfo.SourceEffectScratchBuffer.AddZeroed(NumSamples);

		SourceInfo.SourceBuffer.Reset();
		SourceInfo.SourceBuffer.AddZeroed(NumSamples);

		// If this source is still playing at this point but technically done, return after zeroing the buffers. We haven't yet been removed by the FMixerSource owner.
		// This should be rare but could happen due to thread timing since done-ness is queried on audio thread.
		if (SourceInfo.bIsDone)
		{
			return;
		}

		float* PreDistanceAttenBufferPtr = SourceInfo.PreDistanceAttenuationBuffer.GetData();

		// if this is a bus, we just want to copy the bus audio to this source's output audio
		// Note we need to copy this since bus instances may have different audio via dynamic source effects, etc.
		if (bIsSourceBus)
		{
			// Get the source's rendered and mixed audio bus data
			const TSharedPtr<FMixerAudioBus> AudioBusPtr = AudioBuses.FindRef(SourceInfo.AudioBusId);
			if (AudioBusPtr.IsValid())
			{
				int32 NumFramesPlayed = NumOutputFrames;
				if (SourceInfo.SourceBusDurationFrames != INDEX_NONE)
				{
					// If we're now finishing, only copy over the real data
					if ((SourceInfo.NumFramesPlayed + NumOutputFrames) >= SourceInfo.SourceBusDurationFrames)
					{
						NumFramesPlayed = SourceInfo.SourceBusDurationFrames - SourceInfo.NumFramesPlayed;
						SourceInfo.bIsLastBuffer = true;
					}
				}

				SourceInfo.NumFramesPlayed += NumFramesPlayed;

				// Retrieve the channel map of going from the audio bus channel count to the source channel count since they may not match
				int32 NumAudioBusChannels = AudioBusPtr->GetNumChannels();
				if (NumAudioBusChannels != SourceInfo.NumInputChannels)
				{
					Audio::FAlignedFloatBuffer ChannelMap;
					MixerDevice->Get2DChannelMap(SourceInfo.bIsVorbis, AudioBusPtr->GetNumChannels(), SourceInfo.NumInputChannels, SourceInfo.bIsCenterChannelOnly, ChannelMap);
					AudioBusPtr->CopyCurrentBuffer(ChannelMap, SourceInfo.NumInputChannels, SourceInfo.PreDistanceAttenuationBuffer, NumFramesPlayed);
				}
				else
				{
					AudioBusPtr->CopyCurrentBuffer(SourceInfo.NumInputChannels, SourceInfo.PreDistanceAttenuationBuffer, NumFramesPlayed);
				}
			}
		}
		else
		{
			// Modulate parameter target should modulation be active
			// Due to managing two separate pitch values that are updated at different rates
			// (game thread rate and copy set by SetPitch and buffer callback rate set by Modulation System),
			// the PitchSourceParam's target is marshaled before processing by mult'ing in the modulation pitch,
			// processing the buffer, and then resetting it back if modulation is active. 

			const bool bModActive = MixerDevice->IsModulationPluginEnabled() && MixerDevice->ModulationInterface.IsValid();
			if (bModActive)
			{
				SourceInfo.PitchModulation.ProcessControl(SourceInfo.PitchModulationBase);
			}

			const float TargetPitch = SourceInfo.PitchSourceParam.GetTarget();
			// Convert from semitones to frequency multiplier
			const float ModPitch = bModActive
				? Audio::GetFrequencyMultiplier(SourceInfo.PitchModulation.GetValue())
				: 1.0f;
			const float FinalPitch = FMath::Clamp(TargetPitch * ModPitch, MinModulationPitchRangeFreqCVar, MaxModulationPitchRangeFreqCVar);
			SourceInfo.PitchSourceParam.SetValue(FinalPitch, NumOutputFrames);

#if UE_AUDIO_PROFILERTRACE_ENABLED
			const bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AudioMixerChannel);
			if (bChannelEnabled)
			{
				UE_TRACE_LOG(Audio, SoundPitch, AudioMixerChannel)
					<< SoundPitch.DeviceId(MixerDevice->DeviceID)
					<< SoundPitch.Timestamp(FPlatformTime::Cycles64())
					<< SoundPitch.PlayOrder(SourceInfo.PlayOrder)
					<< SoundPitch.ActiveSoundPlayOrder(SourceInfo.ActiveSoundPlayOrder)
					<< SoundPitch.Pitch(FinalPitch);
			}
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

			const bool bIsGeneratorSource = SourceId >= NumLegacySources;
			if (bIsGeneratorSource)
			{
				SourceInfo.bHasStarted = true;

				const int32 GeneratorSourceIndex = SourceId - NumLegacySources;
				const ISoundGeneratorPtr& SourceGenerator = GeneratorSourceState[GeneratorSourceIndex].Generator;
				check(SourceGenerator.IsValid());
				
				TArrayView<float> WriteBuffer(PreDistanceAttenBufferPtr, NumSamples);
				while (WriteBuffer.Num() > 0)
				{
					const int32 SamplesGenerated = SourceGenerator->OnGenerateAudio(WriteBuffer.GetData(), WriteBuffer.Num());
					WriteBuffer = WriteBuffer.RightChop(SamplesGenerated);
					if (SourceGenerator->IsFinished())
					{
						SourceInfo.bIsLastBuffer = true;
						break;
					}
					else if (!ensureMsgf(SamplesGenerated > 0, TEXT("Source generator returned no samples but is not finished!")))
					{
						break;
					}
				}
				
				const int32 TotalSamplesGenerated = NumSamples - WriteBuffer.Num();
				SourceInfo.CurrentFrameIndex += TotalSamplesGenerated / SourceInfo.NumInputChannels;
				SourceInfo.NumFramesPlayed += TotalSamplesGenerated / SourceInfo.NumInputChannels;
			}
			else
			{
				const bool bIsFirstRun = !SourceInfo.bHasStarted;
				SourceInfo.bHasStarted = true;

				// Fill PreDistanceAttenuationBuffer with incoming source data
				if (ensure(SourceInfo.MixerSourceBuffer.IsValid() && SourceInfo.NumInputChannels > 0))
				{
					int32 FramesToWrite = NumOutputFrames;

					// Skip writing some frames in the first block if a delay is requested.
					if (SourceInfo.SubCallbackDelayLengthInFrames > 0 && !SourceInfo.bDelayLineSet)
					{
						if (ensure(SourceInfo.SubCallbackDelayLengthInFrames <= FramesToWrite))
						{
							FramesToWrite -= SourceInfo.SubCallbackDelayLengthInFrames;
						}
						SourceInfo.bDelayLineSet = true;
					}

					while (FramesToWrite > 0)
					{
						int32 CurrentChunkFrames = SourceInfo.GetCurrentAudioChunkNumFrames();
						if (SourceInfo.CurrentFrameIndex >= CurrentChunkFrames)
						{
							// Time to read another chunk

							// If this is our first PCM buffer, we don't need to do a callback to get more audio
							if (SourceInfo.CurrentPCMBuffer.IsValid())
							{
#if ENABLE_AUDIO_DEBUG
								// Writing to this value is a read/write race condition on the CPUCoreUtilization value. Calling this
								// out as an acceptable race condition given that it is utilized for debug purposes only. 
								GameThreadInfo.CPUCoreUtilization[SourceId] = SourceInfo.MixerSourceBuffer->GetCPUCoreUtilization();
#endif // if ENABLE_AUDIO_DEBUG
								GameThreadInfo.RelativeRenderCost[SourceId] = SourceInfo.MixerSourceBuffer->GetRelativeRenderCost();

								SourceInfo.MixerSourceBuffer->OnBufferEnd();
							}

							// If we're out of audio in our queue, we're done
							if (SourceInfo.MixerSourceBuffer->IsEndOfAudio())
							{
								SourceInfo.bIsLastBuffer = true;
								break;
							}

							SourceInfo.CurrentPCMBuffer = SourceInfo.MixerSourceBuffer->GetNextBuffer();
							SourceInfo.CurrentFrameIndex = 0;
							CurrentChunkFrames = SourceInfo.GetCurrentAudioChunkNumFrames();

							if (!ensure(SourceInfo.CurrentPCMBuffer.IsValid()))
							{
								break;
							}

							if (CurrentChunkFrames == 0)
							{
								// A source can return an empty buffer at the end of its input. In this case
								//  we just ask again for a buffer and it will report it's done.
								continue;
							}
						}

						if (FinalPitch != 1 && !SourceInfo.bResampling)
						{
							SourceInfo.bResampling = true;
							if (bIsFirstRun)
							{
								// Immediately start at the requested playback rate
								SourceInfo.Resampler.SetFrameRatio(FinalPitch, 0);
							}
						}

						// Pump as many samples as possibe from CurrentPCMBuffer to PreDistanceAttenuationBuffer
						check(SourceInfo.CurrentPCMBuffer.IsValid());
						const int32 DestinationOffset = NumSamples - FramesToWrite * SourceInfo.NumInputChannels;
						float* CopyDestination = SourceInfo.PreDistanceAttenuationBuffer.GetData() + DestinationOffset;
						const float* CopySource = SourceInfo.CurrentPCMBuffer->GetData() + SourceInfo.CurrentFrameIndex * SourceInfo.NumInputChannels;

						if (SourceInfo.bResampling)
						{
							// Resample if required
							int32 NumFramesConsumed = 0;
							int32 NumFramesProduced = 0;
							SourceInfo.Resampler.SetFrameRatio(FinalPitch, NumOutputFrames);
							SourceInfo.Resampler.ProcessInterleaved(
								TConstArrayView<float>{ CopySource, SourceInfo.NumInputChannels* (CurrentChunkFrames - SourceInfo.CurrentFrameIndex) },
								TArrayView<float>{ CopyDestination, SourceInfo.NumInputChannels* FramesToWrite },
								NumFramesConsumed, NumFramesProduced);

							// Update bookkeeping
							check(NumFramesConsumed > 0 || NumFramesProduced > 0); // Make sure we don't get completely stuck
							SourceInfo.CurrentFrameIndex += NumFramesConsumed;
							SourceInfo.NumFramesPlayed += NumFramesConsumed;
							FramesToWrite -= NumFramesProduced;
						}
						else
						{
							// No resampling necessary, just copy
							const int32 FramesToCopy = FMath::Min(FramesToWrite, CurrentChunkFrames - SourceInfo.CurrentFrameIndex);
							const int32 SamplesToCopy = FramesToCopy * SourceInfo.NumInputChannels;
							FMemory::Memcpy(CopyDestination, CopySource, SamplesToCopy * sizeof(float));

							// Update bookkeeping
							SourceInfo.CurrentFrameIndex += FramesToCopy;
							SourceInfo.NumFramesPlayed += FramesToCopy;
							FramesToWrite -= FramesToCopy;
						}
					}
				}
			}

			// After processing the frames, reset the pitch param
			SourceInfo.PitchSourceParam.Reset();

			// Reset target value as modulation may have modified prior to processing
			// And source param should not store modulation value internally as its
			// processed by the modulation plugin independently.
			SourceInfo.PitchSourceParam.SetValue(TargetPitch, NumOutputFrames);
		}
	}
	
	void FMixerSourceManager::ConnectBusPatches()
	{
		while (TOptional<FPendingAudioBusConnection> PendingAudioBusConnection = PendingAudioBusConnections.Dequeue())
		{
			FAudioBusKey& AudioBusKey = PendingAudioBusConnection->AudioBusKey;
			int32 NumChannels = PendingAudioBusConnection->NumChannels;
			bool bIsAutomatic = PendingAudioBusConnection->bIsAutomatic;

			bool bAlreadyExisted = false;
			TSharedRef<FMixerAudioBus> AudioBusPtr = FindOrAddAudioBus(AudioBusKey, NumChannels, bIsAutomatic, &bAlreadyExisted);
			if (bAlreadyExisted)
			{
				if (!ensure(AudioBusPtr->GetNumChannels() == NumChannels))
				{
					// Refuse the connection if the patch's channel count disagrees with the bus. The patch carries interleaved samples sized for NumChannels, so attaching it
					//  to a bus with a different channel count would corrupt the bus output.
					UE_LOGF(LogAudioMixer, Error, "Audio bus patch channel count mismatch for bus id %u: bus has %d channels, patch has %d. Refusing connection to avoid distorted audio.",
						AudioBusKey.ObjectId, AudioBusPtr->GetNumChannels(), NumChannels);
					continue;
				}
				AudioBusPtr->SetAutomatic(bIsAutomatic);
			}

			switch (PendingAudioBusConnection->PatchVariant.GetIndex())
			{
			case FPendingAudioBusConnection::FPatchVariant::IndexOfType<FPatchInput>():
				if (UseRenderScheduler())
				{
					const uint32 PlayOrder = PendingAudioBusConnection->WaveInstancePlayOrder;
					if (PlayOrder != INDEX_NONE)
					{
						FAudioRenderStepId FirstStep = FAudioRenderStepId::FromWaveInstancePlayOrder(PlayOrder);
						FAudioRenderStepId SecondStep = FAudioRenderStepId::FromAudioBusKey(AudioBusKey);
						AddBusPatchDependency(PlayOrder, FirstStep, SecondStep);
					}
				}
				AudioBusPtr->AddNewPatchInput(PendingAudioBusConnection->PatchVariant.Get<FPatchInput>());

#if UE_AUDIO_PROFILERTRACE_ENABLED
				{
					const uint32 PlayOrder = ResolvePlayOrder(*PendingAudioBusConnection);
					if (PlayOrder != INDEX_NONE)
					{
						TraceBusPatchWriterConnectedMessage(MixerDevice->DeviceID, PlayOrder, AudioBusKey.ObjectId);
						ActiveBusPatchWriters.AddUnique(PlayOrder, AudioBusKey.ObjectId);
					}
				}
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

				break;
			case FPendingAudioBusConnection::FPatchVariant::IndexOfType<FPatchOutputStrongPtr>():
				if (UseRenderScheduler())
				{
					const uint32 PlayOrder = PendingAudioBusConnection->WaveInstancePlayOrder;
					if (PlayOrder != INDEX_NONE)
					{
						FAudioRenderStepId FirstStep = FAudioRenderStepId::FromAudioBusKey(AudioBusKey);
						FAudioRenderStepId SecondStep = FAudioRenderStepId::FromWaveInstancePlayOrder(PlayOrder);
						AddBusPatchDependency(PlayOrder, FirstStep, SecondStep);
					}
				}
				AudioBusPtr->AddNewPatchOutput(PendingAudioBusConnection->PatchVariant.Get<FPatchOutputStrongPtr>());

#if UE_AUDIO_PROFILERTRACE_ENABLED
				{
					const uint32 PlayOrder = ResolvePlayOrder(*PendingAudioBusConnection);
					if (PlayOrder != INDEX_NONE)
					{
						TraceBusPatchReaderConnectedMessage(MixerDevice->DeviceID, AudioBusKey.ObjectId, PlayOrder);
						ActiveBusPatchReaders.AddUnique(PlayOrder, AudioBusKey.ObjectId);
					}
				}
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

				break;
			}
		}
	}

	void FMixerSourceManager::ComputeBuses()
	{
		RenderThreadPhase = ESourceManagerRenderThreadPhase::ComputeBusses;

		ConnectBusPatches();

		// Loop through the bus registry and mix source audio
		for (auto& [AudioBusKey, AudioBus] : AudioBuses)
		{
			if (!AudioBus.IsValid())
			{
				continue;
			}

			AudioBus->MixBuffer();
		}
	}

	void FMixerSourceManager::UpdateBuses()
	{
		RenderThreadPhase = ESourceManagerRenderThreadPhase::UpdateBusses;

#if UE_AUDIO_PROFILERTRACE_ENABLED
		const bool bAudioChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AudioChannel);
		const bool bAudioMixerChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AudioMixerChannel);
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

		// Update the bus states post mixing. This flips the current/previous buffer indices.
		for (auto& [AudioBusKey, AudioBus] : AudioBuses)
		{
			if (!AudioBus.IsValid())
			{
				continue;
			}

			if (!UseRenderScheduler())
			{
				AudioBus->Update();
			}
			
#if UE_AUDIO_PROFILERTRACE_ENABLED
			if (bAudioChannelEnabled)
			{
				const float* AudioBuffer = AudioBus->GetCurrentBusBuffer();
				const int32 NumSamples = AudioBus->NumChannels * AudioBus->NumFrames;

				const bool bIsAudioBufferSilent = IsAudioBufferSilent(AudioBuffer, NumSamples);

				UE_TRACE_LOG(Audio, AudioBusHasActivity, AudioChannel)
					<< AudioBusHasActivity.DeviceId(MixerDevice->DeviceID)
					<< AudioBusHasActivity.AudioBusId(AudioBusKey.ObjectId)
					<< AudioBusHasActivity.Timestamp(FPlatformTime::Cycles64())
					<< AudioBusHasActivity.HasActivity(!bIsAudioBufferSilent);

				if (BusIsAlivePingTimeSec >= BusIsAlivePingWaitTimeSec)
				{
					TraceAudioBusIsAlivePingMessage(MixerDevice->DeviceID, AudioBusKey.ObjectId, AudioBus->BusName, AudioBus->NumChannels);
				}
			}

			if (bAudioMixerChannelEnabled)
			{
				if (BusIsAlivePingTimeSec >= BusIsAlivePingWaitTimeSec)
				{
					AutomaticallyStartAudioBusEnvelopeFollower(AudioBusKey, AudioBus);
				}

				if (AudioBus->bIsEnvelopeFollowing)
				{
					UE_TRACE_LOG(Audio, AudioBusEnvelopeValues, AudioMixerChannel)
						<< AudioBusEnvelopeValues.DeviceId(MixerDevice->DeviceID)
						<< AudioBusEnvelopeValues.AudioBusId(AudioBusKey.ObjectId)
						<< AudioBusEnvelopeValues.Timestamp(FPlatformTime::Cycles64())
						<< AudioBusEnvelopeValues.EnvelopeValues(AudioBus->EnvelopeValues, AudioBus->GetNumChannels());
				}
			}
			else if (AudioBus->bIsEnvelopeFollowing)
			{
				StopAutoAudioBusEnvelopeFollower(AudioBusKey.ObjectId, *AudioBus);
			}
#endif // UE_AUDIO_PROFILERTRACE_ENABLED
		}

#if UE_AUDIO_PROFILERTRACE_ENABLED
		if (bAudioChannelEnabled)
		{
			if (BusIsAlivePingTimeSec >= BusIsAlivePingWaitTimeSec)
			{
				for (const auto& [PlayOrder, BusObjectId] : ActiveBusPatchWriters)
				{
					TraceBusPatchWriterConnectedMessage(MixerDevice->DeviceID, PlayOrder, BusObjectId);
				}

				for (const auto& [PlayOrder, BusObjectId] : ActiveBusPatchReaders)
				{
					TraceBusPatchReaderConnectedMessage(MixerDevice->DeviceID, BusObjectId, PlayOrder);
				}

				BusIsAlivePingTimeSec = 0.0;
			}
			else
			{
				BusIsAlivePingTimeSec += MixerDevice->GetAudioClockDelta();
			}
		}
#endif // UE_AUDIO_PROFILERTRACE_ENABLED
	}

	void FMixerSourceManager::AddBusPatchDependency(const uint32 WaveInstancePlayOrder, const FAudioRenderStepId FirstStep, const FAudioRenderStepId SecondStep)
	{
		check(UseRenderScheduler());
		if (FirstStep.IsValid() && SecondStep.IsValid())
		{
			GetRenderScheduler().AddDependency(FirstStep, SecondStep);
			BusPatchDependencies.Add(WaveInstancePlayOrder, { FirstStep, SecondStep });
		}
	}

	void FMixerSourceManager::RemoveBusPatchDependencies(const uint32 WaveInstancePlayOrder)
	{
		check(UseRenderScheduler());
		TArray<TPair<FAudioRenderStepId, FAudioRenderStepId>> Dependencies;
		BusPatchDependencies.MultiFind(WaveInstancePlayOrder, Dependencies, false);
		if (!Dependencies.IsEmpty())
		{
			FAudioRenderScheduler& Scheduler = GetRenderScheduler();
			for (TPair<FAudioRenderStepId, FAudioRenderStepId>& Dependency : Dependencies)
			{
				Scheduler.RemoveDependency(Dependency.Get<0>(), Dependency.Get<1>());
			}
			BusPatchDependencies.Remove(WaveInstancePlayOrder);
		}
	}

	float FMixerSourceManager::GetFloatCompareTolerance() const
	{
		const float Percent = GetCommandQueueFillPercentage();
		return FMath::IsNearlyEqual(Percent, 1.0f) ? FloatCompareMaxScaleCVar : FloatCompareMinScaleCVar;
	}

	float FMixerSourceManager::GetCommandQueueFillPercentage() const
	{
		if (NumCmdsConsideredFullCVar == 0)
		{
			return 0.f;
		}
		return static_cast<float>(FMath::Min(NumCommands.GetValue(), NumCmdsConsideredFullCVar) / NumCmdsConsideredFullCVar);
	}

	// ctor
	FMixerSourceManager::FAudioMixerThreadCommand::FAudioMixerThreadCommand(TFunction<void()>&& InFunction, const char* InDebugString, bool bInDeferExecution)
	: Function(MoveTemp(InFunction))
	, bDeferExecution(bInDeferExecution)		
#if WITH_AUDIO_MIXER_THREAD_COMMAND_DEBUG
	, DebugString(InDebugString)
#endif //WITH_AUDIO_MIXER_THREAD_COMMAND_DEBUG
	{
	}

	void FMixerSourceManager::FAudioMixerThreadCommand::operator()() const
	{
#if WITH_AUDIO_MIXER_THREAD_COMMAND_DEBUG
		StartExecuteTimeInCycles = FPlatformTime::Cycles64();
#endif //WITH_AUDIO_MIXER_THREAD_COMMAND_DEBUG

		Function();

#if WITH_AUDIO_MIXER_THREAD_COMMAND_DEBUG
		StartExecuteTimeInCycles = 0;
#endif //WITH_AUDIO_MIXER_THREAD_COMMAND_DEBUG
	}

	FString FMixerSourceManager::FAudioMixerThreadCommand::GetSafeDebugString() const
	{
#if WITH_AUDIO_MIXER_THREAD_COMMAND_DEBUG
		return FString(DebugString ? ANSI_TO_TCHAR(DebugString): TEXT(""));
#else  //WITH_AUDIO_MIXER_THREAD_COMMAND_DEBUG
		return {};
#endif //WITH_AUDIO_MIXER_THREAD_COMMAND_DEBUG
	}

	float FMixerSourceManager::FAudioMixerThreadCommand::GetExecuteTimeInSeconds() const
	{
#if WITH_AUDIO_MIXER_THREAD_COMMAND_DEBUG
		return FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartExecuteTimeInCycles);
#else  //WITH_AUDIO_MIXER_THREAD_COMMAND_DEBUG
		return 0.f;
#endif //WITH_AUDIO_MIXER_THREAD_COMMAND_DEBUG
	}

	void FMixerSourceManager::ApplyDistanceAttenuation(FSourceInfo& SourceInfo, int32 NumSamples)
	{
		if (DisableDistanceAttenuationCvar)
		{
			return;
		}

		TArrayView<float> PostDistanceAttenBufferView(SourceInfo.SourceBuffer.GetData(), SourceInfo.SourceBuffer.Num());
		Audio::ArrayFade(PostDistanceAttenBufferView, SourceInfo.DistanceAttenuationSourceStart, SourceInfo.DistanceAttenuationSourceDestination);
		SourceInfo.DistanceAttenuationSourceStart = SourceInfo.DistanceAttenuationSourceDestination;
	}

	void FMixerSourceManager::ComputePluginAudio(FSourceInfo& SourceInfo, FMixerSourceSubmixOutputBuffer& InSourceSubmixOutputBuffer, int32 SourceId, int32 NumSamples)
	{
		if (BypassAudioPluginsCvar)
		{
			// If we're bypassing audio plugins, our pre- and post-effect channels are the same as the input channels
			SourceInfo.NumPostEffectChannels = SourceInfo.NumInputChannels;

			// Set the ptr to use for post-effect buffers:
			InSourceSubmixOutputBuffer.SetPostAttenuationSourceBuffer(&SourceInfo.SourceBuffer);

			if (SourceInfo.bHasPreDistanceAttenuationSend)
			{
				InSourceSubmixOutputBuffer.SetPreAttenuationSourceBuffer(&SourceInfo.PreDistanceAttenuationBuffer);
			}
			return;
		}

		if (SourceInfo.AudioLink.IsValid())
		{
			IAudioLinkSourcePushed::FOnNewBufferParams Params;
			Params.SourceId = SourceId;
			Params.Buffer = SourceInfo.PreDistanceAttenuationBuffer;
			SourceInfo.AudioLink->OnNewBuffer(Params);
			FMemory::Memzero(SourceInfo.PreDistanceAttenuationBuffer.GetData(), SourceInfo.PreDistanceAttenuationBuffer.Num() * sizeof(float));
			FMemory::Memzero(SourceInfo.SourceBuffer.GetData(), SourceInfo.SourceBuffer.Num() * sizeof(float));
		}

		// If we have Source Buffer Listener
		if (SourceInfo.SourceBufferListener.IsValid())
		{
			// Pack all our state into a single struct.
			ISourceBufferListener::FOnNewBufferParams Params;
			Params.SourceId			= SourceId;
			Params.AudioData		= SourceInfo.PreDistanceAttenuationBuffer.GetData();
			Params.NumSamples		= SourceInfo.PreDistanceAttenuationBuffer.Num();
			Params.NumChannels		= SourceInfo.NumInputChannels;
			Params.SampleRate		= MixerDevice->GetSampleRate();

			// Fire callback.
			SourceInfo.SourceBufferListener->OnNewBuffer(Params);

			// Optionally, clear the buffer after we've broadcast it. 
			if (SourceInfo.bShouldSourceBufferListenerZeroBuffer)
			{
				FMemory::Memzero(SourceInfo.PreDistanceAttenuationBuffer.GetData(), SourceInfo.PreDistanceAttenuationBuffer.Num() * sizeof(float));
				FMemory::Memzero(SourceInfo.SourceBuffer.GetData(), SourceInfo.SourceBuffer.Num() * sizeof(float));
			}
		}

		float* PostDistanceAttenBufferPtr = SourceInfo.SourceBuffer.GetData();

		bool bShouldMixInReverb = false;
		if (SourceInfo.bUseReverbPlugin)
		{
			const FSpatializationParams* SourceSpatParams = &SourceInfo.SpatParams;

			// Move the audio buffer to the reverb plugin buffer
			FAudioPluginSourceInputData AudioPluginInputData;
			AudioPluginInputData.SourceId = SourceId;
			AudioPluginInputData.AudioBuffer = &SourceInfo.SourceBuffer;
			AudioPluginInputData.SpatializationParams = SourceSpatParams;
			AudioPluginInputData.NumChannels = SourceInfo.NumInputChannels;
			AudioPluginInputData.AudioComponentId = SourceInfo.AudioComponentID;
			SourceInfo.AudioPluginOutputData.AudioBuffer.Reset();
			SourceInfo.AudioPluginOutputData.AudioBuffer.AddZeroed(AudioPluginInputData.AudioBuffer->Num());

			MixerDevice->ReverbPluginInterface->ProcessSourceAudio(AudioPluginInputData, SourceInfo.AudioPluginOutputData);

			// Make sure the buffer counts didn't change and are still the same size
			AUDIO_MIXER_CHECK(SourceInfo.AudioPluginOutputData.AudioBuffer.Num() == NumSamples);

			//If the reverb effect doesn't send it's audio to an external device, mix the output data back in.
			if (!MixerDevice->bReverbIsExternalSend)
			{
				// Copy the reverb-processed data back to the source buffer
				InSourceSubmixOutputBuffer.CopyReverbPluginOutputData(SourceInfo.AudioPluginOutputData.AudioBuffer);
				bShouldMixInReverb = true;
			}
		}

		TArrayView<const float> ReverbPluginOutputBufferView(InSourceSubmixOutputBuffer.GetReverbPluginOutputData(), NumSamples);
		TArrayView<const float> AudioPluginOutputDataView(SourceInfo.AudioPluginOutputData.AudioBuffer.GetData(), NumSamples);
		TArrayView<float> PostDistanceAttenBufferView(PostDistanceAttenBufferPtr, NumSamples);

		if (SourceInfo.bUseOcclusionPlugin)
		{
			const FSpatializationParams* SourceSpatParams = &SourceInfo.SpatParams;

			// Move the audio buffer to the occlusion plugin buffer
			FAudioPluginSourceInputData AudioPluginInputData;
			AudioPluginInputData.SourceId = SourceId;
			AudioPluginInputData.AudioBuffer = &SourceInfo.SourceBuffer;
			AudioPluginInputData.SpatializationParams = SourceSpatParams;
			AudioPluginInputData.NumChannels = SourceInfo.NumInputChannels;
			AudioPluginInputData.AudioComponentId = SourceInfo.AudioComponentID;

			SourceInfo.AudioPluginOutputData.AudioBuffer.Reset();
			SourceInfo.AudioPluginOutputData.AudioBuffer.AddZeroed(AudioPluginInputData.AudioBuffer->Num());

			MixerDevice->OcclusionInterface->ProcessAudio(AudioPluginInputData, SourceInfo.AudioPluginOutputData);

			// Make sure the buffer counts didn't change and are still the same size
			AUDIO_MIXER_CHECK(SourceInfo.AudioPluginOutputData.AudioBuffer.Num() == NumSamples);

			// Copy the occlusion-processed data back to the source buffer and mix with the reverb plugin output buffer
			if (bShouldMixInReverb)
			{
				Audio::ArraySum(ReverbPluginOutputBufferView, AudioPluginOutputDataView, PostDistanceAttenBufferView);
			}
			else
			{
				FMemory::Memcpy(PostDistanceAttenBufferPtr, SourceInfo.AudioPluginOutputData.AudioBuffer.GetData(), sizeof(float) * NumSamples);
			}
		}
		else if (bShouldMixInReverb)
		{
			Audio::ArrayMixIn(ReverbPluginOutputBufferView, PostDistanceAttenBufferView);
		}

		// If the source has HRTF processing enabled, run it through the spatializer
		if (SourceInfo.bUseHRTFSpatializer)
		{
			CSV_SCOPED_TIMING_STAT(Audio, HRTF);
			SCOPE_CYCLE_COUNTER(STAT_AudioMixerHRTF);

			AUDIO_MIXER_CHECK(SpatialInterfaceInfo.SpatializationPlugin.IsValid());
			AUDIO_MIXER_CHECK(SourceInfo.NumInputChannels <= SpatialInterfaceInfo.MaxChannelsSupportedBySpatializationPlugin);

			FAudioPluginSourceInputData AudioPluginInputData;
			AudioPluginInputData.AudioBuffer = &SourceInfo.SourceBuffer;
			AudioPluginInputData.NumChannels = SourceInfo.NumInputChannels;
			AudioPluginInputData.SourceId = SourceId;
			AudioPluginInputData.SpatializationParams = &SourceInfo.SpatParams;
			AudioPluginInputData.AudioComponentId = SourceInfo.AudioComponentID;

			if (!SpatialInterfaceInfo.bSpatializationIsExternalSend)
			{
				SourceInfo.AudioPluginOutputData.AudioBuffer.Reset();
				SourceInfo.AudioPluginOutputData.AudioBuffer.AddZeroed(2 * NumOutputFrames);
			}

			{
				LLM_SCOPE(ELLMTag::AudioMixerPlugins);
				SpatialInterfaceInfo.SpatializationPlugin->ProcessAudio(AudioPluginInputData, SourceInfo.AudioPluginOutputData);
			}

			// If this is an external send, we treat this source audio as if it was still a mono source
			// This will allow it to traditionally pan in the ComputeOutputBuffers function and be
			// sent to submixes (e.g. reverb) panned and mixed down. Certain submixes will want this spatial 
			// information in addition to the external send. We've already bypassed adding this source
			// to a base submix (e.g. master/eq, etc)
			if (SpatialInterfaceInfo.bSpatializationIsExternalSend)
			{
				// Otherwise our pre- and post-effect channels are the same as the input channels
				SourceInfo.NumPostEffectChannels = SourceInfo.NumInputChannels;

				// Set the ptr to use for post-effect buffers rather than the plugin output data (since the plugin won't have output audio data)
				InSourceSubmixOutputBuffer.SetPostAttenuationSourceBuffer(&SourceInfo.SourceBuffer);

				if (SourceInfo.bHasPreDistanceAttenuationSend)
				{
					InSourceSubmixOutputBuffer.SetPreAttenuationSourceBuffer(&SourceInfo.PreDistanceAttenuationBuffer);
				}
			}
			else
			{
				// Otherwise, we are now a 2-channel file and should not be spatialized using normal 3d spatialization
				SourceInfo.NumPostEffectChannels = 2;

				// Set the ptr to use for post-effect buffers rather than the plugin output data (since the plugin won't have output audio data)
				InSourceSubmixOutputBuffer.SetPostAttenuationSourceBuffer(&SourceInfo.AudioPluginOutputData.AudioBuffer);

				if (SourceInfo.bHasPreDistanceAttenuationSend)
				{
					InSourceSubmixOutputBuffer.SetPreAttenuationSourceBuffer(&SourceInfo.PreDistanceAttenuationBuffer);
				}
			}
		}
		else
		{
			// Otherwise our pre- and post-effect channels are the same as the input channels
			SourceInfo.NumPostEffectChannels = SourceInfo.NumInputChannels;

			InSourceSubmixOutputBuffer.SetPostAttenuationSourceBuffer(&SourceInfo.SourceBuffer);

			if (SourceInfo.bHasPreDistanceAttenuationSend)
			{
				InSourceSubmixOutputBuffer.SetPreAttenuationSourceBuffer(&SourceInfo.PreDistanceAttenuationBuffer);
			}
		}
	}

	void FMixerSourceManager::ComputePostSourceEffectBuffer(const bool bGenerateBuses, const int32 SourceId)
	{
		CSV_SCOPED_TIMING_STAT(Audio, SourceEffectsBuffers);
		SCOPE_CYCLE_COUNTER(STAT_AudioMixerSourceEffectBuffers);

		const bool bIsDebugModeEnabled = DebugSoloSources.Num() > 0;

		FSourceInfo& SourceInfo = SourceInfos[SourceId];

		if (!SourceInfo.bIsBusy || !SourceInfo.bIsPlaying || SourceInfo.bIsPaused || SourceInfo.bIsPausedForQuantization || (SourceInfo.bIsDone && SourceInfo.bEffectTailsDone))
		{
			return;
		}

		const bool bIsSourceBus = SourceInfo.AudioBusId != INDEX_NONE;
		if ((bGenerateBuses && !bIsSourceBus) || (!bGenerateBuses && bIsSourceBus))
		{
			return;
		}

		// Copy and store the current state of the pre-distance attenuation buffer before we feed it through our source effects
		// This is used by pre-effect sends
		if (SourceInfo.AudioBusSends[(int32)EBusSendType::PreEffect].Num() > 0)
		{
			SourceInfo.PreEffectBuffer.Reset();
			SourceInfo.PreEffectBuffer.AddZeroed(SourceInfo.PreDistanceAttenuationBuffer.Num());

			FMemory::Memcpy(SourceInfo.PreEffectBuffer.GetData(), SourceInfo.PreDistanceAttenuationBuffer.GetData(), sizeof(float) * SourceInfo.PreDistanceAttenuationBuffer.Num());
		}

		float* PreDistanceAttenBufferPtr = SourceInfo.PreDistanceAttenuationBuffer.GetData();
		const int32 NumSamples = SourceInfo.PreDistanceAttenuationBuffer.Num();

		TArrayView<float> PreDistanceAttenBufferView(PreDistanceAttenBufferPtr, NumSamples);

		// Update volume fade information if we're stopping
		{
			float VolumeStart = 1.0f;
			float VolumeDestination = 1.0f;
			if (SourceInfo.bIsStopping)
			{
				int32 NumFadeFrames = FMath::Min(SourceInfo.VolumeFadeNumFrames - SourceInfo.VolumeFadeFramePosition, NumOutputFrames);

				SourceInfo.VolumeFadeFramePosition += NumFadeFrames;
				SourceInfo.VolumeSourceDestination = SourceInfo.VolumeFadeSlope * (float)SourceInfo.VolumeFadeFramePosition + SourceInfo.VolumeFadeStart;

				if (FMath::IsNearlyZero(SourceInfo.VolumeSourceDestination, KINDA_SMALL_NUMBER))
				{
					SourceInfo.VolumeSourceDestination = 0.0f;
				}

				const int32 NumFadeSamples = NumFadeFrames * SourceInfo.NumInputChannels;

				VolumeStart = SourceInfo.VolumeSourceStart;
				VolumeDestination = SourceInfo.VolumeSourceDestination;
				if (MixerDevice->IsModulationPluginEnabled() && MixerDevice->ModulationInterface.IsValid())
				{
					const bool bHasProcessed = SourceInfo.VolumeModulation.GetHasProcessed();
					const float ModVolumeStart = SourceInfo.VolumeModulation.GetValue();
					SourceInfo.VolumeModulation.ProcessControl(SourceInfo.VolumeModulationBase);
					const float ModVolumeEnd = SourceInfo.VolumeModulation.GetValue();
					if (bHasProcessed)
					{
						VolumeStart *= ModVolumeStart;
					}
					else
					{
						VolumeStart *= ModVolumeEnd;
					}
					VolumeDestination *= ModVolumeEnd;
					if (SourceId < NumLegacySources)
					{
						GameThreadInfo.ModulationVolume[SourceId] = ModVolumeEnd;
					}
				}

				TArrayView<float> PreDistanceAttenBufferFadeSamplesView(PreDistanceAttenBufferPtr, NumFadeSamples);
				Audio::ArrayFade(PreDistanceAttenBufferFadeSamplesView, VolumeStart, VolumeDestination);

				// Zero the rest of the buffer
				if (NumFadeFrames < NumOutputFrames)
				{
					int32 SamplesLeft = NumSamples - NumFadeSamples;

					// Protect memzero call with some sanity checking on the inputs.
					if (SamplesLeft > 0 && NumFadeSamples >= 0 && NumFadeSamples < NumSamples)
					{
						FMemory::Memzero(&PreDistanceAttenBufferPtr[NumFadeSamples], sizeof(float) * SamplesLeft);
					}
				}
			}
			else
			{
				VolumeStart = SourceInfo.VolumeSourceStart;
				VolumeDestination = SourceInfo.VolumeSourceDestination;
				if (MixerDevice->IsModulationPluginEnabled() && MixerDevice->ModulationInterface.IsValid())
				{
					const bool bHasProcessed = SourceInfo.VolumeModulation.GetHasProcessed();
					const float ModVolumeStart = SourceInfo.VolumeModulation.GetValue();
					SourceInfo.VolumeModulation.ProcessControl(SourceInfo.VolumeModulationBase);
					const float ModVolumeEnd = SourceInfo.VolumeModulation.GetValue();
					if (bHasProcessed)
					{
						VolumeStart *= ModVolumeStart;
					}
					else
					{
						VolumeStart *= ModVolumeEnd;
					}
					VolumeDestination *= ModVolumeEnd;
					if (SourceId < NumLegacySources)
					{
						GameThreadInfo.ModulationVolume[SourceId] = ModVolumeEnd;
					}
				}

				Audio::ArrayFade(PreDistanceAttenBufferView, VolumeStart, VolumeDestination);
			}

#if UE_AUDIO_PROFILERTRACE_ENABLED
			const bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AudioMixerChannel);
			if (bChannelEnabled)
			{
				UE_TRACE_LOG(Audio, SoundVolume, AudioMixerChannel)
					<< SoundVolume.DeviceId(MixerDevice->DeviceID)
					<< SoundVolume.Timestamp(FPlatformTime::Cycles64())
					<< SoundVolume.PlayOrder(SourceInfo.PlayOrder)
					<< SoundVolume.ActiveSoundPlayOrder(SourceInfo.ActiveSoundPlayOrder)
					<< SoundVolume.Volume(VolumeDestination);

				UE_TRACE_LOG(Audio, SoundDistanceAttenuation, AudioMixerChannel)
					<< SoundDistanceAttenuation.DeviceId(MixerDevice->DeviceID)
					<< SoundDistanceAttenuation.Timestamp(FPlatformTime::Cycles64())
					<< SoundDistanceAttenuation.PlayOrder(SourceInfo.PlayOrder)
					<< SoundDistanceAttenuation.ActiveSoundPlayOrder(SourceInfo.ActiveSoundPlayOrder)
					<< SoundDistanceAttenuation.DistanceAttenuation(SourceInfo.DistanceAttenuationSourceDestination);

				TraceSourceToSubmixSendMessage(SourceInfo);
				TraceSourceToBusSendMessage(SourceId, SourceInfo);
			}
#endif // UE_AUDIO_PROFILERTRACE_ENABLED
		}

		SourceInfo.VolumeSourceStart = SourceInfo.VolumeSourceDestination;

		// Now process the effect chain if it exists
		if (!DisableSourceEffectsCvar && SourceInfo.SourceEffects.Num() > 0)
		{
			// Prepare this source's effect chain input data
			SourceInfo.SourceEffectInputData.CurrentVolume = SourceInfo.VolumeSourceDestination;

			const float Pitch = Audio::GetFrequencyMultiplier(SourceInfo.PitchModulation.GetValue());
			SourceInfo.SourceEffectInputData.CurrentPitch = SourceInfo.PitchSourceParam.GetValue() * Pitch;
			SourceInfo.SourceEffectInputData.AudioClock = MixerDevice->GetAudioClock();
			if (SourceInfo.NumInputFrames > 0)
			{
				SourceInfo.SourceEffectInputData.CurrentPlayFraction = (float)SourceInfo.NumFramesPlayed / SourceInfo.NumInputFrames;
			}
			SourceInfo.SourceEffectInputData.SpatParams = SourceInfo.SpatParams;

			// Get a ptr to pre-distance attenuation buffer ptr
			float* OutputSourceEffectBufferPtr = SourceInfo.SourceEffectScratchBuffer.GetData();

			SourceInfo.SourceEffectInputData.InputSourceEffectBufferPtr = SourceInfo.PreDistanceAttenuationBuffer.GetData();
			SourceInfo.SourceEffectInputData.NumSamples = NumSamples;

			// Loop through the effect chain passing in buffers
			FScopeLock ScopeLock(&EffectChainMutationCriticalSection);
			{
				for (TSoundEffectSourcePtr& SoundEffectSource : SourceInfo.SourceEffects)
				{
					bool bPresetUpdated = false;
					if (SoundEffectSource->IsActive())
					{
						bPresetUpdated = SoundEffectSource->Update();
					}

					if (SoundEffectSource->IsActive())
					{
						SoundEffectSource->ProcessAudio(SourceInfo.SourceEffectInputData, OutputSourceEffectBufferPtr);

						// Copy output to input
						FMemory::Memcpy(SourceInfo.SourceEffectInputData.InputSourceEffectBufferPtr, OutputSourceEffectBufferPtr, sizeof(float) * NumSamples);
					}
				}
			}
		}

		const bool bWasEffectTailsDone = SourceInfo.bEffectTailsDone;

		if (!DisableEnvelopeFollowingCvar)
		{
			// Compute the source envelope using pre-distance attenuation buffer
			float AverageSampleValue = Audio::ArrayGetAverageAbsValue(PreDistanceAttenBufferView);
			SourceInfo.SourceEnvelopeValue = SourceInfo.SourceEnvelopeFollower.ProcessSample(AverageSampleValue);
			SourceInfo.SourceEnvelopeValue = FMath::Clamp(SourceInfo.SourceEnvelopeValue, 0.f, 1.f);

			SourceInfo.bEffectTailsDone = SourceInfo.bEffectTailsDone || SourceInfo.SourceEnvelopeValue < ENVELOPE_TAIL_THRESHOLD;
		}
		else
		{
			SourceInfo.bEffectTailsDone = true;
		}

		if (SourceInfo.SourceListener && !bWasEffectTailsDone && SourceInfo.bEffectTailsDone)
		{
			SourceInfo.SourceListener->OnEffectTailsDone();
		}

		const bool bModActive = MixerDevice->IsModulationPluginEnabled() && MixerDevice->ModulationInterface.IsValid();
		bool bUpdateModFilters = bModActive && (SourceInfo.bModFiltersUpdated || SourceInfo.LowpassModulation.IsActive() || SourceInfo.HighpassModulation.IsActive());

		if (SourceInfo.IsRenderingToSubmixes() || bUpdateModFilters)
		{
			// Only scale with distance attenuation and send to source audio to plugins if we're not in output-to-bus only mode
			const int32 NumOutputSamplesThisSource = NumOutputFrames * SourceInfo.NumInputChannels;

			if (!SourceInfo.IsRenderingToSubmixes())
			{
				SourceInfo.LowpassModulation.ProcessControl(SourceInfo.LowpassModulationBase);
				SourceInfo.LowPassFilter.StartFrequencyInterpolation(SourceInfo.LowpassModulation.GetValue(), NumOutputFrames);

				SourceInfo.HighpassModulation.ProcessControl(SourceInfo.HighpassModulationBase);
				SourceInfo.HighPassFilter.StartFrequencyInterpolation(SourceInfo.HighpassModulation.GetValue(), NumOutputFrames);
			}
			else if (bUpdateModFilters)
			{
				const float LowpassFreq = FMath::Min(SourceInfo.LowpassModulationBase, SourceInfo.LowPassFreq);
				SourceInfo.LowpassModulation.ProcessControl(LowpassFreq);
				SourceInfo.LowPassFilter.StartFrequencyInterpolation(SourceInfo.LowpassModulation.GetValue(), NumOutputFrames);

				const float HighpassFreq = FMath::Max(SourceInfo.HighpassModulationBase, SourceInfo.HighPassFreq);
				SourceInfo.HighpassModulation.ProcessControl(HighpassFreq);
				SourceInfo.HighPassFilter.StartFrequencyInterpolation(SourceInfo.HighpassModulation.GetValue(), NumOutputFrames);
			}

			const bool bBypassLPF = DisableFilteringCvar || (SourceInfo.LowPassFilter.GetCutoffFrequency() >= (MAX_FILTER_FREQUENCY - KINDA_SMALL_NUMBER));
			const bool bBypassHPF = DisableFilteringCvar || DisableHPFilteringCvar || (SourceInfo.HighPassFilter.GetCutoffFrequency() <= (MIN_FILTER_FREQUENCY + KINDA_SMALL_NUMBER));

			float* SourceBuffer = SourceInfo.SourceBuffer.GetData();
			float* HpfInputBuffer = PreDistanceAttenBufferPtr; // assume bypassing LPF (HPF uses input buffer as input)

			if (!bBypassLPF)
			{
				// Not bypassing LPF, so tell HPF to use LPF output buffer as input
				HpfInputBuffer = SourceBuffer;

				// process LPF audio block
				SourceInfo.LowPassFilter.ProcessAudioBuffer(PreDistanceAttenBufferPtr, SourceBuffer, NumOutputSamplesThisSource);
			}

			if (!bBypassHPF)
			{
				// process HPF audio block
				SourceInfo.HighPassFilter.ProcessAudioBuffer(HpfInputBuffer, SourceBuffer, NumOutputSamplesThisSource);
			}

#if UE_AUDIO_PROFILERTRACE_ENABLED
			const bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AudioMixerChannel);
			if (bChannelEnabled)
			{
				float LPFFrequency = MAX_FILTER_FREQUENCY;
				if (!bBypassLPF)
				{
					LPFFrequency = SourceInfo.LowpassModulation.GetValue();
				}

				float HPFFrequency = MIN_FILTER_FREQUENCY;
				if (!bBypassHPF)
				{
					HPFFrequency = SourceInfo.HighpassModulation.GetValue();
				}

				UE_TRACE_LOG(Audio, SoundSourceFilters, AudioMixerChannel)
					<< SoundSourceFilters.DeviceId(MixerDevice->DeviceID)
					<< SoundSourceFilters.Timestamp(FPlatformTime::Cycles64())
					<< SoundSourceFilters.PlayOrder(SourceInfo.PlayOrder)
					<< SoundSourceFilters.ActiveSoundPlayOrder(SourceInfo.ActiveSoundPlayOrder)
					<< SoundSourceFilters.HPFFrequency(HPFFrequency)
					<< SoundSourceFilters.LPFFrequency(LPFFrequency);
				UE_TRACE_LOG(Audio, SoundSourceEnvelope, AudioMixerChannel)
					<< SoundSourceEnvelope.DeviceId(MixerDevice->DeviceID)
					<< SoundSourceEnvelope.Timestamp(FPlatformTime::Cycles64())
					<< SoundSourceEnvelope.PlayOrder(SourceInfo.PlayOrder)
					<< SoundSourceEnvelope.ActiveSoundPlayOrder(SourceInfo.ActiveSoundPlayOrder)
					<< SoundSourceEnvelope.Envelope(SourceInfo.SourceEnvelopeValue);
			}
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

			// We manually reset interpolation to avoid branches in filter code
			SourceInfo.LowPassFilter.StopFrequencyInterpolation();
			SourceInfo.HighPassFilter.StopFrequencyInterpolation();

			if (bBypassLPF && bBypassHPF)
			{
				FMemory::Memcpy(SourceBuffer, PreDistanceAttenBufferPtr, NumSamples * sizeof(float));
			}
		}

		if (SourceInfo.IsRenderingToSubmixes() || SpatialInterfaceInfo.bSpatializationIsExternalSend || SourceInfo.AudioLink.IsValid())
		{
			// Apply distance attenuation
			ApplyDistanceAttenuation(SourceInfo, NumSamples);

			FMixerSourceSubmixOutputBuffer& SourceSubmixOutputBuffer = SourceSubmixOutputBuffers[SourceId];

			// Copy output buffer to Post-Attenuation Bus Send
			SourceInfo.PostDistanceAttenuationBuffer = SourceInfo.SourceBuffer;

			// Send source audio to plugins
			ComputePluginAudio(SourceInfo, SourceSubmixOutputBuffer, SourceId, NumSamples);
		}

		// Check the source effect tails condition
		if (SourceInfo.bIsLastBuffer && SourceInfo.bEffectTailsDone)
		{
			// If we're done and our tails our done, clear everything out
			SourceInfo.CurrentPCMBuffer = nullptr;
		}
	}

	void FMixerSourceManager::ComputeOutputBuffers(const bool bGenerateBuses, const int32 SourceId)
	{
		CSV_SCOPED_TIMING_STAT(Audio, SourceOutputBuffers);
		SCOPE_CYCLE_COUNTER(STAT_AudioMixerSourceOutputBuffers);

		FSourceInfo& SourceInfo = SourceInfos[SourceId];

		// Don't need to compute anything if the source is not playing or paused (it will remain at 0.0 volume)
		// Note that effect chains will still be able to continue to compute audio output. The source output 
		// will simply stop being read from.
		if (!SourceInfo.bIsBusy || !SourceInfo.bIsPlaying || (SourceInfo.bIsDone && SourceInfo.bEffectTailsDone))
		{
			return;
		}

		// If we're in generate buses mode and not a bus, or vice versa, or if we're set to only output audio to buses.
		// If set to output buses, no need to do any panning for the source. The buses will do the panning.
		const bool bIsSourceBus = SourceInfo.AudioBusId != INDEX_NONE;
		if ((bGenerateBuses && !bIsSourceBus) || (!bGenerateBuses && bIsSourceBus) || !SourceInfo.IsRenderingToSubmixes())
		{
			return;
		}

		FMixerSourceSubmixOutputBuffer& SourceSubmixOutputBuffer = SourceSubmixOutputBuffers[SourceId];
		SourceSubmixOutputBuffer.ComputeOutput(SourceInfo.SpatParams);
	}

	void FMixerSourceManager::GenerateSourceAudio(const bool bGenerateBuses, const int32 SourceIdStart, const int32 SourceIdEnd)
	{
		// Buses generate their input buffers independently
		// Get the next block of frames from the source buffers
		for (int32 SourceId = SourceIdStart; SourceId < SourceIdEnd; ++SourceId)
		{
			ComputeSourceBuffer(bGenerateBuses, SourceId);
		}

		// Compute the audio source buffers after their individual effect chain processing
		for (int32 SourceId = SourceIdStart; SourceId < SourceIdEnd; ++SourceId)
		{
			ComputePostSourceEffectBuffer(bGenerateBuses, SourceId);
		}

		// Get the audio for the output buffers
		for (int32 SourceId = SourceIdStart; SourceId < SourceIdEnd; ++SourceId)
		{
			ComputeOutputBuffers(bGenerateBuses, SourceId);
		}
	}

	void FMixerSourceManager::GenerateSourceAudio(const bool bGenerateBuses)
	{
		RenderThreadPhase = bGenerateBuses ?
			ESourceManagerRenderThreadPhase::GenerateSrcAudio_WithBusses :
			ESourceManagerRenderThreadPhase::GenerateSrcAudio_WithoutBusses;
					
		// If there are no buses, don't need to do anything here
		if (bGenerateBuses && !AudioBuses.Num())
		{
			return;
		}

		if (!DisableParallelSourceProcessingCvar)
		{
			auto RenderAll = [this, bGenerateBuses]()
				{
					for (int SourceId = 0; SourceId < NumTotalSources; ++SourceId)
					{
						// Skip launching a task if there's nothing to do for this source
						const FSourceInfo& SourceInfo = SourceInfos[SourceId];
						const bool bIsSourceBus = SourceInfo.AudioBusId != INDEX_NONE;
						if (!SourceInfo.bIsPlaying || !SourceInfo.bIsBusy || bGenerateBuses != bIsSourceBus)
							continue;

						// Add a nested task to render this source
						auto RenderOne = [this, SourceId]()
							{
								RenderSource(SourceId);
							};
						UE::Tasks::AddNested(UE::Tasks::Launch(TEXT("Render Audio Source"), MoveTemp(RenderOne), LowLevelTasks::ETaskPriority::High));
					}
				};

			// Launch a single task to do all the rendering, and block on its completion
			UE::Tasks::Launch(TEXT("Render All Audio Sources"), MoveTemp(RenderAll), LowLevelTasks::ETaskPriority::High).Wait();
		}
		else
		{
			GenerateSourceAudio(bGenerateBuses, 0, NumTotalSources);
		}
	}

	void FMixerSourceManager::MixOutputBuffers(const int32 SourceId, int32 InNumOutputChannels, const float InSendLevel, EMixerSourceSubmixSendStage InSubmixSendStage, uint32 InSubmixId, FAlignedFloatBuffer& ScratchBuffer, FAlignedFloatBuffer& OutWetBuffer)
	{
		if (InSendLevel > 0.0f)
		{
			FSourceInfo& SourceInfo = SourceInfos[SourceId];

			// Don't need to mix into submixes if the source is paused
			if (!SourceInfo.bIsPaused && !SourceInfo.bIsPausedForQuantization && !SourceInfo.bIsDone && SourceInfo.bIsPlaying)
			{
				FMixerSourceSubmixOutputBuffer& SourceSubmixOutputBuffer = SourceSubmixOutputBuffers[SourceId];

				// Find the send matching this submix to check for per-send filter state.
				// SubmixId is cached on the send to avoid atomic Pin() calls on every buffer tick.
				FMixerSourceSubmixSend* MatchingSend = nullptr;
				for (FMixerSourceSubmixSend& Send : SourceInfo.SubmixSends)
				{
					if (Send.SubmixId == InSubmixId)
					{
						MatchingSend = &Send;
						break;
					}
				}

				if (MatchingSend && MatchingSend->Filter.IsFiltering())
				{
					SourceSubmixOutputBuffer.MixOutput(InSendLevel, InSubmixSendStage, MatchingSend->Filter, MatchingSend->Lpf, MatchingSend->Hpf, ScratchBuffer, OutWetBuffer);
				}
				else
				{
					SourceSubmixOutputBuffer.MixOutput(InSendLevel, InSubmixSendStage, OutWetBuffer);
				}
			}
		}
	}

	void FMixerSourceManager::RenderSource(const int32 SourceId)
	{
		const bool bIsBus = SourceInfos[SourceId].AudioBusId != INDEX_NONE;
		ComputeSourceBuffer(bIsBus, SourceId);
		ComputePostSourceEffectBuffer(bIsBus, SourceId);
		ComputeOutputBuffers(bIsBus, SourceId);
	}

	void FMixerSourceManager::Get2DChannelMap(const int32 SourceId, int32 InNumOutputChannels, Audio::FAlignedFloatBuffer& OutChannelMap)
	{
		// This check had to be disabled for now. It is safe to call this both directly from the render thread and also from render tasks.
		// AUDIO_MIXER_CHECK_AUDIO_PLAT_THREAD(MixerDevice);

		const FSourceInfo& SourceInfo = SourceInfos[SourceId];
		MixerDevice->Get2DChannelMap(SourceInfo.bIsVorbis, SourceInfo.NumInputChannels, InNumOutputChannels, SourceInfo.bIsCenterChannelOnly, OutChannelMap);
	}

	const ISoundfieldAudioPacket* FMixerSourceManager::GetEncodedOutput(const int32 SourceId, const FSoundfieldEncodingKey& InKey) const
	{
		AUDIO_MIXER_CHECK_AUDIO_PLAT_THREAD(MixerDevice);

		const FSourceInfo& SourceInfo = SourceInfos[SourceId];

		// Don't need to mix into submixes if the source is paused
		if (!SourceInfo.bIsPaused && !SourceInfo.bIsPausedForQuantization && !SourceInfo.bIsDone && SourceInfo.bIsPlaying)
		{
			const FMixerSourceSubmixOutputBuffer& SourceSubmixOutputBuffer = SourceSubmixOutputBuffers[SourceId];
			return SourceSubmixOutputBuffer.GetSoundfieldPacket(InKey);
		}

		return nullptr;
	}

	const FQuat FMixerSourceManager::GetListenerRotation(const int32 SourceId) const
	{
		const FMixerSourceSubmixOutputBuffer& SubmixOutputBuffer = SourceSubmixOutputBuffers[SourceId];
		return SubmixOutputBuffer.GetListenerRotation();
	}

	void FMixerSourceManager::UpdateDeviceChannelCount(const int32 InNumOutputChannels)
	{
		AudioMixerThreadCommand([this, InNumOutputChannels]()
		{
			NumOutputSamples = NumOutputFrames * MixerDevice->GetNumDeviceChannels();

			// Update all source's to appropriate channel maps
			for (int32 SourceId = 0; SourceId < NumTotalSources; ++SourceId)
			{
				FSourceInfo& SourceInfo = SourceInfos[SourceId];

				// Don't need to do anything if it's not active or not paused. 
				if (!SourceInfo.bIsActive && !SourceInfo.bIsPaused)
				{
					continue;
				}

				FMixerSourceSubmixOutputBuffer& SourceSubmixOutputBuffer = SourceSubmixOutputBuffers[SourceId];
				SourceSubmixOutputBuffer.SetNumOutputChannels(InNumOutputChannels);

				SourceInfo.ScratchChannelMap.Reset();
				const int32 NumSourceChannels = SourceInfo.bUseHRTFSpatializer ? 2 : SourceInfo.NumInputChannels;

				// If this is a 3d source, then just zero out the channel map, it'll cause a temporary blip
				// but it should reset in the next tick
				if (SourceInfo.bIs3D)
				{
					if (SourceId < NumLegacySources)
					{
						GameThreadInfo.bNeedsSpeakerMap[SourceId] = true;
					}
					SourceInfo.ScratchChannelMap.AddZeroed(NumSourceChannels * InNumOutputChannels);
				}
				// If it's a 2D sound, then just get a new channel map appropriate for the new device channel count
				else
				{
					SourceInfo.ScratchChannelMap.Reset();
					MixerDevice->Get2DChannelMap(SourceInfo.bIsVorbis, NumSourceChannels, InNumOutputChannels, SourceInfo.bIsCenterChannelOnly, SourceInfo.ScratchChannelMap);
				}

				SourceSubmixOutputBuffer.SetChannelMap(SourceInfo.ScratchChannelMap, SourceInfo.bIsCenterChannelOnly);
			}
		}, AUDIO_MIXER_THREAD_COMMAND_STRING("UpdateDeviceChannelCount()"));
	}

	void FMixerSourceManager::UpdateSourceEffectChain(const uint32 InSourceEffectChainId, const TArray<FSourceEffectChainEntry>& InSourceEffectChain, const bool bPlayEffectChainTails)
	{
		AudioMixerThreadCommand([this, InSourceEffectChainId, InSourceEffectChain, bPlayEffectChainTails]()
		{
			FSoundEffectSourceInitData InitData;
			InitData.AudioClock = MixerDevice->GetAudioClock();
			InitData.SampleRate = MixerDevice->SampleRate;
			InitData.AudioDeviceId = MixerDevice->DeviceID;

			for (int32 SourceId = 0; SourceId < NumTotalSources; ++SourceId)
			{
				FSourceInfo& SourceInfo = SourceInfos[SourceId];

				if (SourceInfo.SourceEffectChainId == InSourceEffectChainId)
				{
					SourceInfo.bEffectTailsDone = !bPlayEffectChainTails;

					// Check to see if the chain didn't actually change
					FScopeLock ScopeLock(&EffectChainMutationCriticalSection);
					{
						TArray<TSoundEffectSourcePtr>& ThisSourceEffectChain = SourceInfo.SourceEffects;
						bool bReset = false;
						if (InSourceEffectChain.Num() == ThisSourceEffectChain.Num())
						{
							for (int32 SourceEffectId = 0; SourceEffectId < ThisSourceEffectChain.Num(); ++SourceEffectId)
							{
								const FSourceEffectChainEntry& ChainEntry = InSourceEffectChain[SourceEffectId];

								TSoundEffectSourcePtr SourceEffectInstance = ThisSourceEffectChain[SourceEffectId];
								if (!SourceEffectInstance->IsPreset(ChainEntry.Preset))
								{
									// As soon as one of the effects change or is not the same, then we need to rebuild the effect graph
									bReset = true;
									break;
								}

								// Otherwise just update if it's just to bypass
								SourceEffectInstance->SetEnabled(!ChainEntry.bBypass);
							}
						}
						else
						{
							bReset = true;
						}

						if (bReset)
						{
							InitData.NumSourceChannels = SourceInfo.NumInputChannels;

							// First reset the source effect chain
							ResetSourceEffectChain(SourceId);

							// Rebuild it
							TArray<TSoundEffectSourcePtr> SourceEffects;
							BuildSourceEffectChain(SourceId, InitData, InSourceEffectChain, SourceEffects);

							SourceInfo.SourceEffects = SourceEffects;
							SourceInfo.SourceEffectPresets.Add(nullptr);
						}
					}
				}
			}
		}, AUDIO_MIXER_THREAD_COMMAND_STRING("UpdateSourceEffectChain()"), /*bDeferExecution*/true);
	}

	void FMixerSourceManager::PauseSoundForQuantizationCommand(const int32 SourceId)
	{
		AUDIO_MIXER_CHECK(SourceId < NumLegacySources);
		AUDIO_MIXER_CHECK_AUDIO_PLAT_THREAD(MixerDevice);

		FSourceInfo& SourceInfo = SourceInfos[SourceId];

		SourceInfo.bIsPausedForQuantization = true;
		SourceInfo.bIsActive = false;
	}

	void FMixerSourceManager::SetSubBufferDelayForSound(const int32 SourceId, const int32 FramesToDelay)
	{
		AUDIO_MIXER_CHECK(SourceId < NumLegacySources);
		AUDIO_MIXER_CHECK_AUDIO_PLAT_THREAD(MixerDevice);

		FSourceInfo& SourceInfo = SourceInfos[SourceId];

		SourceInfo.SubCallbackDelayLengthInFrames = FramesToDelay;
	}

	void FMixerSourceManager::UnPauseSoundForQuantizationCommand(const int32 SourceId)
	{
		AUDIO_MIXER_CHECK(SourceId < NumLegacySources);
		AUDIO_MIXER_CHECK_AUDIO_PLAT_THREAD(MixerDevice);

		FSourceInfo& SourceInfo = SourceInfos[SourceId];

		SourceInfo.bIsPausedForQuantization = false;
		SourceInfo.bIsActive = !SourceInfo.bIsPaused;

		SourceInfo.QuantizedCommandHandle.Reset();
	}

	const float* FMixerSourceManager::GetPreDistanceAttenuationBuffer(const int32 SourceId) const
	{
		const FSourceInfo& SourceInfo = SourceInfos[SourceId];

		if (SourceInfo.bIsPaused || SourceInfo.bIsPausedForQuantization)
		{
			return nullptr;
		}

		return SourceInfo.PreDistanceAttenuationBuffer.GetData();
	}

	const float* FMixerSourceManager::GetPostDistanceAttenuationBuffer(const int32 SourceId) const
	{
		const FSourceInfo& SourceInfo = SourceInfos[SourceId];

		if (SourceInfo.bIsPaused || SourceInfo.bIsPausedForQuantization)
		{
			return nullptr;
		}

		return SourceInfo.PostDistanceAttenuationBuffer.GetData();
	}

	const float* FMixerSourceManager::GetPreEffectBuffer(const int32 SourceId) const
	{
		const FSourceInfo& SourceInfo = SourceInfos[SourceId];

		if (SourceInfo.bIsPaused || SourceInfo.bIsPausedForQuantization)
		{
			return nullptr;
		}
		
		return SourceInfo.PreEffectBuffer.GetData();
	}

	const float* FMixerSourceManager::GetPreviousSourceBusBuffer(const int32 SourceId) const
	{
		if (SourceId < SourceInfos.Num())
		{
			return GetPreviousAudioBusBuffer(SourceInfos[SourceId].AudioBusId);
		}
		return nullptr;
	}

	const float* FMixerSourceManager::GetPreviousAudioBusBuffer(const int32 AudioBusId) const
	{
		// This is only called from within a scope-lock
		const TSharedPtr<FMixerAudioBus> AudioBusPtr = AudioBuses.FindRef(AudioBusId);
		if (AudioBusPtr.IsValid())
		{
			return AudioBusPtr->GetPreviousBusBuffer();
		}
		return nullptr;
	}

	int32 FMixerSourceManager::GetNumChannels(const int32 SourceId) const
	{
		return SourceInfos[SourceId].NumInputChannels;
	}

	bool FMixerSourceManager::IsSourceBus(const int32 SourceId) const
	{
		return SourceInfos[SourceId].AudioBusId != INDEX_NONE;
	}

	void FMixerSourceManager::ComputeNextBlockOfSamples()
	{
		AUDIO_MIXER_CHECK_AUDIO_PLAT_THREAD(MixerDevice);

		CSV_SCOPED_TIMING_STAT(Audio, SourceManagerUpdate);
		SCOPE_CYCLE_COUNTER(STAT_AudioMixerSourceManagerUpdate);
		CSV_CUSTOM_STAT(Audio, NumActiveSources, NumActiveSources, ECsvCustomStatOp::Set);

		RenderThreadPhase = ESourceManagerRenderThreadPhase::Begin;

		if (FPlatformProcess::SupportsMultithreading())
		{
			// Get the this blocks commands before rendering audio
			PumpCommandQueue();
		}
		else if (bPumpQueue)
		{
			bPumpQueue = false;
			PumpCommandQueue();
		}

		// Notify modulation interface that we are beginning to update
		RenderThreadPhase = ESourceManagerRenderThreadPhase::ProcessModulators;
		if (MixerDevice->IsModulationPluginEnabled() && MixerDevice->ModulationInterface.IsValid())
		{
			MixerDevice->ModulationInterface->ProcessModulators(MixerDevice->GetAudioClockDelta());
		}

		// Update pending tasks and release them if they're finished
		UpdatePendingReleaseData();

		if (UseRenderScheduler())
		{
			ConnectBusPatches();

			RenderThreadPhase = ESourceManagerRenderThreadPhase::RunScheduledRenderSteps;
			GetRenderScheduler().RenderBlock(DisableParallelSourceProcessingCvar != 0);
		}
		else
		{
			// First generate non-bus audio (bGenerateBuses = false)
			GenerateSourceAudio(false);

			// Now mix in the non-bus audio into the buses
			ComputeBuses();

			// Now generate bus audio (bGenerateBuses = true)
			GenerateSourceAudio(true);
		}

		// Update the buses now
		UpdateBuses();

		// Let the plugin know we finished processing all sources
		if (bUsingSpatializationPlugin)
		{
			RenderThreadPhase = ESourceManagerRenderThreadPhase::SpatialInterface_OnAllSourcesProcessed;
			AUDIO_MIXER_CHECK(SpatialInterfaceInfo.SpatializationPlugin.IsValid());
			LLM_SCOPE(ELLMTag::AudioMixerPlugins);
			SpatialInterfaceInfo.SpatializationPlugin->OnAllSourcesProcessed();
		}

		// Let the plugin know we finished processing all sources
		if (bUsingSourceDataOverridePlugin)
		{
			RenderThreadPhase = ESourceManagerRenderThreadPhase::SourceDataOverride_OnAllSourcesProcessed;
			AUDIO_MIXER_CHECK(SourceDataOverridePlugin.IsValid());
			LLM_SCOPE(ELLMTag::AudioMixerPlugins);
			SourceDataOverridePlugin->OnAllSourcesProcessed();
		}		
	}

	void FMixerSourceManager::UpdateSourceState()
	{
		RenderThreadPhase = ESourceManagerRenderThreadPhase::UpdateGameThreadCopies;
		for (int32 SourceId = 0; SourceId < NumTotalSources; ++SourceId)
		{		
			FSourceInfo& SourceInfo = SourceInfos[SourceId];

			// Check for the stopping condition to "turn the sound off"
			if (SourceInfo.bIsLastBuffer)
			{
				// since we started with a delay, give the sound one more callback to finish rendering the delayline
				if (SourceInfo.SubCallbackDelayLengthInFrames > 0) 
				{
					SourceInfo.SubCallbackDelayLengthInFrames = 0;
					continue;
				}
				
				if (!SourceInfo.bIsDone)
				{
					SetSourceDone(SourceId);
				}
			}
		}
		RenderThreadPhase = ESourceManagerRenderThreadPhase::Finished;
	}

	void FMixerSourceManager::ClearStoppingSounds()
	{
		for (int32 SourceId = 0; SourceId < NumTotalSources; ++SourceId)
		{
			FSourceInfo& SourceInfo = SourceInfos[SourceId];

			if (!SourceInfo.bIsDone && SourceInfo.bIsStopping && SourceInfo.VolumeSourceDestination == 0.0f)
			{
				SourceInfo.bIsStopping = false;
				SetSourceDone(SourceId);
			}
		}
	}

	void FMixerSourceManager::SetSourceDone(const int32 SourceId)
	{
		FSourceInfo& SourceInfo = SourceInfos[SourceId];

		SourceInfo.bIsDone = true;

		// Notify that we're now done with this source
		if (SourceInfo.SourceListener)
		{
			SourceInfo.SourceListener->OnDone();
		}

		if (SourceInfo.AudioLink)
		{
			SourceInfo.AudioLink->OnSourceDone(SourceId);
		}

		if (SourceId >= NumLegacySources)
		{
			FGeneratorSourceState& State = GeneratorSourceState[SourceId - NumLegacySources];
			if (State.Generator.IsValid())
			{
				State.Generator->OnEndGenerate();
				State.Generator.Reset();
			}
			State.bSourceBusy = false;
		}
	}

	void FMixerSourceManager::AudioMixerThreadMPSCCommand(TFunction<void()>&& InCommand, const char* InDebugString)
	{
		MpscCommandQueue.Enqueue( FAudioMixerMpscCommand{ MoveTemp(InCommand), InDebugString, false });
	}

	void FMixerSourceManager::AudioMixerThreadCommand(TFunction<void()>&& InFunction, const char* InDebugString, bool bInDeferExecution /*= false*/)
	{
		FAudioMixerThreadCommand AudioCommand(MoveTemp(InFunction), InDebugString, bInDeferExecution);

		// collect values for debugging
		// outside of the ScopeLock so we can avoid doing a bunch of work that doesn't require the lock
		SIZE_T OldMax = 0;
		SIZE_T NewMax = 0;
		SIZE_T NewNum = 0;
		SIZE_T CurrentBufferSizeInBytes = 0;
		int32 AudioThreadCommandIndex = -1;
		{
			// Here, we make sure that we don't flip our command double buffer while modifying the command buffer
			FScopeLock ScopeLock(&CommandBufferIndexCriticalSection);
			AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

			// Add the function to the command queue:
			AudioThreadCommandIndex = !RenderThreadCommandBufferIndex.GetValue();
			FCommands& Commands = CommandBuffers[AudioThreadCommandIndex];

			OldMax = Commands.SourceCommandQueue.Max();
			
			// always add commands to the buffer. If we're not going to assert, might as well chug along and hope we can recover!
			Commands.SourceCommandQueue.Add(AudioCommand);
			NumCommands.Increment();
			TRACE_COUNTER_INCREMENT(AudioMixerSourceManager_TotalQdCmds);
#if !UE_BUILD_SHIPPING			
			if (LogCmdQueueWhenNumberReachedCVar > 0 && LogCmdQueueWhenNumberReachedCVar == Commands.SourceCommandQueue.Num())
			{
				TMap<const char*, int32> CmdsCounts;
				for (const FAudioMixerThreadCommand& i : Commands.SourceCommandQueue)
				{
					CmdsCounts.FindOrAdd(i.DebugString)++;
				}
				for (auto i : CmdsCounts)
				{
					UE_LOGF(LogAudioMixer, Display, "Commands %ls:%d", StringCast<TCHAR>(i.Key).Get(), i.Value);
				}
			}
#endif //!UE_BUILD_SHIPPING						

			NewNum = Commands.SourceCommandQueue.Num();
			NewMax = Commands.SourceCommandQueue.Max();
			CurrentBufferSizeInBytes = Commands.SourceCommandQueue.GetAllocatedSize();
		}
		
		// log warnings for command buffer growing too large
		if (OldMax != NewMax)
		{
			// Only throw a warning every time we have to reallocate, which will be less often then every single time we add
			static SIZE_T WarnSize = 1024 * 1024;
			if (CurrentBufferSizeInBytes > WarnSize )
			{
				float TimeSinceLastComplete = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - LastPumpCompleteTimeInCycles);

				UE_LOG(LogAudioMixer, Error, TEXT("Command Queue %d has grown to %" SIZE_T_FMT "kb, containing %" SIZE_T_FMT " cmds, last complete pump was %2.5f seconds ago."),
					AudioThreadCommandIndex, CurrentBufferSizeInBytes >> 10, NewNum, TimeSinceLastComplete);
				WarnSize *= 2;

				DoStallDiagnostics();
			}
			
			// check that we haven't gone over the max size
			// if so, set a flag that the AudioMixerDevice can check and try swapping to safe hardware
			const SIZE_T MaxBufferSizeInBytes = ((SIZE_T)CommandBufferMaxSizeInMbCvar) << 20;
			if (CurrentBufferSizeInBytes >= MaxBufferSizeInBytes)
			{
				int32 NumTimesOvergrown = CommandBuffers[AudioThreadCommandIndex].NumTimesOvergrown.Increment();
				UE_LOG(LogAudioMixer, Error, TEXT("%d: Command buffer %d allocated size has grown to %" SIZE_T_FMT "mb! Likely cause the AudioRenderer has hung"),
					NumTimesOvergrown, AudioThreadCommandIndex, CurrentBufferSizeInBytes >> 20);

				bShouldResetHardwareCallback = true;
			}
		}

		// update trace values
		CSV_CUSTOM_STAT(Audio, AudioMixerThreadCommands, static_cast<int32>(NewNum), ECsvCustomStatOp::Set);
		TRACE_INT_VALUE(TEXT("AudioMixerThreadCommands::NumCommands"), NewNum);
		TRACE_INT_VALUE(TEXT("AudioMixerThreadCommands::CurrentBufferSizeInKb"), CurrentBufferSizeInBytes >> 10);
	}


	void FMixerSourceManager::PumpCommandQueue()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(AudioMixerThreadCommands::PumpCommandQueue)
		AudioRenderThreadId = FPlatformTLS::GetCurrentThreadId();

		// If we're already triggered, we need to wait for the audio thread to reset it before pumping
		if (FPlatformProcess::SupportsMultithreading())
		{
			if (CommandsProcessedEvent->Wait(0))
			{
				return;
			}
		}

		// Pump the MPSC command queue
		RenderThreadPhase = ESourceManagerRenderThreadPhase::PumpMpscCmds;
		TOptional Opt{ MpscCommandQueue.Dequeue() };
		while (Opt.IsSet())
		{
			// First copy/move out the command and keep a copy of it.
			{
				FWriteScopeLock Lock(CurrentlyExecutingCmdLock);
				CurrentlyExecuteingCmd = MoveTemp(Opt.GetValue());
			}
			
			// Execute the current under a read-lock.
			{
				FReadScopeLock Lock(CurrentlyExecutingCmdLock);
				CurrentlyExecuteingCmd();
			}
				
			Opt = MpscCommandQueue.Dequeue();
		}

		int32 CurrentRenderThreadIndex = RenderThreadCommandBufferIndex.GetValue();
		FCommands& Commands = CommandBuffers[CurrentRenderThreadIndex];

		const int32 NumCommandsToExecute = Commands.SourceCommandQueue.Num();
		TRACE_INT_VALUE(TEXT("AudioMixerThreadCommands::NumCommandsToExecute"), NumCommandsToExecute);

		// Pop and execute all the commands that came since last update tick
		TArray<FAudioMixerThreadCommand> DelayedCommands;
		RenderThreadPhase = ESourceManagerRenderThreadPhase::PumpCmds;
		for (int32 Id = 0; Id < NumCommandsToExecute; ++Id)
		{
			// First copy/move out the command and keep a copy of it.
			{ 
				FWriteScopeLock Lock(CurrentlyExecutingCmdLock);
				CurrentlyExecuteingCmd = MoveTemp(Commands.SourceCommandQueue[Id]);
			}
			
			// Execute the current command or differ under a read-lock.
			{
				FReadScopeLock Lock(CurrentlyExecutingCmdLock);
				if (CurrentlyExecuteingCmd.bDeferExecution)
				{
					CurrentlyExecuteingCmd.bDeferExecution = false;
					DelayedCommands.Add(CurrentlyExecuteingCmd);
				}
				else
				{
					CurrentlyExecuteingCmd(); // execute
				}
			}
			
			NumCommands.Decrement();
			TRACE_COUNTER_DECREMENT(AudioMixerSourceManager_TotalQdCmds);
		}

		LastPumpCompleteTimeInCycles = FPlatformTime::Cycles64();
		// This is intentionally re-assigning the Command Queue and clearing the buffer in the process
		Commands.SourceCommandQueue = MoveTemp(DelayedCommands);
		Commands.SourceCommandQueue.Reserve(GetCommandBufferInitialCapacity());

		if (FPlatformProcess::SupportsMultithreading())
		{
			check(CommandsProcessedEvent != nullptr);
			CommandsProcessedEvent->Trigger();
		}
		else
		{
			RenderThreadCommandBufferIndex.Set(!CurrentRenderThreadIndex);
		}
	}

	void FMixerSourceManager::FlushCommandQueue(bool bPumpInCommand, bool bSkipWaitForRenderThread)
	{
		check(CommandsProcessedEvent != nullptr);

		// If we have no commands enqueued, exit
		if (NumCommands.GetValue() == 0)
		{
			UE_LOGF(LogAudioMixer, Verbose, "No commands were queued while flushing the source manager.");
			return;
		}

		bool bTimedOut = false;

		if (bSkipWaitForRenderThread)
		{
			// The render thread is known to be stopped (e.g. during device swap).
			// Skip waiting on CommandsProcessedEvent since no thread will signal it.
			// We'll pump commands inline below.
			CommandsProcessedEvent->Trigger();
			bTimedOut = true;
		}
		// Make sure render thread is still executing
		else if (!CommandsProcessedEvent->Wait(CommandBufferFlushWaitTimeMsCvar))
		{
			CommandsProcessedEvent->Trigger();
			bTimedOut = true;
			UE_LOGF(LogAudioMixer, Warning, "Timed out waiting to flush the source manager command queue (1).");
		}
		else
		{
			UE_LOGF(LogAudioMixer, Verbose, "Flush succeeded in the source manager command queue (1).");
		}

		// Call update to trigger a final pump of commands
		Update(bTimedOut);

		if (bPumpInCommand)
		{
			PumpCommandQueue();
		}

		if (bSkipWaitForRenderThread)
		{
			// Already pumped inline, no second wait needed.
			CommandsProcessedEvent->Trigger();
		}
		// wait one more time for the double pump
		else if (!CommandsProcessedEvent->Wait(1000))
		{
			CommandsProcessedEvent->Trigger();
			UE_LOGF(LogAudioMixer, Warning, "Timed out waiting to flush the source manager command queue (2).");
		}
		else
		{
			UE_LOGF(LogAudioMixer, Verbose, "Flush succeeded the source manager command queue (2).");
		}
	}

	void FMixerSourceManager::UpdatePendingReleaseData(bool bForceWait)
	{
		RenderThreadPhase = ESourceManagerRenderThreadPhase::UpdatePendingReleaseData;
		
		// Don't block, but let tasks finish naturally
		for (int32 i = PendingSourceBuffers.Num() - 1; i >= 0; --i)
		{
			FMixerSourceBuffer* MixerSourceBuffer = PendingSourceBuffers[i].Get();

			bool bDeleteSourceBuffer = true;
			if (bForceWait)
			{
				MixerSourceBuffer->EnsureAsyncTaskFinishes();
			}
			else if (!MixerSourceBuffer->IsAsyncTaskDone())
			{			
				bDeleteSourceBuffer = false;
			}

			if (bDeleteSourceBuffer)
			{
				PendingSourceBuffers.RemoveAtSwap(i, EAllowShrinking::No);
			}
		}
	}

	bool FMixerSourceManager::FSourceInfo::IsRenderingToSubmixes() const
	{
		return bEnableBaseSubmix || bEnableSubmixSends || bHasAttenuationSubmixSends;
	}

	void FMixerSourceManager::DoStallDiagnostics()
	{
		LogRenderThreadStall();
		LogInflightAsyncTasks();
		if (bDumpCallstackOnStallDiagnostics)
		{
			LogCallstacks();
		}
	}

	void FMixerSourceManager::LogRenderThreadStall()
	{
		// If we are in either of the Cmd pump phases dump the current command.
		if (RenderThreadPhase == ESourceManagerRenderThreadPhase::PumpMpscCmds ||
			RenderThreadPhase == ESourceManagerRenderThreadPhase::PumpCmds)
		{
			FReadScopeLock Lock(CurrentlyExecutingCmdLock);
			UE_LOGF(LogAudioMixer, Warning, "Stall in Cmd Queue: Cmd='%ls', Executing For: %2.5f secs, AudioRenderThread='%ls'",
				*CurrentlyExecuteingCmd.GetSafeDebugString(), CurrentlyExecuteingCmd.GetExecuteTimeInSeconds(), ToCStr(LexToString(RenderThreadPhase)));
		}
		else
		{
			UE_LOGF(LogAudioMixer, Warning, "Stall in AudioRenderThread Phase: '%ls'", ToCStr(LexToString(RenderThreadPhase)));
		}
	}

	void FMixerSourceManager::LogInflightAsyncTasks()
	{
		// NOTE: we iterate these lists without a lock, so this is somewhat dangerous!

		// Dump all in flight decodes/procedural sources.
		using FSrcBuffer = TSharedPtr<FMixerSourceBuffer, ESPMode::ThreadSafe>;
		TArray<FMixerSourceBuffer::FDiagnosticState> InflightTasks;
		for (FSourceInfo& i : SourceInfos)
		{
			if (i.MixerSourceBuffer.IsValid())
			{
				FMixerSourceBuffer::FDiagnosticState State;
				i.MixerSourceBuffer->GetDiagnosticState(State);
				if (State.bInFlight)
				{
					InflightTasks.Add(State);
				}
			}
		}
		for (FSrcBuffer& i : PendingSourceBuffers)
		{
			FMixerSourceBuffer::FDiagnosticState State;
			if (i.IsValid())
			{
				i->GetDiagnosticState(State);
				if (State.bInFlight)
				{
					InflightTasks.Add(State);
				}
			}
		}
		for (FMixerSourceBuffer::FDiagnosticState& i : InflightTasks)
		{
			UE_LOGF(LogAudioMixer, Warning, "Inflight Task: %ls, %2.2f secs, Procedural=%d",
				*i.WaveName.ToString(), i.RunTimeInSecs, (int32)i.bProcedural);
		}
	}

	void FMixerSourceManager::LogCallstacks()
	{
		LogCallstack(AudioRenderThreadId);
	}

	TSharedRef<FMixerAudioBus> FMixerSourceManager::FindOrAddAudioBus(const FAudioBusKey BusKey, const int32 InNumChannels, const bool bIsAutomatic, bool* bOutAlreadyExisted)
	{
		TSharedPtr<FMixerAudioBus> AudioBusPtr = AudioBuses.FindRef(BusKey);
		if (bOutAlreadyExisted != nullptr)
		{
			*bOutAlreadyExisted = AudioBusPtr.IsValid();
		}
		if (!AudioBusPtr.IsValid())
		{
			AudioBusPtr = MakeShared<FMixerAudioBus>(this, bIsAutomatic, InNumChannels, BusKey);
			AudioBuses.Add(BusKey, AudioBusPtr);
		}

		return AudioBusPtr.ToSharedRef();
	}

	void FMixerSourceManager::LogCallstack(uint32 InThreadId)
	{
		if (InThreadId != INVALID_AUDIO_RENDER_THREAD_ID)
		{
			const SIZE_T StackTraceSize = 65536;
			ANSICHAR StackTrace[StackTraceSize] = { 0 };
			FPlatformStackWalk::ThreadStackWalkAndDump(StackTrace, StackTraceSize, 0, InThreadId);
			UE_LOGF(LogAudioMixer, Warning, "***** ThreadStackWalkAndDump for ThreadId(%lu) ******\n%ls", InThreadId, ANSI_TO_TCHAR(StackTrace));
		}
	}

	int32 FMixerSourceManager::AllocateGeneratorSource(ISoundGeneratorRef InGenerator)
	{
		auto IsFree = [this](const FGeneratorSourceState& State)
			{
				return !State.bAllocated && !State.bSourceBusy;
			};
		const int32 GeneratorSourceIndex = GeneratorSourceState.IndexOfByPredicate(IsFree);
		if (GeneratorSourceIndex != INDEX_NONE)
		{
			GeneratorSourceState[GeneratorSourceIndex].bAllocated = true;
			GeneratorSourceState[GeneratorSourceIndex].bSourceBusy = true;
			const int32 SourceId = GeneratorSourceIndex + NumLegacySources;
			check(SourceId >= NumLegacySources && SourceId < NumTotalSources);
			AudioMixerThreadMPSCCommand([this, SourceId, InGenerator]
				{
					static int32 PlayOrder = 1 << 31;

					FSourceInfo& SourceInfo = SourceInfos[SourceId];
					SourceInfo.bIsBusy = true;
					SourceInfo.NumInputChannels = InGenerator->GetNumChannels();
					SourceInfo.PlayOrder = PlayOrder++;

					check(SourceInfo.NumInputChannels > 0);
					GeneratorSourceState[SourceId - NumLegacySources].Generator = InGenerator;
					InGenerator->OnBeginGenerate();

					// Initialize the submix output source for this source id
					FMixerSourceSubmixOutputBufferSettings SourceSubmixOutputResetSettings = {};
					SourceSubmixOutputResetSettings.NumOutputChannels = MixerDevice->GetDeviceOutputChannels();
					SourceSubmixOutputResetSettings.NumSourceChannels = SourceInfo.NumInputChannels;
					SourceSubmixOutputBuffers[SourceId].Reset(SourceSubmixOutputResetSettings);

					MixerDevice->Get2DChannelMap(SourceInfo.bIsVorbis, SourceSubmixOutputResetSettings.NumSourceChannels, SourceSubmixOutputResetSettings.NumOutputChannels, SourceInfo.bIsCenterChannelOnly, SourceInfo.ScratchChannelMap);
					SourceSubmixOutputBuffers[SourceId].SetChannelMap(SourceInfo.ScratchChannelMap, SourceInfo.bIsCenterChannelOnly);

					FMixerSubmixPtr SubmixPtr = MixerDevice->GetMasterSubmix().Pin();
					if (SubmixPtr.IsValid())
					{
						SubmixPtr->AddOrSetSourceById(SourceId, 1.0f, EMixerSourceSubmixSendStage::PostDistanceAttenuation);
					}

					if (UseRenderScheduler())
					{
						GetRenderScheduler().AddStep(FAudioRenderStepId::FromWaveInstancePlayOrder(SourceInfo.PlayOrder), &SourceInfo);
					}
				}
			);

			return SourceId;
		}
		else
		{
			return INDEX_NONE;
		}
	}

	void FMixerSourceManager::FreeGeneratorSource(const int32 SourceId)
	{
		const int32 GeneratorSourceIndex = SourceId - NumLegacySources;
		check(SourceId >= NumLegacySources && SourceId < NumTotalSources && GeneratorSourceState[GeneratorSourceIndex].bAllocated);
		GeneratorSourceState[GeneratorSourceIndex].bAllocated = false;
		AudioMixerThreadMPSCCommand([this, SourceId, GeneratorSourceIndex]
			{
				FMixerSubmixPtr SubmixPtr = MixerDevice->GetMasterSubmix().Pin();
				if (SubmixPtr.IsValid())
				{
					SubmixPtr->RemoveSourceById(SourceId);
				}
				ISoundGeneratorPtr& Generator = GeneratorSourceState[GeneratorSourceIndex].Generator;
				if (GeneratorSourceState[GeneratorSourceIndex].bSourceBusy && ensure(Generator.IsValid()))
				{
					Generator->OnEndGenerate();
					Generator.Reset();
				}
				ReleaseSource(SourceId);
				GeneratorSourceState[GeneratorSourceIndex].bSourceBusy = false;
			}
		);
	}

	void FMixerSourceManager::PlayGeneratorSource(const int32 SourceId)
	{
		check(SourceId >= NumLegacySources && SourceId < NumTotalSources && GeneratorSourceState[SourceId - NumLegacySources].bAllocated);
		auto Execute = [this, SourceId]()
			{
				FSourceInfo& SourceInfo = SourceInfos[SourceId];
				SourceInfo.bIsDone = false;
				SourceInfo.bIsPlaying = true;
				SourceInfo.bIsPaused = false;
				SourceInfo.bIsActive = true;
				SourceInfo.bEnableBaseSubmix = true;
			};

		if (MixerDevice->IsAudioRenderingThread())
		{
			Execute();
		}
		else
		{
			AudioMixerThreadMPSCCommand(MoveTemp(Execute));
		}
	}

	void FMixerSourceManager::StopGeneratorSource(const int32 SourceId)
	{
		check(SourceId >= NumLegacySources && SourceId < NumTotalSources && GeneratorSourceState[SourceId - NumLegacySources].bAllocated);
		auto Execute = [this, SourceId]()
			{
				FSourceInfo& SourceInfo = SourceInfos[SourceId];
				SourceInfo.bIsPlaying = false;
				SourceInfo.bIsPaused = false;
				SourceInfo.bIsActive = false;
				SourceInfo.bIsStopping = false;
				if (!SourceInfo.bIsDone)
				{
					SetSourceDone(SourceId);
				}
			};

		if (MixerDevice->IsAudioRenderingThread())
		{
			Execute();
		}
		else
		{
			AudioMixerThreadMPSCCommand(MoveTemp(Execute));
		}
	}

	bool FMixerSourceManager::IsGeneratorSourceDone(const int32 SourceId)
	{
		check(SourceId >= NumLegacySources && SourceId < NumTotalSources && GeneratorSourceState[SourceId - NumLegacySources].bAllocated);
		return !GeneratorSourceState[SourceId - NumLegacySources].bSourceBusy;
	}

	void FMixerSourceManager::FSourceInfo::DoRenderStep()
	{
		const int32 SourceId = this - SourceManager->SourceInfos.GetData();
		SourceManager->RenderSource(SourceId);
	}

	const TCHAR* FMixerSourceManager::FSourceInfo::GetRenderStepName()
	{
#if AUDIO_MIXER_ENABLE_DEBUG_MODE
		return *DebugName;
#else
		return TEXT("Render mixer source");
#endif
	}
}
