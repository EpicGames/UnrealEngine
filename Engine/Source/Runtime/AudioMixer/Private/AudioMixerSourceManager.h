// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioBusSubsystem.h"
#include "AudioMixerBuffer.h"
#include "AudioMixerBus.h"
#include "AudioMixerDevice.h"
#include "AudioMixerSourceOutputBuffer.h"
#include "AudioMixerSubmix.h"
#include "AudioMixerTrace.h"
#include "AudioRenderScheduler.h"
#include "Containers/MpscQueue.h"
#include "DSP/BufferVectorOperations.h"
#include "DSP/EnvelopeFollower.h"
#include "DSP/InterpolatedOnePole.h"
#include "DSP/ParamInterpolator.h"
#include "DSP/RuntimeResampler.h"
#include "IAudioExtensionPlugin.h"
#include "IAudioMixerRenderStep.h"
#include "ISoundfieldFormat.h"
#include "Misc/SpinLock.h"
#include "Sound/SoundModulationDestination.h"
#include "Sound/QuartzQuantizationUtilities.h"
#include "Stats/Stats.h"

#if UE_AUDIO_PROFILERTRACE_ENABLED
#include "ProfilingDebugging/TraceAuxiliary.h"
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

#include "AudioMixerSourceManager.generated.h"

// Default this to on (it's quite a small memory footprint).
#ifndef WITH_AUDIO_MIXER_THREAD_COMMAND_DEBUG
	#define WITH_AUDIO_MIXER_THREAD_COMMAND_DEBUG (1)
#endif //WITH_AUDIO_MIXER_THREAD_COMMAND_DEBUG

// Tracks the time it takes to up the source manager (computes source buffers, source effects, sample rate conversion)
DECLARE_CYCLE_STAT_EXTERN(TEXT("Source Manager Update"), STAT_AudioMixerSourceManagerUpdate, STATGROUP_AudioMixer, AUDIOMIXER_API);

// The time it takes to compute the source buffers (handle decoding tasks, resampling)
DECLARE_CYCLE_STAT_EXTERN(TEXT("Source Buffers"), STAT_AudioMixerSourceBuffers, STATGROUP_AudioMixer, AUDIOMIXER_API);

// The time it takes to process the source buffers through their source effects
DECLARE_CYCLE_STAT_EXTERN(TEXT("Source Effect Buffers"), STAT_AudioMixerSourceEffectBuffers, STATGROUP_AudioMixer, AUDIOMIXER_API);

// The time it takes to apply channel maps and get final pre-submix source buffers
DECLARE_CYCLE_STAT_EXTERN(TEXT("Source Output Buffers"), STAT_AudioMixerSourceOutputBuffers, STATGROUP_AudioMixer, AUDIOMIXER_API);

// The time it takes to process the HRTF effect.
DECLARE_CYCLE_STAT_EXTERN(TEXT("HRTF"), STAT_AudioMixerHRTF, STATGROUP_AudioMixer, AUDIOMIXER_API);

// For diagnostics, keep track of what phase of updating the Source manager is in currently.
UENUM()
enum class ESourceManagerRenderThreadPhase: uint8
{
	Begin,
	
	PumpMpscCmds,	
	PumpCmds,
	ProcessModulators,
	UpdatePendingReleaseData,
	RunScheduledRenderSteps,
	GenerateSrcAudio_WithBusses,
	ComputeBusses,
	GenerateSrcAudio_WithoutBusses,
	UpdateBusses,
	SpatialInterface_OnAllSourcesProcessed,
	SourceDataOverride_OnAllSourcesProcessed,
	UpdateGameThreadCopies,
	
	Finished,
};

namespace Audio
{
	class FMixerSubmix;
	class FMixerDevice;
	class FMixerSourceVoice;
	class FMixerSourceBuffer;
	class ISourceListener;
	class FMixerSourceSubmixOutputBuffer;

	class ISourceListener
	{
	public:
		virtual ~ISourceListener() = default;

		// Called before a source begins to generate audio. 
		virtual void OnBeginGenerate() = 0;

		// Called when a loop point is hit
		virtual void OnLoopEnd() = 0;

		// Called when the source finishes on the audio render thread
		virtual void OnDone() = 0;

		// Called when the source's effect tails finish on the audio render thread.
		virtual void OnEffectTailsDone() = 0;

	};

	struct FMixerSourceSubmixSend
	{
		// The submix ptr
		FMixerSubmixWeakPtr Submix;

		// Cached submix ID — avoids atomic Pin() calls in the render-thread hot path
		uint32 SubmixId = TNumericLimits<uint32>::Max();

		// The amount of audio that is to be mixed into this submix
		float SendLevel = 0.0f;

		// Whather or not this is the primary send (i.e. first in the send chain)
		bool bIsMainSend = false;

		// Whether or not this is a pre-distance attenuation send
		EMixerSourceSubmixSendStage SubmixSendStage = EMixerSourceSubmixSendStage::PostDistanceAttenuation;

		// If this is a soundfield submix, this is a pointer to the submix's Soundfield Factory.
		// If this is nullptr, the submix is not a soundfield submix.
		ISoundfieldFactory* SoundfieldFactory = nullptr;

		FSendFilterSettings Filter;

		FInterpolatedLPF Lpf;
		FInterpolatedHPF Hpf;
	};

	// Struct holding mappings of bus ids (unique ids) to send level
	struct FInitAudioBusSend
	{
		uint32 AudioBusId = INDEX_NONE;
		float SendLevel = 0.0f;
		int32 BusChannels = 0;
		FSendFilterSettings Filter;
#if ENABLE_AUDIO_DEBUG
		FString AudioBusName;
#endif // ENABLE_AUDIO_DEBUG
	};

	struct FMixerSourceVoiceInitParams
	{
		TSharedPtr<FMixerSourceBuffer, ESPMode::ThreadSafe> MixerSourceBuffer = nullptr;
		ISourceListener* SourceListener = nullptr;
		TArray<FMixerSourceSubmixSend> SubmixSends;
		TArray<FInitAudioBusSend> AudioBusSends[(int32)EBusSendType::Count];
		uint32 AudioBusId = INDEX_NONE;
		int32 AudioBusChannels = 0;
#if ENABLE_AUDIO_DEBUG
		FString AudioBusName;
#endif // ENABLE_AUDIO_DEBUG
		float SourceBusDuration = 0.0f;
		uint32 SourceEffectChainId = INDEX_NONE;
		TArray<FSourceEffectChainEntry> SourceEffectChain;
		int32 SourceEffectChainMaxSupportedChannels = 0;
		FMixerSourceVoice* SourceVoice = nullptr;
		int32 NumInputChannels = 0;
		int32 NumInputFrames = 0;
		float EnvelopeFollowerAttackTime = 10.0f;
		float EnvelopeFollowerReleaseTime = 100.0f;
		FString DebugName;
		USpatializationPluginSourceSettingsBase* SpatializationPluginSettings = nullptr;
		UOcclusionPluginSourceSettingsBase* OcclusionPluginSettings = nullptr;
		UReverbPluginSourceSettingsBase* ReverbPluginSettings = nullptr;
		USourceDataOverridePluginSourceSettingsBase* SourceDataOverridePluginSettings = nullptr;

		FSoundModulationDefaultSettings ModulationSettings;

		FQuartzQuantizedRequestData QuantizedRequestData;

		FSharedISourceBufferListenerPtr SourceBufferListener;

		IAudioLinkFactory::FAudioLinkSourcePushedSharedPtr AudioLink;

		FName AudioComponentUserID;
		uint64 AudioComponentID = 0;
		bool bIs3D = false;
		bool bPlayEffectChainTails = false;
		bool bUseHRTFSpatialization = false;
		bool bIsExternalSend = false;
		bool bIsDebugMode  = false;
		bool bEnableBusSends = false;
		bool bEnableBaseSubmix = false;
		bool bEnableSubmixSends = false;
		bool bHasAttenuationSubmixSends = false;
		bool bIsVorbis = false;
		bool bIsSoundfield = false;
		bool bIsSeeking = false;
		bool bShouldSourceBufferListenerZeroBuffer = false;

		uint32 PlayOrder = INDEX_NONE;
		uint32 ActiveSoundPlayOrder = INDEX_NONE;
	};

	struct FSourceManagerInitParams
	{
		// Total number of sources to use in the source manager
		int32 NumSources = 0;
		int32 NumGeneratorSources = 0;
	};

	class FMixerSourceManager
	{
		struct FPendingAudioBusConnection;

	public:
		FMixerSourceManager(FMixerDevice* InMixerDevice);
		~FMixerSourceManager();

		void Init(const FSourceManagerInitParams& InitParams);
		void Update(bool bTimedOut = false);

		bool GetFreeSourceId(int32& OutSourceId);
		int32 GetNumActiveSources() const;
		int32 GetNumActiveAudioBuses() const;

		void ReleaseSourceId(const int32 SourceId);
		void InitSource(const int32 SourceId, const FMixerSourceVoiceInitParams& InitParams);

		// Creates and starts an audio bus manually.
		UE_DEPRECATED(5.6, "Use the StartAudioBus version that requires an AudioBus name.")
		void StartAudioBus(FAudioBusKey InAudioBusKey, int32 InNumChannels, bool bInIsAutomatic);

		void StartAudioBus(FAudioBusKey InAudioBusKey, const FString& InAudioBusName, int32 InNumChannels, bool bInIsAutomatic);

		// Stops an audio bus manually
		void StopAudioBus(FAudioBusKey InAudioBusKey);

		// Queries if an audio bus is active. Must be called from the audio thread.
		bool IsAudioBusActive(FAudioBusKey InAudioBusKey) const;

		// Returns the number of channels currently set for the audio bus associated with
		// the provided BusId.  Returns 0 if the audio bus is inactive.
		int32 GetAudioBusNumChannels(FAudioBusKey InAudioBusKey) const;

		// Adds a patch output for an audio bus from the Audio Render Thread
		void AddPatchOutputForAudioBus(FAudioBusKey InAudioBusKey, const FPatchOutputStrongPtr& InPatchOutputStrongPtr);

		// Adds a patch output for an audio bus from the Audio Thread
		void AddPatchOutputForAudioBus_AudioThread(FAudioBusKey InAudioBusKey, const FPatchOutputStrongPtr& InPatchOutputStrongPtr);

		// Adds a patch input for an audio bus
		void AddPatchInputForAudioBus(FAudioBusKey InAudioBusKey, const FPatchInput& InPatchInput);

		// Adds a patch input for an audio bus from the Audio Thread
		void AddPatchInputForAudioBus_AudioThread(FAudioBusKey InAudioBusKey, const FPatchInput& InPatchInput);

		void Play(const int32 SourceId);
		void Stop(const int32 SourceId);
		void CancelQuantizedSound(const int32 SourceId);
		void StopInternal(const int32 SourceId);
		void StopFade(const int32 SourceId, const int32 NumFrames);
		void Pause(const int32 SourceId);
		void SetPitch(const int32 SourceId, const float Pitch);
		void SetVolume(const int32 SourceId, const float Volume);
		void SetDistanceAttenuation(const int32 SourceId, const float DistanceAttenuation);
		void SetSpatializationParams(const int32 SourceId, const FSpatializationParams& InParams);
		void SetChannelMap(const int32 SourceId, const uint32 NumInputChannels, const Audio::FAlignedFloatBuffer& InChannelMap, const bool bInIs3D, const bool bInIsCenterChannelOnly);
		void SetLPFFrequency(const int32 SourceId, const float Frequency);
		void SetHPFFrequency(const int32 SourceId, const float Frequency);

		// Sets base (i.e. carrier) frequency of modulatable parameters
		void SetModPitch(const int32 SourceId, const float InModPitch);
		void SetModVolume(const int32 SourceId, const float InModVolume);
		void SetModLPFFrequency(const int32 SourceId, const float InModFrequency);
		void SetModHPFFrequency(const int32 SourceId, const float InModFrequency);
		
		void SetModulationRouting(const int32 SourceId, FSoundModulationDefaultSettings& ModulationSettings);

		void SetSourceBufferListener(const int32 SourceId, FSharedISourceBufferListenerPtr& InSourceBufferListener, bool InShouldSourceBufferListenerZeroBuffer);

		void SetListenerTransforms(const TArray<FTransform>& ListenerTransforms);
		const TArray<FTransform>* GetListenerTransforms() const;

		int64 GetNumFramesPlayed(const int32 SourceId) const;
		float GetEnvelopeValue(const int32 SourceId) const;

		float GetVolumeModulationValue(const int32 SourceId) const;

#if ENABLE_AUDIO_DEBUG
		double GetCPUCoreUtilization(const int32 SourceId) const;
#endif // ENABLE_AUDIO_DEBUG
		float GetRelativeRenderCost(const int32 SourceId) const;
		bool IsUsingHRTFSpatializer(const int32 SourceId) const;
		bool NeedsSpeakerMap(const int32 SourceId) const;
		void ComputeNextBlockOfSamples();
		void UpdateSourceState();
		void ClearStoppingSounds();
		void MixOutputBuffers(const int32 SourceId, int32 InNumOutputChannels, const float InSendLevel, EMixerSourceSubmixSendStage InSubmixSendStage, uint32 InSubmixId, FAlignedFloatBuffer& ScratchBuffer, FAlignedFloatBuffer& OutWetBuffer);
		void RenderSource(const int32 SourceId);

		// Retrieves a channel map for the given source ID for the given output channels
		// can be used even when a source is 3D if the source is doing any kind of bus sending or otherwise needs a channel map
		void Get2DChannelMap(const int32 SourceId, int32 InNumOutputChannels, Audio::FAlignedFloatBuffer& OutChannelMap);

		// Called by a soundfield submix to get encoded audio.
		// If this source wasn't encoded (possibly because it is paused or finished playing),
		// this returns nullptr.
		// Returned nonnull pointers are only guaranteed to be valid on the audio mixer render thread.
		const ISoundfieldAudioPacket* GetEncodedOutput(const int32 SourceId, const FSoundfieldEncodingKey& InKey) const;

		const FQuat GetListenerRotation(const int32 SourceId) const;

		void SetSubmixSendInfo(const int32 SourceId, const FMixerSourceSubmixSend& SubmixSend);
		void ClearSubmixSendInfo(const int32 SourceId, const FMixerSourceSubmixSend& SubmixSend);

		void SetBusSendInfo(const int32 SourceId, EBusSendType InAudioBusSendType, uint32 AudiobusId, float BusSendLevel, const FString& InBusName, const FSendFilterSettings& InFilter = FSendFilterSettings{});

		void UpdateDeviceChannelCount(const int32 InNumOutputChannels);

		void UpdateSourceEffectChain(const uint32 SourceEffectChainId, const TArray<FSourceEffectChainEntry>& SourceEffectChain, const bool bPlayEffectChainTails);


		// Quantized event methods
		void PauseSoundForQuantizationCommand(const int32 SourceId);
		void SetSubBufferDelayForSound(const int32 SourceId, const int32 FramesToDelay);
		void UnPauseSoundForQuantizationCommand(const int32 SourceId);

		// Buffer getters
		const float* GetPreDistanceAttenuationBuffer(const int32 SourceId) const;
		const float* GetPostDistanceAttenuationBuffer(const int32 SourceId) const;
		const float* GetPreEffectBuffer(const int32 SourceId) const;
		const float* GetPreviousSourceBusBuffer(const int32 SourceId) const;
		const float* GetPreviousAudioBusBuffer(const int32 AudioBusId) const;
		int32 GetNumChannels(const int32 SourceId) const;
		int32 GetNumOutputFrames() const { return NumOutputFrames; }
		bool IsSourceBus(const int32 SourceId) const;
		void PumpCommandQueue();
		void UpdatePendingReleaseData(bool bForceWait = false);
		void FlushCommandQueue(bool bPumpCommandQueue = false, bool bSkipWaitForRenderThread = false);

		// Pushes a TFUnction command into an MPSC queue from an arbitrary thread to the audio render thread
		void AudioMixerThreadMPSCCommand(TFunction<void()>&& InCommand, const char* InDebugString=nullptr);
		
		void AddPendingAudioBusConnection(FAudioBusKey AudioBusKey, int32 NumChannels, bool bIsAutomatic, FPatchInput PatchInput, uint32 WaveInstancePlayOrder = INDEX_NONE, const uint64 InstanceID = 0u)
		{
			PendingAudioBusConnections.Enqueue(FPendingAudioBusConnection{ FPendingAudioBusConnection::FPatchVariant(TInPlaceType<FPatchInput>()
												, MoveTemp(PatchInput))
												, MoveTemp(AudioBusKey)
												, NumChannels
												, bIsAutomatic
												, WaveInstancePlayOrder
#if	UE_AUDIO_PROFILERTRACE_ENABLED
												, InstanceID
#endif // UE_AUDIO_PROFILERTRACE_ENABLED
			});
		}

		void AddPendingAudioBusConnection(FAudioBusKey AudioBusKey, int32 NumChannels, bool bIsAutomatic, FPatchOutputStrongPtr PatchOutputStrongPtr, uint32 WaveInstancePlayOrder = INDEX_NONE, const uint64 InstanceID = 0u)
		{
			PendingAudioBusConnections.Enqueue(FPendingAudioBusConnection{ FPendingAudioBusConnection::FPatchVariant(TInPlaceType<FPatchOutputStrongPtr>()
												, MoveTemp(PatchOutputStrongPtr))
												, MoveTemp(AudioBusKey)
												, NumChannels
												, bIsAutomatic
												, WaveInstancePlayOrder
#if	UE_AUDIO_PROFILERTRACE_ENABLED
												, InstanceID 
#endif // UE_AUDIO_PROFILERTRACE_ENABLED
			});
		}

		bool UseRenderScheduler() const
		{
			return MixerDevice->UseRenderScheduler();
		}

		FAudioRenderScheduler& GetRenderScheduler() const
		{
			return MixerDevice->GetRenderScheduler();
		}

		const bool CheckAndResetHardwareCallbackStallFlag()
		{
			bool bShouldResetTemp = bShouldResetHardwareCallback;
			bShouldResetHardwareCallback = false;
			return bShouldResetTemp; 
		}

		int32 AllocateGeneratorSource(ISoundGeneratorRef InGenerator);
		bool IsGeneratorSourceDone(const int32 SourceId);
		void FreeGeneratorSource(const int32 SourceId);
		void PlayGeneratorSource(const int32 SourceId);
		void StopGeneratorSource(const int32 SourceId);

	private:
#define INVALID_AUDIO_RENDER_THREAD_ID static_cast<uint32>(-1)
		uint32 AudioRenderThreadId = INVALID_AUDIO_RENDER_THREAD_ID;
		void ReleaseSource(const int32 SourceId);
		void BuildSourceEffectChain(const int32 SourceId, FSoundEffectSourceInitData& InitData, const TArray<FSourceEffectChainEntry>& SourceEffectChain, TArray<TSoundEffectSourcePtr>& OutSourceEffects);
		void ResetSourceEffectChain(const int32 SourceId);
		void SetSourceDone(const int32 SourceId);

		void GenerateSourceAudio(const bool bGenerateBuses);
		void GenerateSourceAudio(const bool bGenerateBuses, const int32 SourceIdStart, const int32 SourceIdEnd);

		void ComputeSourceBuffer(const bool bGenerateBuses, const int32 SourceId);
		void ComputePostSourceEffectBuffer(const bool bGenerateBuses, const int32 SourceId);
		void ComputeOutputBuffers(const bool bGenerateBuses, const int32 SourceId);

		void ConnectBusPatches();
		void ComputeBuses();
		void UpdateBuses();
		void AddBusPatchDependency(const uint32 WaveInstancePlayOrder, const FAudioRenderStepId FirstStep, const FAudioRenderStepId SecondStep);
		void RemoveBusPatchDependencies(const uint32 WaveInstancePlayOrder);

		float GetFloatCompareTolerance() const;
		float GetCommandQueueFillPercentage() const;

		struct FAudioMixerThreadCommand
		{
			// ctor
			FAudioMixerThreadCommand() = default;
			FAudioMixerThreadCommand(TFunction<void()>&& InFunction, const char* InDebugString, bool bInDeferExecution = false);
			
			// function-call operator
			void operator()() const;

			// data
			TFunction<void()> Function;

			// Defers the execution by a single call to PumpCommandQueue()
			// (used for commands that affect a playing source,
			// and that source gets initialized after the command executes
			bool bDeferExecution = false;

#if WITH_AUDIO_MIXER_THREAD_COMMAND_DEBUG			
			const char* DebugString=nullptr;					// Statically defined string from macro AUDIO_MIXER_THREAD_COMMAND_STRING
			mutable uint64_t StartExecuteTimeInCycles=0;		// Set just before Function is called, for diagnostics.
#endif // #if WITH_AUDIO_MIXER_THREAD_COMMAND_DEBUG

			FString GetSafeDebugString() const;
			float GetExecuteTimeInSeconds() const;
		};

		void AudioMixerThreadCommand(TFunction<void()>&& InFunction, const char* InDebugString = nullptr, bool bInDeferExecution = false);

		// Critical section to ensure mutating effect chains is thread-safe
		FCriticalSection EffectChainMutationCriticalSection;

		FMixerDevice* MixerDevice;

		// Info about spatialization plugin
		FAudioDevice::FAudioSpatializationInterfaceInfo SpatialInterfaceInfo;
		
		// Cached ptr to an optional source data override plugin
		TAudioSourceDataOverridePtr SourceDataOverridePlugin;

		IAudioLinkFactory* AudioLinkFactory = nullptr;

		// Array of pointers to game thread audio source objects
		TArray<FMixerSourceVoice*> MixerSources;

		// A command queue to execute commands from audio thread (or game thread) to audio mixer device thread.
		struct FCommands
		{
			FThreadSafeCounter NumTimesOvergrown = 0;
			TArray<FAudioMixerThreadCommand> SourceCommandQueue;
		};
		
		FCommands CommandBuffers[2];
		FThreadSafeCounter RenderThreadCommandBufferIndex;
		bool bShouldResetHardwareCallback = false;

		FEvent* CommandsProcessedEvent;
		FCriticalSection CommandBufferIndexCriticalSection;

		TArray<int32> DebugSoloSources;

		using FAudioMixerMpscCommand = FAudioMixerThreadCommand;
		TMpscQueue<FAudioMixerMpscCommand> MpscCommandQueue;
		
		struct FSourceInfo : public IAudioMixerRenderStep
		{
			FSourceInfo() : Resampler(1 /* NumChannels */) {}
			~FSourceInfo() = default;

			// The owning source manager
			FMixerSourceManager* SourceManager;

			// Object which handles source buffer decoding
			TSharedPtr<FMixerSourceBuffer, ESPMode::ThreadSafe> MixerSourceBuffer;
			ISourceListener* SourceListener;

			// Data used for rendering sources
			TSharedPtr<Audio::FAlignedFloatBuffer, ESPMode::ThreadSafe> CurrentPCMBuffer;

			// The post-attenuation source buffer, used to send audio to submixes
			Audio::FAlignedFloatBuffer SourceBuffer;
			Audio::FAlignedFloatBuffer PreEffectBuffer;
			Audio::FAlignedFloatBuffer PreDistanceAttenuationBuffer;
			Audio::FAlignedFloatBuffer PostDistanceAttenuationBuffer;
			Audio::FAlignedFloatBuffer SourceEffectScratchBuffer;

			// Data used for delaying the rendering of source audio for sample-accurate quantization
			int32 SubCallbackDelayLengthInFrames{ 0 };
			int32 CurrentFrameIndex;
			int64 NumFramesPlayed;

			TArray<FMixerSourceSubmixSend> SubmixSends;

			// What audio bus Id this source is sonfiying, if it is a source bus. This is INDEX_NONE for sources which are not source buses.
			uint32 AudioBusId;

			// Number of samples to count for source bus
			int64 SourceBusDurationFrames;

			// What buses this source is sending its audio to. Used to remove this source from the bus send list.
			TArray<uint32> AudioBusSends[(int32)EBusSendType::Count];

			// Interpolated source params
			FParam PitchSourceParam;
			float VolumeSourceStart;
			float VolumeSourceDestination;
			float VolumeFadeSlope;
			float VolumeFadeStart;
			int32 VolumeFadeFramePosition;
			int32 VolumeFadeNumFrames;

			float DistanceAttenuationSourceStart;
			float DistanceAttenuationSourceDestination;

			// Legacy filter LFP & HPF frequency set directly (not by modulation) on source
			float LowPassFreq;
			float HighPassFreq;

			// One-Pole LPFs and HPFs per source
			Audio::FInterpolatedLPF LowPassFilter;
			Audio::FInterpolatedHPF HighPassFilter;

			// Source effect instances
			uint32 SourceEffectChainId;
			TArray<TSoundEffectSourcePtr> SourceEffects;
			TArray<USoundEffectSourcePreset*> SourceEffectPresets;
			bool bEffectTailsDone;
			FSoundEffectSourceInputData SourceEffectInputData;

			FAudioPluginSourceOutputData AudioPluginOutputData;

			// A DSP object which tracks the amplitude envelope of a source.
			Audio::FInlineEnvelopeFollower SourceEnvelopeFollower;
			float SourceEnvelopeValue;

			// Modulation destinations
			Audio::FModulationDestination VolumeModulation;
			Audio::FModulationDestination PitchModulation;
			Audio::FModulationDestination LowpassModulation;
			Audio::FModulationDestination HighpassModulation;

			// Modulation Base (i.e. Carrier) Values
			float VolumeModulationBase;
			float PitchModulationBase;
			float LowpassModulationBase;
			float HighpassModulationBase;

			FSpatializationParams SpatParams;
			Audio::FAlignedFloatBuffer ScratchChannelMap;

			// Quantization data
			FQuartzQuantizedCommandHandle QuantizedCommandHandle;

			// Optional Source buffer listener.
			FSharedISourceBufferListenerPtr SourceBufferListener;

			// Optional AudioLink.
			IAudioLinkFactory::FAudioLinkSourcePushedSharedPtr AudioLink;

			// State management
			uint8 bIs3D:1;
			uint8 bIsCenterChannelOnly:1;
			uint8 bIsActive:1;
			uint8 bIsPlaying:1;
			uint8 bIsPaused:1;
			uint8 bIsPausedForQuantization:1;
			uint8 bDelayLineSet:1;
			uint8 bIsStopping:1;
			uint8 bHasStarted:1;
			uint8 bIsBusy:1;
			uint8 bUseHRTFSpatializer:1;
			uint8 bIsExternalSend:1;
			uint8 bUseOcclusionPlugin:1;
			uint8 bUseReverbPlugin:1;
			uint8 bIsDone:1;
			uint8 bIsLastBuffer:1;
			uint8 bEnableBusSends : 1;
			uint8 bEnableBaseSubmix : 1;
			uint8 bEnableSubmixSends : 1;
			uint8 bHasAttenuationSubmixSends : 1;
			uint8 bIsVorbis:1;
			uint8 bIsSoundfield:1;
			uint8 bHasPreDistanceAttenuationSend:1;
			uint8 bModFiltersUpdated : 1;
			uint8 bShouldSourceBufferListenerZeroBuffer : 1;
			uint8 bResampling : 1;

			// Source format info
			int32 NumInputChannels;
			int32 NumPostEffectChannels;
			int32 NumInputFrames;

			uint32 PlayOrder;
			uint32 ActiveSoundPlayOrder;

			// ID for associated Audio Component if there is one, 0 otherwise
			uint64 AudioComponentID;

			// Resampler instance
			FRuntimeResampler Resampler;

			FORCEINLINE void ResetModulators(const Audio::FDeviceId InDeviceId)
			{
				VolumeModulation.Init(InDeviceId, FName("Volume"), false /* bInIsBuffered */, true /* bInValueLinear */);
				PitchModulation.Init(InDeviceId, FName("Pitch"));
				HighpassModulation.Init(InDeviceId, FName("HPFCutoffFrequency"));
				LowpassModulation.Init(InDeviceId, FName("LPFCutoffFrequency"));

				VolumeModulationBase = 0.0f;
				PitchModulationBase = 0.0f;
				HighpassModulationBase = MIN_FILTER_FREQUENCY;
				LowpassModulationBase = MAX_FILTER_FREQUENCY;
			}

			//Helper function for determining if OutputToBusOnly is enabled
			bool IsRenderingToSubmixes() const;

#if AUDIO_MIXER_ENABLE_DEBUG_MODE
			uint8 bIsDebugMode : 1;
			FString DebugName;
#endif // AUDIO_MIXER_ENABLE_DEBUG_MODE
			FString GetDebugName() const
			{
#if AUDIO_MIXER_ENABLE_DEBUG_MODE
				return DebugName;
#else
				return {};
#endif //AUDIO_MIXER_ENABLE_DEBUG_MODE
				
			}

			// return the number of sample frames in the current input PCM buffer (if any)
			int32 GetCurrentAudioChunkNumFrames() const
			{
				if (CurrentPCMBuffer)
				{
					check(NumInputChannels != 0);
					return CurrentPCMBuffer->Num() / NumInputChannels;
				}
				else
				{
					return 0;
				}
			}

			virtual void DoRenderStep() override;
			virtual const TCHAR* GetRenderStepName() override;
		};

		static void ApplyDistanceAttenuation(FSourceInfo& InSourceInfo, int32 NumSamples);
		void ComputePluginAudio(FSourceInfo& InSourceInfo, FMixerSourceSubmixOutputBuffer& InSourceSubmixOutputBuffer, int32 SourceId, int32 NumSamples);

		// Hang/crash diagnostics.
		void DoStallDiagnostics();

		void LogRenderThreadStall();
		void LogInflightAsyncTasks();
		void LogCallstacks();
		void LogCallstack(uint32 InThreadId);

		TSharedRef<FMixerAudioBus> FindOrAddAudioBus(const FAudioBusKey BusKey, const int32 InNumChannels, const bool bIsAutomatic, bool* bOutAlreadyExisted = nullptr);

#if UE_AUDIO_PROFILERTRACE_ENABLED
		void OnTraceStarted(FTraceAuxiliary::EConnectionType TraceType, const FString& TraceDestination);
		void OnTraceStopped(FTraceAuxiliary::EConnectionType TraceType, const FString& TraceDestination);

		void RegisterAudioBusEnvelopeFollowerCVars();
		void UnregisterAudioBusEnvelopeFollowerCVars();

		void OnStartAudioBusEnvelopeFollowerCVarChanged(IConsoleVariable* InStartAudioBusEnvelopeFollowerCVar);
		void OnStopAudioBusEnvelopeFollowerCVarChanged(IConsoleVariable* InStopAudioBusEnvelopeFollowerCVar);

		void StartAudioBusEnvelopeFollowerFromCVar(const uint32 InAudioBusId);
		void StopAudioBusEnvelopeFollowerFromCVar(const uint32 InAudioBusId);

		void AutomaticallyStartAudioBusEnvelopeFollower(const FAudioBusKey& InAudioBusKey, const TSharedPtr<FMixerAudioBus>& InAudioBus);
		void StopAutoAudioBusEnvelopeFollower(const uint32 InAudioBusId, FMixerAudioBus& InAudioBus);

		void TraceSourceToSubmixSendMessage(const FSourceInfo& SourceInfo) const;
		void TraceSourceToBusSendMessage(const uint32 SourceId, const FSourceInfo& SourceInfo) const;

		uint32 ResolvePlayOrder(const FPendingAudioBusConnection& Connection) const;

		IConsoleVariable* StartAudioBusEnvelopeFollowerCVar = nullptr;
		IConsoleVariable* StopAudioBusEnvelopeFollowerCVar = nullptr;

		TSet<uint32> AudioBusesWithAutoEnvelopeFollowers;

		double BusIsAlivePingTimeSec = 0.0;
		static constexpr double BusIsAlivePingWaitTimeSec = 1.0;

		// Audio render thread only
		TMultiMap<uint32, uint32> ActiveBusPatchWriters;  // WaveInstancePlayOrder -> BusObjectId (source->bus, writer)
		TMultiMap<uint32, uint32> ActiveBusPatchReaders;  // WaveInstancePlayOrder -> BusObjectId (bus->source, reader)
		TMap<uint64, uint32> InstanceIdToPlayOrderMap;    // MixerSourceBuffer InstanceID -> WaveInstancePlayOrder
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

		// Array of listener transforms
		TArray<FTransform> ListenerTransforms;

		// Array of source infos.
		TArray<FSourceInfo> SourceInfos;

		// This array is independent of SourceInfos array to optimize for cache coherency
		TArray<FMixerSourceSubmixOutputBuffer> SourceSubmixOutputBuffers;

		// Map of bus object Id's to audio bus data. 
		TMap<FAudioBusKey, TSharedPtr<FMixerAudioBus>> AudioBuses; 
		TArray<FAudioBusKey> AudioBusKeys_AudioThread;

		// Array of task data waiting to finished. Processed on audio render thread.
		TArray<TSharedPtr<FMixerSourceBuffer, ESPMode::ThreadSafe>> PendingSourceBuffers;

		// General information about sources in source manager accessible from game thread
		struct FGameThreadInfo
		{
			TArray<int32> FreeSourceIndices;
			TArray<bool> bIsBusy;
			TArray<bool> bNeedsSpeakerMap;
			TArray<bool> bIsDebugMode;
			TArray<bool> bIsUsingHRTFSpatializer;
#if ENABLE_AUDIO_DEBUG
			TArray<double> CPUCoreUtilization;
#endif // #if ENABLE_AUDIO_DEBUG
			TArray<float> RelativeRenderCost;
			TArray<float> ModulationVolume;
		} GameThreadInfo;

		int32 NumActiveSources;
		int32 NumTotalSources;
		int32 NumLegacySources;
		int32 NumGeneratorSources;
		int32 NumOutputFrames;
		int32 NumOutputSamples;

		struct FGeneratorSourceState
		{
			// Generator used for rendering, owned by render thread.
			ISoundGeneratorPtr Generator;

			// Set on game thread, cleared from render thread when rendering is complete
			std::atomic<uint8> bSourceBusy = false;

			// Is this voice allocated on the game thread?
			uint8 bAllocated:1 = false;
		};

		TArray<FGeneratorSourceState> GeneratorSourceState;

		// Commands queued up to execute
		FThreadSafeCounter NumCommands;

		uint8 bInitialized : 1;
		uint8 bUsingSpatializationPlugin : 1;
		uint8 bUsingSourceDataOverridePlugin : 1;

		// Set to true when the audio source manager should pump the command queue
		FThreadSafeBool bPumpQueue;
		std::atomic<uint64> LastPumpCompleteTimeInCycles=0;
		std::atomic<ESourceManagerRenderThreadPhase> RenderThreadPhase=ESourceManagerRenderThreadPhase::Begin;
		FRWLock CurrentlyExecutingCmdLock;						// R/W slim lock for the currently executing cmd, so we can safely query it.
		FAudioMixerThreadCommand CurrentlyExecuteingCmd;		// Keep this as a member so we can't always peek the executing cmd.

		struct FPendingAudioBusConnection
		{
			using FPatchVariant = TVariant<FPatchInput, FPatchOutputStrongPtr>;
			FPatchVariant PatchVariant;
			FAudioBusKey AudioBusKey;
			int32 NumChannels = 0;
			bool bIsAutomatic = false;
			uint32 WaveInstancePlayOrder = INDEX_NONE;

#if UE_AUDIO_PROFILERTRACE_ENABLED
			uint64 InstanceID = 0u; // MixerSourceBuffer instance ID, used for PlayOrder resolution in trace profiling
#endif // UE_AUDIO_PROFILERTRACE_ENABLED
		};

		TMpscQueue<FPendingAudioBusConnection> PendingAudioBusConnections;

		TMultiMap<uint64, TPair<FAudioRenderStepId, FAudioRenderStepId>> BusPatchDependencies;

		friend class FMixerSourceVoice;
	};
}
