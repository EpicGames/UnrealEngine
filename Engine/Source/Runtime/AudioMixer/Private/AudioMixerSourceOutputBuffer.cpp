// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioMixerSourceOutputBuffer.h"
#include "AudioMixerBus.h"
#include "DSP/InterpolatedOnePole.h"
#include "SoundFieldRendering.h"
#include "AudioMixerSubmix.h"
#include "DSP/FloatArrayMath.h"

namespace Audio
{
	// Utility function to convert cartesian coordinates to spherical coordinates
	static void ConvertCartesianToSpherical(const FVector& InVector, float& OutAzimuth, float& OutElevation, float& OutRadius)
	{
		// Convert coordinates from unreal cartesian system to left handed spherical coordinates (zenith is positive elevation, right is positive azimuth)
		const float InX = -InVector.Z; //InVector.Y;
		const float InY = InVector.X;// -InVector.Z;
		const float InZ = -InVector.Y;


		OutElevation = FMath::Atan2(InY, InX);

		// Note, rather than using arccos(z / radius) here, we use Atan2 to avoid wrapping issues with negative elevation values.
		OutAzimuth = FMath::Atan2(FMath::Sqrt(InX * InX + InY * InY), InZ);
		OutRadius = InVector.Size();
	}

	FMixerSourceSubmixOutputBuffer::FMixerSourceSubmixOutputBuffer(FMixerDevice* InMixerDevice, uint32 InNumSourceChannels, uint32 InNumOutputChannels, uint32 InNumFrames)
		: PreAttenuationSourceBuffer(nullptr)
		, PostAttenuationSourceBuffer(nullptr)
		, SourceChannelMap(InNumSourceChannels, InNumOutputChannels)
		, NumSourceChannels(InNumSourceChannels)
		, NumFrames(InNumFrames)
		, NumOutputChannels(InNumOutputChannels)
		, MixerDevice(InMixerDevice)
		, bIsInitialDownmix(true)
		, bIs3D(false)
		, bIsVorbis(false)
	{
		PreAttenuationOutputBuffer.Reset();
		PreAttenuationOutputBuffer.AddUninitialized(InNumFrames * InNumOutputChannels);

		PostAttenuationOutputBuffer.Reset();
		PostAttenuationOutputBuffer.AddUninitialized(InNumFrames * InNumOutputChannels);
	}

	FMixerSourceSubmixOutputBuffer::~FMixerSourceSubmixOutputBuffer()
	{

	}

	void FMixerSourceSubmixOutputBuffer::Reset(const FMixerSourceSubmixOutputBufferSettings& InInitSettings)
	{
		// Whether or not this is a 3D submix output
		bIs3D = InInitSettings.bIs3D;
		bIsVorbis = InInitSettings.bIsVorbis;

		// Reset our record-keeping data
		NumOutputChannels = InInitSettings.NumOutputChannels;
		NumSourceChannels = InInitSettings.NumSourceChannels;

		// Reset the source channel map
		SourceChannelMap.Reset(NumSourceChannels, NumOutputChannels);

		// Reset the output buffers
		PreAttenuationOutputBuffer.Reset();
		PreAttenuationOutputBuffer.AddUninitialized(NumFrames * NumOutputChannels);

		PostAttenuationOutputBuffer.Reset();
		PostAttenuationOutputBuffer.AddUninitialized(NumFrames * NumOutputChannels);

		// Reset the post and pre source buffers
		PostAttenuationSourceBuffer = nullptr;
		PreAttenuationSourceBuffer = nullptr;

		// Reset the soundfield data
		EncodedSoundfieldDownmixes.Reset();
		SoundfieldDecoder.Reset();
		bIsInitialDownmix = true;

		// Make a new soundfield decoder if this sound is a soundfield source
		if (InInitSettings.bIsSoundfield)
		{
			SoundfieldDecoder = CreateDefaultSourceAmbisonicsDecoder(MixerDevice);
		}

		// Create data for each of the soundfield submix sends
		for (const FMixerSubmixPtr& SoundfieldSubmixPtr : InInitSettings.SoundfieldSubmixSends)
		{
			// Get the encoding key for the given soundfield submix this source is sending to
			FSoundfieldEncodingKey EncodingKey = SoundfieldSubmixPtr->GetKeyForSubmixEncoding();

			// Create a data bucket based on this encoding key
			FSoundfieldData& SoundfieldData = EncodedSoundfieldDownmixes.FindOrAdd(EncodingKey);

			// Get the soundfield submix's factory object, which creates encoders, decoders, transcoders, etc.
			ISoundfieldFactory* Factory = SoundfieldSubmixPtr->GetSoundfieldFactory();
			check(Factory);

			FAudioPluginInitializationParams PluginInitParams = SoundfieldSubmixPtr->GetInitializationParamsForSoundfieldStream();
			PluginInitParams.NumOutputChannels = NumSourceChannels;

			SoundfieldData.EncoderSettings = SoundfieldSubmixPtr->GetSoundfieldSettings().Duplicate();

			// If this source is soundfield, we need to use a transcoder
			if (InInitSettings.bIsSoundfield)
			{
				if (Factory->GetSoundfieldFormatName() == GetUnrealAmbisonicsFormatName())
				{
					SoundfieldData.bIsUnrealAmbisonicsSubmix = true;
				}
				else if (Factory->CanTranscodeFromSoundfieldFormat(GetUnrealAmbisonicsFormatName(), GetAmbisonicsSourceDefaultSettings()))
				{
					SoundfieldData.SoundfieldTranscoder = Factory->CreateTranscoderStream(GetUnrealAmbisonicsFormatName(), GetAmbisonicsSourceDefaultSettings(), Factory->GetSoundfieldFormatName(), *SoundfieldData.EncoderSettings, PluginInitParams);
				}
			}
			else
			{
				check(SoundfieldData.EncoderSettings.IsValid());

				SoundfieldData.SoundfieldEncoder = Factory->CreateEncoderStream(PluginInitParams, *SoundfieldData.EncoderSettings);
			}

			// Create a blank packet for memory of the encoded packet
			SoundfieldData.EncodedPacket = Factory->CreateEmptyPacket();
		}
	}

	void FMixerSourceSubmixOutputBuffer::SetNumOutputChannels(uint32 InNumOutputChannels)
	{
		NumOutputChannels = InNumOutputChannels;

		SourceChannelMap.Reset(NumSourceChannels, NumOutputChannels);

		PreAttenuationOutputBuffer.Reset();
		PreAttenuationOutputBuffer.AddUninitialized(NumFrames * NumOutputChannels);

		PostAttenuationOutputBuffer.Reset();
		PostAttenuationOutputBuffer.AddUninitialized(NumFrames * NumOutputChannels);
	}

	bool FMixerSourceSubmixOutputBuffer::SetChannelMap(const FAlignedFloatBuffer& InChannelMap, bool bInIsCenterChannelOnly)
	{
		bool bNeedsNewChannelMap = false;

		// Fix up the channel map in case the device output count changed
		const uint32 ChannelMapSize = SourceChannelMap.CopySize / sizeof(float);

		if (InChannelMap.Num() != ChannelMapSize)
		{
			FAlignedFloatBuffer NewChannelMap;

			if (bIs3D)
			{
				NewChannelMap.AddZeroed(ChannelMapSize);
				bNeedsNewChannelMap = true;
			}
			else
			{
				const uint32 ChannelMapOutputChannels = ChannelMapSize / NumSourceChannels;
				FMixerDevice::Get2DChannelMap(bIsVorbis, NumSourceChannels, ChannelMapOutputChannels, bInIsCenterChannelOnly, NewChannelMap);
			}

			SourceChannelMap.SetChannelMap(NewChannelMap.GetData());
		}
		else
		{
			SourceChannelMap.SetChannelMap(InChannelMap.GetData());
		}

		return bNeedsNewChannelMap;
	}

	void FMixerSourceSubmixOutputBuffer::SetPreAttenuationSourceBuffer(FAlignedFloatBuffer* InPreAttenuationSourceBuffer)
	{
		PreAttenuationSourceBuffer = InPreAttenuationSourceBuffer;
	}

	void FMixerSourceSubmixOutputBuffer::SetPostAttenuationSourceBuffer(FAlignedFloatBuffer* InPostAttenuationSourceBuffer)
	{
		PostAttenuationSourceBuffer = InPostAttenuationSourceBuffer;
	}

	void FMixerSourceSubmixOutputBuffer::ComputeOutput(const FSpatializationParams& InSpatParams)
	{
		// No need to compute an output if there is no source buffer available
		if (!PostAttenuationSourceBuffer && !PreAttenuationSourceBuffer)
		{
			return;
		}

		// Update our rotational data based off the spat params
		SoundfieldPositionalData.Rotation = InSpatParams.ListenerOrientation;
		SoundSourceRotation = InSpatParams.EmitterWorldRotation;
		
		if (bIs3D && !bIsInitialDownmix)
		{
			ComputeOutput3D();
		}
		else
		{
			ComputeOutput2D();
			bIsInitialDownmix = false;
		}

		// Now check if we need to do any sound field encoding
		if (EncodedSoundfieldDownmixes.Num())
		{
			EncodeToSoundfieldFormats(InSpatParams);
		}
	}

	void FMixerSourceSubmixOutputBuffer::ComputeOutput3D(FAlignedFloatBuffer& InSourceBuffer, FAlignedFloatBuffer& OutSourceBuffer)
	{
		if (SoundfieldDecoder.IsValid())
		{
			FAmbisonicsSoundfieldBuffer SoundfieldBuffer;
			SoundfieldBuffer.AudioBuffer = MoveTemp(InSourceBuffer);
			SoundfieldBuffer.NumChannels = NumSourceChannels;
			SoundfieldBuffer.PreviousRotation = SoundfieldBuffer.Rotation;
			SoundfieldBuffer.Rotation = SoundSourceRotation;

			SoundfieldPositionalData.NumChannels = NumOutputChannels;
			SoundfieldPositionalData.ChannelPositions = MixerDevice->GetDefaultPositionMap(NumOutputChannels);

			FSoundfieldDecoderInputData SoundfieldDecoderInputData =
			{
				SoundfieldBuffer,
				SoundfieldPositionalData,
				static_cast<int32>(InSourceBuffer.Num() / NumSourceChannels),
				MixerDevice->GetSampleRate()
			};

			FSoundfieldDecoderOutputData SoundFieldDecoderOutputData = { OutSourceBuffer };

			SoundfieldDecoder->Decode(SoundfieldDecoderInputData, SoundFieldDecoderOutputData);

			InSourceBuffer = MoveTemp(SoundfieldBuffer.AudioBuffer);
		}
		else if (NumSourceChannels == 1)
		{
			switch (NumOutputChannels)
			{
			case 8:
				MixMonoTo8ChannelsFast(InSourceBuffer, OutSourceBuffer, SourceChannelMap.ChannelStartGains, SourceChannelMap.ChannelDestinationGains);
				break;
			case 6:
				MixMonoTo6ChannelsFast(InSourceBuffer, OutSourceBuffer, SourceChannelMap.ChannelStartGains, SourceChannelMap.ChannelDestinationGains);
				break;
			case 4:
				MixMonoTo4ChannelsFast(InSourceBuffer, OutSourceBuffer, SourceChannelMap.ChannelStartGains, SourceChannelMap.ChannelDestinationGains);
				break;
			case 2:
				MixMonoTo2ChannelsFast(InSourceBuffer, OutSourceBuffer, SourceChannelMap.ChannelStartGains, SourceChannelMap.ChannelDestinationGains);
				break;
			}

		}
		else if (NumSourceChannels == 2)
		{
			switch (NumOutputChannels)
			{
			case 8:
				Mix2ChannelsTo8ChannelsFast(InSourceBuffer, OutSourceBuffer, SourceChannelMap.ChannelStartGains, SourceChannelMap.ChannelDestinationGains);
				break;
			case 6:
				Mix2ChannelsTo6ChannelsFast(InSourceBuffer, OutSourceBuffer, SourceChannelMap.ChannelStartGains, SourceChannelMap.ChannelDestinationGains);
				break;
			case 4:
				Mix2ChannelsTo4ChannelsFast(InSourceBuffer, OutSourceBuffer, SourceChannelMap.ChannelStartGains, SourceChannelMap.ChannelDestinationGains);
				break;
			case 2:
				Mix2ChannelsTo2ChannelsFast(InSourceBuffer, OutSourceBuffer, SourceChannelMap.ChannelStartGains, SourceChannelMap.ChannelDestinationGains);
				break;
			}
		}
		else
		{
			DownmixBuffer(NumSourceChannels, NumOutputChannels, InSourceBuffer, OutSourceBuffer, SourceChannelMap.ChannelStartGains, SourceChannelMap.ChannelDestinationGains);
		}
	}

	void FMixerSourceSubmixOutputBuffer::ComputeOutput3D()
	{
		// Compute the output buffers if there is a corresponding pre or post source buffer set

		if (PreAttenuationSourceBuffer)
		{
			ComputeOutput3D(*PreAttenuationSourceBuffer, PreAttenuationOutputBuffer);
		}

		if (PostAttenuationSourceBuffer)
		{
			ComputeOutput3D(*PostAttenuationSourceBuffer, PostAttenuationOutputBuffer);
		}

		// Do the channel map copy from dest to start (which prevents zippering of dynamic channel mapping for 3d audio) after the data has been computed for both pre- and post- attenuation channel downmixing
		SourceChannelMap.CopyDestinationToStart();
	}

	void FMixerSourceSubmixOutputBuffer::ComputeOutput2D(FAlignedFloatBuffer& InSourceBuffer, FAlignedFloatBuffer& OutSourceBuffer)
	{
		// For 2D sources, we just apply the gain matrix in ChannelDestionationGains with no interpolation.
		if (SoundfieldDecoder.IsValid())
		{
			FAmbisonicsSoundfieldBuffer SoundfieldBuffer;
			SoundfieldBuffer.AudioBuffer = MoveTemp(InSourceBuffer);
			SoundfieldBuffer.NumChannels = NumSourceChannels;
			SoundfieldBuffer.PreviousRotation = SoundfieldBuffer.Rotation;
			SoundfieldBuffer.Rotation = SoundSourceRotation;

			SoundfieldPositionalData.NumChannels = NumOutputChannels;
			SoundfieldPositionalData.ChannelPositions = MixerDevice->GetDefaultPositionMap(NumOutputChannels);

			FSoundfieldDecoderInputData SoundfieldDecoderInputData =
			{
				SoundfieldBuffer,
				SoundfieldPositionalData,
				static_cast<int32>(InSourceBuffer.Num() / NumSourceChannels),
				MixerDevice->GetSampleRate()
			};

			FSoundfieldDecoderOutputData SoundfieldDecoderOutputData = { OutSourceBuffer };

			SoundfieldDecoder->Decode(SoundfieldDecoderInputData, SoundfieldDecoderOutputData);

			// Move the encoded ambisonics source buffer back to PostEffectBuffers to prevent reallocation
			InSourceBuffer = MoveTemp(SoundfieldBuffer.AudioBuffer);
		}
		else if (NumSourceChannels == 1)
		{
			switch (NumOutputChannels)
			{
			case 8:
				Audio::MixMonoTo8ChannelsFast(InSourceBuffer, OutSourceBuffer, SourceChannelMap.ChannelDestinationGains);
				break;
			case 6:
				Audio::MixMonoTo6ChannelsFast(InSourceBuffer, OutSourceBuffer, SourceChannelMap.ChannelDestinationGains);
				break;
			case 4:
				Audio::MixMonoTo4ChannelsFast(InSourceBuffer, OutSourceBuffer, SourceChannelMap.ChannelDestinationGains);
				break;
			case 2:
				Audio::MixMonoTo2ChannelsFast(InSourceBuffer, OutSourceBuffer, SourceChannelMap.ChannelDestinationGains);
				break;
			}
		}
		else if (NumSourceChannels == 2)
		{
			switch (NumOutputChannels)
			{
			case 8:
				Audio::Mix2ChannelsTo8ChannelsFast(InSourceBuffer, OutSourceBuffer, SourceChannelMap.ChannelDestinationGains);
				break;
			case 6:
				Audio::Mix2ChannelsTo6ChannelsFast(InSourceBuffer, OutSourceBuffer, SourceChannelMap.ChannelDestinationGains);
				break;
			case 4:
				Audio::Mix2ChannelsTo4ChannelsFast(InSourceBuffer, OutSourceBuffer, SourceChannelMap.ChannelDestinationGains);
				break;
			case 2:
				Audio::Mix2ChannelsTo2ChannelsFast(InSourceBuffer, OutSourceBuffer, SourceChannelMap.ChannelDestinationGains);
				break;
			}
		}
		else
		{
			Audio::DownmixBuffer(NumSourceChannels, NumOutputChannels, InSourceBuffer, OutSourceBuffer, SourceChannelMap.ChannelDestinationGains);
		}
	}

	void FMixerSourceSubmixOutputBuffer::ComputeOutput2D()
	{
		if (PreAttenuationSourceBuffer)
		{
			ComputeOutput2D(*PreAttenuationSourceBuffer, PreAttenuationOutputBuffer);
		}

		if (PostAttenuationSourceBuffer)
		{
			ComputeOutput2D(*PostAttenuationSourceBuffer, PostAttenuationOutputBuffer);
		}
	}

	void FMixerSourceSubmixOutputBuffer::EncodeToSoundfieldFormats(const FSpatializationParams& InSpatParams)
	{
		check(MixerDevice);

		SoundfieldPositionalData.NumChannels = NumSourceChannels;

		// Spoof rotation of the source as if it's rotation of the listener when encoding non-sound-field to sound-field
		SoundfieldPositionalData.Rotation = InSpatParams.EmitterWorldRotation;

		InputChannelPositions.Reset();

		if (bIs3D)
		{
			if (NumSourceChannels == 1)
			{
				FChannelPositionInfo ChannelPosition;
				ChannelPosition.Channel = EAudioMixerChannel::FrontCenter;
				ConvertCartesianToSpherical(InSpatParams.EmitterPosition, ChannelPosition.Azimuth, ChannelPosition.Elevation, ChannelPosition.Radius);

				ChannelPosition.Radius = InSpatParams.Distance;
				InputChannelPositions.Add(ChannelPosition);

				SoundfieldPositionalData.ChannelPositions = &InputChannelPositions;
			}
			else if (NumSourceChannels == 2)
			{
				FChannelPositionInfo LeftChannelPosition;
				LeftChannelPosition.Channel = EAudioMixerChannel::FrontLeft;
				ConvertCartesianToSpherical(InSpatParams.LeftChannelPosition, LeftChannelPosition.Azimuth, LeftChannelPosition.Elevation, LeftChannelPosition.Radius);

				InputChannelPositions.Add(LeftChannelPosition);

				FChannelPositionInfo RightChannelPosition;
				LeftChannelPosition.Channel = EAudioMixerChannel::FrontRight;
				ConvertCartesianToSpherical(InSpatParams.RightChannelPosition, RightChannelPosition.Azimuth, RightChannelPosition.Elevation, RightChannelPosition.Radius);

				InputChannelPositions.Add(RightChannelPosition);

				SoundfieldPositionalData.ChannelPositions = &InputChannelPositions;
			}
		}

		// if 2D or not a supported channel configuration, use default position map
		if (!InputChannelPositions.Num())
		{
			SoundfieldPositionalData.ChannelPositions = MixerDevice->GetDefaultPositionMap(NumSourceChannels);
		}

		// Run the encoders.
		for (auto& Soundfield : EncodedSoundfieldDownmixes)
		{
			FSoundfieldData& SoundfieldData = Soundfield.Value;

			if (PreAttenuationSourceBuffer)
			{
				EncodeSoundfield(SoundfieldData , *PreAttenuationSourceBuffer);
			}

			if (PostAttenuationSourceBuffer)
			{
				EncodeSoundfield(SoundfieldData, *PostAttenuationSourceBuffer);
			}
		}
	}

	void FMixerSourceSubmixOutputBuffer::EncodeSoundfield(FSoundfieldData& InSoundfieldData, Audio::FAlignedFloatBuffer& InSourceBuffer)
	{
		check(InSoundfieldData.EncoderSettings.IsValid());
		check(InSoundfieldData.EncodedPacket.IsValid());

		InSoundfieldData.EncodedPacket->Reset();

		// We will have an soundfield transcoder if this sound source is a soundfield format
		if (InSoundfieldData.SoundfieldTranscoder)
		{
			FAmbisonicsSoundfieldBuffer SoundfieldBuffer;
			SoundfieldBuffer.AudioBuffer = MoveTemp(InSourceBuffer);
			SoundfieldBuffer.NumChannels = NumSourceChannels;
			SoundfieldBuffer.PreviousRotation = SoundfieldBuffer.Rotation;
			SoundfieldBuffer.Rotation = SoundSourceRotation;

			InSoundfieldData.SoundfieldTranscoder->Transcode(SoundfieldBuffer, GetAmbisonicsSourceDefaultSettings(), *InSoundfieldData.EncodedPacket, *InSoundfieldData.EncoderSettings);
			InSourceBuffer = MoveTemp(SoundfieldBuffer.AudioBuffer);
		}
		else if (InSoundfieldData.SoundfieldEncoder)
		{
			FSoundfieldEncoderInputData SoundfieldEncoderInputData =
			{
				InSourceBuffer,
				static_cast<int32>(NumSourceChannels),
				*InSoundfieldData.EncoderSettings,
				SoundfieldPositionalData
			};

			InSoundfieldData.SoundfieldEncoder->Encode(SoundfieldEncoderInputData, *InSoundfieldData.EncodedPacket);
		}
		else if (InSoundfieldData.bIsUnrealAmbisonicsSubmix)
		{
			FAmbisonicsSoundfieldBuffer& OutputPacket = DowncastSoundfieldRef<FAmbisonicsSoundfieldBuffer>(*InSoundfieldData.EncodedPacket);

			// Fixme: This is an array copy. Can we serve InPositionalData directly to this soundfield?
			OutputPacket.AudioBuffer = InSourceBuffer;
			OutputPacket.NumChannels = NumSourceChannels;
			OutputPacket.PreviousRotation = OutputPacket.Rotation;
			OutputPacket.Rotation = SoundSourceRotation;
		}
	}

	void FMixerSourceSubmixOutputBuffer::MixOutput(float InSendLevel, EMixerSourceSubmixSendStage InSubmixSendStage, FAlignedFloatBuffer& OutMixedBuffer) const
	{
		if (InSubmixSendStage == EMixerSourceSubmixSendStage::PostDistanceAttenuation)
		{
			Audio::ArrayMixIn(PostAttenuationOutputBuffer, OutMixedBuffer, InSendLevel);
		}
		else
		{
			Audio::ArrayMixIn(PreAttenuationOutputBuffer, OutMixedBuffer, InSendLevel);
		}
	}

	void FMixerSourceSubmixOutputBuffer::MixOutput(float InSendLevel, EMixerSourceSubmixSendStage InSubmixSendStage, const FSendFilterSettings& InFilter, FInterpolatedLPF& InLpf, FInterpolatedHPF& InHpf, FAlignedFloatBuffer& ScratchBuffer, FAlignedFloatBuffer& OutMixedBuffer)
	{
		const FAlignedFloatBuffer& SrcBuffer = (InSubmixSendStage == EMixerSourceSubmixSendStage::PostDistanceAttenuation) ? PostAttenuationOutputBuffer : PreAttenuationOutputBuffer;

		Audio::ArrayFilterAndMixIn(SrcBuffer, OutMixedBuffer, InSendLevel, NumOutputChannels, InFilter.bEnableLPF, InFilter.LpfCutoff, InFilter.bEnableHPF, InFilter.HpfCutoff, InLpf, InHpf, ScratchBuffer);
	}

	FQuat FMixerSourceSubmixOutputBuffer::GetListenerRotation() const
	{
		return SoundfieldPositionalData.Rotation;
	}

	void FMixerSourceSubmixOutputBuffer::CopyReverbPluginOutputData(FAlignedFloatBuffer& InAudioBuffer)
	{
		ReverbPluginOutputBuffer.Reset();
		ReverbPluginOutputBuffer.Append(InAudioBuffer);
	}

	const float* FMixerSourceSubmixOutputBuffer::GetReverbPluginOutputData() const
	{
		return ReverbPluginOutputBuffer.GetData();
	}

	const ISoundfieldAudioPacket* FMixerSourceSubmixOutputBuffer::GetSoundfieldPacket(const FSoundfieldEncodingKey& InKey) const
	{
		if(EncodedSoundfieldDownmixes.Contains(InKey))
		{
			const FSoundfieldData& SoundfieldData = EncodedSoundfieldDownmixes[InKey];
			return SoundfieldData.EncodedPacket.Get();
		}

		return nullptr;
	}

	ISoundfieldAudioPacket* FMixerSourceSubmixOutputBuffer::GetSoundFieldPacket(const FSoundfieldEncodingKey& InKey)
	{
		FSoundfieldData& SoundfieldData = EncodedSoundfieldDownmixes.FindOrAdd(InKey);
		return SoundfieldData.EncodedPacket.Get();
	}


}
