// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "CoreMinimal.h"
#include "ISoundfieldFormat.h"
#include "IAudioExtensionPlugin.h"
#include "AudioMixerDevice.h"
#include "AudioMixerSubmix.h"

namespace Audio
{
	class FInterpolatedLPF;
	class FInterpolatedHPF;
	struct FSendFilterSettings;

	class FMixerSubmix;

	struct FMixerSourceSubmixOutputBufferSettings
	{
		uint32 NumSourceChannels;
		uint32 NumOutputChannels;
		TArray<FMixerSubmixPtr> SoundfieldSubmixSends;
		bool bIs3D;
		bool bIsVorbis;
		bool bIsSoundfield;
	};

	/** 
	* Used in audio mixer source manager to convert source audio to channel output and mix into submixes
	*/
	class FMixerSourceSubmixOutputBuffer
	{
	public:
		FMixerSourceSubmixOutputBuffer(FMixerDevice* InMixerDevice, uint32 InNumSourceChannels, uint32 InNumOutputChannels, uint32 InNumFrames);

		~FMixerSourceSubmixOutputBuffer();

		// Resets the source submix output to prepare for new source data
		void Reset(const FMixerSourceSubmixOutputBufferSettings& InResetSettings);

		// Sets the number of output channels
		void SetNumOutputChannels(uint32 InNumOutputChannels);

		// Returns the number of source channels of this source
		uint32 GetNumSourceChannels() const { return NumSourceChannels; }

		// Sets the channel map
		bool SetChannelMap(const FAlignedFloatBuffer& InChannelMap, bool bInIsCenterChannelOnly);

		// Sets the pre and post attenuation source buffers. This is the source buffer data derived from source manager source processing.
		void SetPreAttenuationSourceBuffer(FAlignedFloatBuffer* InPreAttenuationBuffer);
		void SetPostAttenuationSourceBuffer(FAlignedFloatBuffer* InPostAttenuationBuffer);
		void CopyReverbPluginOutputData(FAlignedFloatBuffer& InAudioBuffer);
		const float* GetReverbPluginOutputData() const;

		// Retrieves the current sound field packet for the given key
		const ISoundfieldAudioPacket* GetSoundfieldPacket(const FSoundfieldEncodingKey& InKey) const;
		ISoundfieldAudioPacket* GetSoundFieldPacket(const FSoundfieldEncodingKey& InKey);

		// Computes the output buffer given the spat params. This buffer can then be mixed to submixes
		void ComputeOutput(const FSpatializationParams& InSpatParams);

		// Called by submixes to mix this output buffer to their buffer
		void MixOutput(float SendLevel, EMixerSourceSubmixSendStage InSubmixSendStage, FAlignedFloatBuffer& OutMixedBuffer) const;
		void MixOutput(float InSendLevel, EMixerSourceSubmixSendStage InSubmixSendStage, const FSendFilterSettings& InFilter, FInterpolatedLPF& InLpf, FInterpolatedHPF& InHpf, FAlignedFloatBuffer& ScratchBuffer, FAlignedFloatBuffer& OutMixedBuffer);

		// Return the listener rotation
		// TODO: consolidate the code that is using this to be private to this class.
		FQuat GetListenerRotation() const;

	private: // private classes

		// Private struct used for channel mapping
		struct FSourceChannelMap
		{
			alignas(16) float ChannelStartGains[AUDIO_MIXER_MAX_OUTPUT_CHANNELS * AUDIO_MIXER_MAX_OUTPUT_CHANNELS];
			alignas(16) float ChannelDestinationGains[AUDIO_MIXER_MAX_OUTPUT_CHANNELS * AUDIO_MIXER_MAX_OUTPUT_CHANNELS];

			// This is the number of bytes the gain array is using:
			// (Number of input channels * number of output channels) * sizeof float.
			int32 CopySize = 0;
			bool bIsInit = false;

			FSourceChannelMap(int32 InNumInChannels, int32 InNumOutChannels)
				: CopySize(InNumInChannels* InNumOutChannels * sizeof(float))
			{
				checkSlow(InNumInChannels <= AUDIO_MIXER_MAX_OUTPUT_CHANNELS);
				checkSlow(InNumOutChannels <= AUDIO_MIXER_MAX_OUTPUT_CHANNELS);
				FMemory::Memzero(ChannelStartGains, CopySize);
			}

			FORCEINLINE void Reset(int32 InNumInChannels, int32 InNumOutChannels)
			{
				checkSlow(InNumInChannels <= AUDIO_MIXER_MAX_OUTPUT_CHANNELS);
				checkSlow(InNumOutChannels <= AUDIO_MIXER_MAX_OUTPUT_CHANNELS);

				CopySize = InNumInChannels * InNumOutChannels * sizeof(float);
				FMemory::Memzero(ChannelStartGains, CopySize);
				FMemory::Memzero(ChannelDestinationGains, CopySize);
				bIsInit = false;
			}

			FORCEINLINE void CopyDestinationToStart()
			{
				FMemory::Memcpy(ChannelStartGains, ChannelDestinationGains, CopySize);
			}

			FORCEINLINE void SetChannelMap(const float* RESTRICT InChannelGains)
			{
				FMemory::Memcpy(ChannelDestinationGains, InChannelGains, CopySize);
				if (!bIsInit)
				{
					FMemory::Memcpy(ChannelStartGains, InChannelGains, CopySize);
					bIsInit = true;
				}
			}

		private:
			FSourceChannelMap()
				: CopySize(0)
				, bIsInit(false)
			{
			}
		};

		struct FSoundfieldData
		{
			// If the sound source is not a soundfield source and we're sending to a soundfield submix, it gets encoded
			TUniquePtr<ISoundfieldEncoderStream> SoundfieldEncoder;

			// If the sound source is soundfield, and we're sending to a soundfield submix, it gets transcoded
			TUniquePtr<ISoundfieldTranscodeStream> SoundfieldTranscoder;

			// The optional soundfield encoder settings
			TUniquePtr<ISoundfieldEncodingSettingsProxy> EncoderSettings;

			// The encoded soundfield packet
			TUniquePtr<ISoundfieldAudioPacket> EncodedPacket;

			// If this is a unreal ambisonics soundfield buffer, we hand it the submixed buffer directly.
			bool bIsUnrealAmbisonicsSubmix;
		};

	private: // private methods

		void ComputeOutput3D(FAlignedFloatBuffer& InSource, FAlignedFloatBuffer& InOutput);
		void ComputeOutput3D();
		void ComputeOutput2D(FAlignedFloatBuffer& InSource, FAlignedFloatBuffer& InOutput);
		void ComputeOutput2D();

		void EncodeSoundfield(FSoundfieldData& InSoundfieldData, Audio::FAlignedFloatBuffer& InSourceBuffer);
		void EncodeToSoundfieldFormats(const FSpatializationParams& InSpatParams);

	private: // private data

		// Buffer for reverb plugins to write into
		FAlignedFloatBuffer ReverbPluginOutputBuffer;

		// The source buffer created before distance attenuation is applied
		FAlignedFloatBuffer* PreAttenuationSourceBuffer;

		// The source buffer created after distance attenuation is applied
		FAlignedFloatBuffer* PostAttenuationSourceBuffer;

		// Data used to map the source channel data to the output channel configuration
		FSourceChannelMap SourceChannelMap;

		// The result of the source output with source data derived pre distance attenuation
		FAlignedFloatBuffer PreAttenuationOutputBuffer;

		// The result of the source output with source data derived post distance attenuation
		FAlignedFloatBuffer PostAttenuationOutputBuffer;

		// The number of source channels (sound input)
		uint32 NumSourceChannels;

		// The number of interleaved frames in the source
		const uint32 NumFrames;

		// The number of device output channels
		uint32 NumOutputChannels;

		// The owning mixer device
		FMixerDevice* MixerDevice;

		// Whether this is the initial downmix
		bool bIsInitialDownmix;

		// If this is a 3D sound source
		bool bIs3D;

		// Whether this source is vorbis encoded (used for 2D speakermapping)
		bool bIsVorbis;

		// Cached parameters for encoding to a soundfield format
		FSoundfieldSpeakerPositionalData SoundfieldPositionalData;

		// The current sound source rotation
		FQuat SoundSourceRotation;

		// Map of sound field encoding to encoding data
		TMap<FSoundfieldEncodingKey, FSoundfieldData> EncodedSoundfieldDownmixes;

		// Channel positions of the sound field
		TArray<Audio::FChannelPositionInfo> InputChannelPositions;

		// If this source is an ambisonics source, we use this to down mix the source to output channel mix.
		TUniquePtr<ISoundfieldDecoderStream> SoundfieldDecoder;

	};


}
