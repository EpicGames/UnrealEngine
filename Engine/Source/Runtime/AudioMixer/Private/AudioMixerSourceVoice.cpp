// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioMixerSourceVoice.h"
#include "AudioMixerSource.h"
#include "AudioMixerSourceManager.h"
#include "AudioMixerDevice.h"

namespace Audio
{

	/**
	* FMixerSourceVoice Implementation
	*/

	FMixerSourceVoice::FMixerSourceVoice()
	{
		Reset(nullptr);
	}

	FMixerSourceVoice::~FMixerSourceVoice()
	{
	}

	void FMixerSourceVoice::Reset(FMixerDevice* InMixerDevice)
	{
		if (InMixerDevice)
		{
			MixerDevice = InMixerDevice;
			SourceManager = MixerDevice->GetSourceManager();
		}
		else
		{
			MixerDevice = nullptr;
			SourceManager = nullptr;
		}

		Pitch = -1.0f;
		Volume = -1.0f;
		DistanceAttenuation = -1.0f;
		Distance = -1.0f;
		LPFFrequency = -1.0f;
		HPFFrequency = -1.0f;
		SourceId = INDEX_NONE;
		bIsPlaying = false;
		bIsPaused = false;
		bIsActive = false;
		bIsBus = false;
		bEnableBusSends = false;
		bEnableBaseSubmix = false;
		bEnableSubmixSends = false;
		bHasAttenuationSubmixSends = false;
		bStopFadedOut = false;

		PitchModBase = TNumericLimits<float>::Max();
		VolumeModBase = TNumericLimits<float>::Max();
		LPFFrequencyModBase = TNumericLimits<float>::Max();
		HPFFrequencyModBase = TNumericLimits<float>::Max();

		SubmixSends.Reset();
	}

	bool FMixerSourceVoice::Init(const FMixerSourceVoiceInitParams& InitParams)
	{
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		if (SourceManager->GetFreeSourceId(SourceId))
		{
			AUDIO_MIXER_CHECK(InitParams.SourceListener != nullptr);
			AUDIO_MIXER_CHECK(InitParams.NumInputChannels > 0);

			bEnableBusSends = InitParams.bEnableBusSends;
			bEnableBaseSubmix = InitParams.bEnableBaseSubmix;
			bEnableSubmixSends = InitParams.bEnableSubmixSends;
			bHasAttenuationSubmixSends = InitParams.bHasAttenuationSubmixSends;

			bIsBus = InitParams.AudioBusId != INDEX_NONE;

			for (int32 i = 0; i < InitParams.SubmixSends.Num(); ++i)
			{
				FMixerSubmixPtr SubmixPtr = InitParams.SubmixSends[i].Submix.Pin();
				if (SubmixPtr.IsValid())
				{
					SubmixSends.Add(SubmixPtr->GetId(), InitParams.SubmixSends[i]);
				}
			}

			bStopFadedOut = false;
			SourceManager->InitSource(SourceId, InitParams);
			return true;
		}

		return false;
	}

	void FMixerSourceVoice::Release()
	{
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		SourceManager->ReleaseSourceId(SourceId);
	}

	void FMixerSourceVoice::SetPitch(const float InPitch)
	{
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);
		if (!FMath::IsNearlyEqual(Pitch, InPitch, SourceManager->GetFloatCompareTolerance()))
		{
			Pitch = InPitch;
			SourceManager->SetPitch(SourceId, InPitch);
		}
	}

	void FMixerSourceVoice::SetVolume(const float InVolume)
	{
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);
		if (!FMath::IsNearlyEqual(Volume, InVolume, SourceManager->GetFloatCompareTolerance()))
		{
			Volume = InVolume;
			SourceManager->SetVolume(SourceId, InVolume);
		}
	}

	void FMixerSourceVoice::SetDistanceAttenuation(const float InDistanceAttenuation)
	{
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		if (!FMath::IsNearlyEqual(DistanceAttenuation, InDistanceAttenuation, SourceManager->GetFloatCompareTolerance()))
		{
			DistanceAttenuation = InDistanceAttenuation;
			SourceManager->SetDistanceAttenuation(SourceId, InDistanceAttenuation);
		}
	}

	void FMixerSourceVoice::SetLPFFrequency(const float InLPFFrequency)
	{
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		if (!FMath::IsNearlyEqual(LPFFrequency, InLPFFrequency, SourceManager->GetFloatCompareTolerance()))
		{
			LPFFrequency = InLPFFrequency;
			SourceManager->SetLPFFrequency(SourceId, LPFFrequency);
		}
	}

	void FMixerSourceVoice::SetHPFFrequency(const float InHPFFrequency)
	{
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		if (!FMath::IsNearlyEqual(HPFFrequency, InHPFFrequency, SourceManager->GetFloatCompareTolerance()))
		{
			HPFFrequency = InHPFFrequency;
			SourceManager->SetHPFFrequency(SourceId, HPFFrequency);
		}
	}

	void FMixerSourceVoice::SetModVolume(const float InVolumeModBase)
	{
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		if (!FMath::IsNearlyEqual(VolumeModBase, InVolumeModBase, SourceManager->GetFloatCompareTolerance()))
		{
			VolumeModBase = InVolumeModBase;
			SourceManager->SetModVolume(SourceId, VolumeModBase);
		}
	}

	void FMixerSourceVoice::SetModPitch(const float InPitchModBase)
	{
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		if (!FMath::IsNearlyEqual(PitchModBase, InPitchModBase, SourceManager->GetFloatCompareTolerance()))	
		{
			PitchModBase = InPitchModBase;
			SourceManager->SetModPitch(SourceId, InPitchModBase);
		}
	}

	void FMixerSourceVoice::SetModHPFFrequency(const float InHPFFrequencyModBase)
	{
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		if (!FMath::IsNearlyEqual(HPFFrequencyModBase, InHPFFrequencyModBase, SourceManager->GetFloatCompareTolerance()))	
		{
			HPFFrequencyModBase = InHPFFrequencyModBase;
			SourceManager->SetModHPFFrequency(SourceId, InHPFFrequencyModBase);
		}
	}

	void FMixerSourceVoice::SetModLPFFrequency(const float InLPFFrequencyModBase)
	{
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		if (!FMath::IsNearlyEqual(LPFFrequencyModBase, InLPFFrequencyModBase, SourceManager->GetFloatCompareTolerance()))	
		{
			LPFFrequencyModBase = InLPFFrequencyModBase;
			SourceManager->SetModLPFFrequency(SourceId, InLPFFrequencyModBase);
		}
	}

	void FMixerSourceVoice::SetModulationRouting(FSoundModulationDefaultRoutingSettings& RoutingSettings)
	{
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		SourceManager->SetModulationRouting(SourceId, RoutingSettings);
	}

	void FMixerSourceVoice::SetSourceBufferListener(FSharedISourceBufferListenerPtr& InSourceBufferListener, bool InShouldSourceBufferListenerZeroBuffer)
	{
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		SourceManager->SetSourceBufferListener(SourceId, InSourceBufferListener, InShouldSourceBufferListenerZeroBuffer);
	}

	void FMixerSourceVoice::SetChannelMap(const uint32 NumInputChannels, const Audio::FAlignedFloatBuffer& InChannelMap, const bool bInIs3D, const bool bInIsCenterChannelOnly)
	{
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		SourceManager->SetChannelMap(SourceId, NumInputChannels, InChannelMap, bInIs3D, bInIsCenterChannelOnly);
	}

	void FMixerSourceVoice::SetSpatializationParams(const FSpatializationParams& InParams)
	{
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		SourceManager->SetSpatializationParams(SourceId, InParams);
	}

	void FMixerSourceVoice::Play()
	{
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		bIsPlaying = true;
		bIsPaused = false;
		bIsActive = true;

		SourceManager->Play(SourceId);
	}

	void FMixerSourceVoice::Stop()
	{
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		bIsPlaying = false;
		bIsPaused = false;
		bIsActive = false;
		// We are instantly fading out with this stop command
		bStopFadedOut = true;
		SourceManager->Stop(SourceId);
	}

	void FMixerSourceVoice::StopFade(int32 NumFrames)
	{
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		bIsPaused = false;
		SourceManager->StopFade(SourceId, NumFrames);
	}

	int32 FMixerSourceVoice::GetSourceId() const
	{
		return SourceId;
	}

	float FMixerSourceVoice::GetDistanceAttenuation() const
	{
		return DistanceAttenuation;
	}

	float FMixerSourceVoice::GetDistance() const
	{
		return Distance;
	}

	void FMixerSourceVoice::Pause()
	{
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		bIsPaused = true;
		bIsActive = false;
		SourceManager->Pause(SourceId);
	}

	bool FMixerSourceVoice::IsPlaying() const
	{
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		return bIsPlaying;
	}

	bool FMixerSourceVoice::IsPaused() const
	{
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		return bIsPaused;
	}

	bool FMixerSourceVoice::IsActive() const
	{
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		return bIsActive;
	}

	bool FMixerSourceVoice::NeedsSpeakerMap() const
	{
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		return SourceManager->NeedsSpeakerMap(SourceId);
	}

	bool FMixerSourceVoice::IsUsingHRTFSpatializer(bool bDefaultValue) const
	{
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		if (SourceId != INDEX_NONE)
		{
			return SourceManager->IsUsingHRTFSpatializer(SourceId);
		}

		return bDefaultValue;
	}

	int64 FMixerSourceVoice::GetNumFramesPlayed() const
	{
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		return SourceManager->GetNumFramesPlayed(SourceId);
	}

	float FMixerSourceVoice::GetEnvelopeValue() const
	{
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		return SourceManager->GetEnvelopeValue(SourceId);
	}

	float FMixerSourceVoice::GetVolumeModulationValue() const
	{
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		return SourceManager->GetVolumeModulationValue(SourceId);
	}

#if ENABLE_AUDIO_DEBUG
	double FMixerSourceVoice::GetCPUCoreUtilization() const
	{
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		return SourceManager->GetCPUCoreUtilization(SourceId);
	}
#endif // ENABLE_AUDIO_DEBUG

	float FMixerSourceVoice::GetRelativeRenderCost() const
	{
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		return SourceManager->GetRelativeRenderCost(SourceId);
	}

	void FMixerSourceVoice::SetSubmixSendInfo(FMixerSubmixWeakPtr Submix, const float SendLevel, const EMixerSourceSubmixSendStage SendStage/* = EMixerSourceSubmixSendStage::PostDistanceAttenuation*/, const FSendFilterSettings& InFilter/* = FSendFilterSettings{}*/)
	{
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		FMixerSubmixPtr SubmixPtr = Submix.Pin();
		if (SubmixPtr.IsValid())
		{
			FMixerSourceSubmixSend* SubmixSend = SubmixSends.Find(SubmixPtr->GetId());

			if (!SubmixSend)
			{
				FMixerSourceSubmixSend NewSubmixSend;
				NewSubmixSend.Submix = Submix;
				NewSubmixSend.SendLevel = SendLevel;
				NewSubmixSend.bIsMainSend = false;
				NewSubmixSend.SubmixSendStage = SendStage;
				NewSubmixSend.Filter = InFilter;

				SubmixSends.Add(SubmixPtr->GetId(), NewSubmixSend);
				SourceManager->SetSubmixSendInfo(SourceId, NewSubmixSend);
			}
			else if (!FMath::IsNearlyEqual(SubmixSend->SendLevel, SendLevel, SourceManager->GetFloatCompareTolerance())
				|| SubmixSend->SubmixSendStage != SendStage
				|| SubmixSend->Filter.HasChanged(InFilter))
			{
				SubmixSend->SendLevel = SendLevel;
				SubmixSend->SubmixSendStage = SendStage;
				SubmixSend->Filter = InFilter;
				SourceManager->SetSubmixSendInfo(SourceId, *SubmixSend);
			}
		}
	}

	void FMixerSourceVoice::ClearSubmixSendInfo(FMixerSubmixWeakPtr Submix)
	{
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		FMixerSubmixPtr SubmixPtr = Submix.Pin();
		if (SubmixPtr.IsValid())
		{
			FMixerSourceSubmixSend* SubmixSend = SubmixSends.Find(SubmixPtr->GetId());
			if (SubmixSend)
			{
				SourceManager->ClearSubmixSendInfo(SourceId, *SubmixSend);
				SubmixSends.Remove(SubmixPtr->GetId());
			}
		}
	}

	void FMixerSourceVoice::SetOutputToBusOnly(bool bInOutputToBusOnly)
	{
		if (bInOutputToBusOnly)
		{
			bEnableBusSends = true;
		}

		bEnableBaseSubmix = !bInOutputToBusOnly;
		bEnableSubmixSends = !bInOutputToBusOnly;
	}

	void FMixerSourceVoice::SetEnablement(bool bInEnableBusSendRouting, bool bInEnableMainSubmixOutput, bool bInEnableSubmixSendRouting)
	{
		bEnableBusSends = bInEnableBusSendRouting;
		bEnableBaseSubmix = bInEnableMainSubmixOutput;
		bEnableSubmixSends = bInEnableSubmixSendRouting;
	}


	void FMixerSourceVoice::SetAudioBusSendInfo(EBusSendType InBusSendType, uint32 AudioBusId, float BusSendLevel, const FString& InBusName, const FSendFilterSettings& InFilter)
	{
		AUDIO_MIXER_CHECK_GAME_THREAD(MixerDevice);

		if (!bEnableBusSends)
		{
			BusSendLevel = 0.0f;
		}

		SourceManager->SetBusSendInfo(SourceId, InBusSendType, AudioBusId, BusSendLevel, InBusName, InFilter);
	}

	bool FMixerSourceVoice::IsRenderingToSubmixes() const
	{
		return bEnableBaseSubmix || bEnableSubmixSends || bHasAttenuationSubmixSends;
	}
}
