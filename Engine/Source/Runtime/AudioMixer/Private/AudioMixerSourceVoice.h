// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioMixerBuffer.h"
#include "AudioMixerSourceManager.h"

namespace Audio
{
	struct FMixerSourceVoiceFilterParams;
	struct FMixerSourceVoiceInitParams;
	class FMixerDevice;
	class FMixerSubmix;
	class FMixerSource;
	class FMixerSourceManager;
	class ISourceBufferQueueListener;


	class FMixerSourceVoice
	{
	public:
		FMixerSourceVoice();
		~FMixerSourceVoice();

		// Resets the source voice state
		void Reset(FMixerDevice* InMixerDevice);

		// Initializes the mixer source voice
		bool Init(const FMixerSourceVoiceInitParams& InFormat);

		// Releases the source voice back to the source buffer pool
		void Release();

		// Sets the source voice pitch value.
		void SetPitch(const float InPitch);

		// Sets the source voice volume value.
		void SetVolume(const float InVolume);

		// Sets the source voice distance attenuation.
		void SetDistanceAttenuation(const float InDistanceAttenuation);
		
		// Sets the source voice's LPF filter frequency.
		void SetLPFFrequency(const float InFrequency);

		// Sets the source voice's HPF filter frequency.
		void SetHPFFrequency(const float InFrequency);

		// Sets the source voice modulation base pitch value.
		void SetModPitch(const float InPitch);

		// Sets the source voice's volume modulation base frequency.
		void SetModVolume(const float InVolume);

		// Sets the source voice's LPF filter modulation base frequency.
		void SetModLPFFrequency(const float InFrequency);

		// Sets the source voice's HPF filter modulation base frequency.
		void SetModHPFFrequency(const float InFrequency);

		void SetModulationRouting(FSoundModulationDefaultRoutingSettings& RoutingSettings);

		// Set the source voice's SourceBufferListener and associated boolean.
		void SetSourceBufferListener(FSharedISourceBufferListenerPtr& InSourceBufferListener, bool InShouldSourceBufferListenerZeroBuffer);

		// Sets the source voice's channel map (2d or 3d).
		void SetChannelMap(const uint32 NumInputChannels, const Audio::FAlignedFloatBuffer& InChannelMap, const bool bInIs3D, const bool bInIsCenterChannelOnly);

		// Sets params used by HRTF spatializer
		void SetSpatializationParams(const FSpatializationParams& InParams);

		// Starts the source voice generating audio output into it's submix.
		void Play();

		// Pauses the source voice (i.e. stops generating output but keeps its state as "active and playing". Can be restarted.)
		void Pause();

		// Immediately stops the source voice (no longer playing or active, can't be restarted.)
		void Stop();

		// Does a faded stop (to avoid discontinuity)
		void StopFade(int32 NumFrames);

		// Returns the source's Id
		int32 GetSourceId() const;

		// Returns the source's distance attenuation
		float GetDistanceAttenuation() const;

		// Returns the source's distance from the closest listener
		float GetDistance() const;

		// Queries if the voice is playing
		bool IsPlaying() const;

		// Queries if the voice is paused
		bool IsPaused() const;

		// Queries if the source voice is active.
		bool IsActive() const;

		// Queries if the source has finished its fade out.
		bool IsStopFadedOut() const { return bStopFadedOut; }

		// Whether or not the device changed and needs another speaker map sent
		bool NeedsSpeakerMap() const;

		// Whether or not the voice is currently using HRTF spatialization.
		//
		// @param bDefaultValue - This value will be returned if voice does not have a valid source id.
		bool IsUsingHRTFSpatializer(bool bDefaultValue) const;

		// Retrieves the total number of samples played.
		int64 GetNumFramesPlayed() const;

		// Retrieves the envelope value of the source.
		float GetEnvelopeValue() const;

		// Retrieves final mixed Modulation value of the source.
		float GetVolumeModulationValue() const;

#if ENABLE_AUDIO_DEBUG
		double GetCPUCoreUtilization() const;
#endif // ENABLE_AUDIO_DEBUG

		// Retrieves the current "render cost" of the mixer source voice. Used for debug display and for limiting voice count.
		// 1.0 is equivalent to a single decoding sound source. 
		// Useful as a metric more human-centric than CPU utilization.
		float GetRelativeRenderCost() const;

		// Sets the send level and optional filter for a single submix send
		void SetSubmixSendInfo(FMixerSubmixWeakPtr Submix, const float SendLevel, const EMixerSourceSubmixSendStage SendStage = EMixerSourceSubmixSendStage::PostDistanceAttenuation, const FSendFilterSettings& InFilter = FSendFilterSettings{});

		// Clears the submix send to the given submix
		void ClearSubmixSendInfo(FMixerSubmixWeakPtr Submix);

		// Sets whether or not we are enabling sending audio to submixes (we could be sending audio to source buses though).
		void SetOutputToBusOnly(bool bInOutputToBusOnly);

		//Updates internal settings on which output types are enabled
		void SetEnablement(bool bInEnableBusSendRouting, bool bInEnableMainSubmixOutput, bool bInEnableSubmixSendRouting);

		// Set the source bus send levels
		void SetAudioBusSendInfo(EBusSendType InBusSendType, uint32 AudioBusId, float BusSendLevel, const FString& InBusName, const FSendFilterSettings& InFilter = FSendFilterSettings{});

	private:

		friend class FMixerSourceManager;

		FMixerSourceManager* SourceManager;
		TMap<uint32, FMixerSourceSubmixSend> SubmixSends;
		FMixerDevice* MixerDevice;
		TArray<float> DeviceChannelMap;
		FThreadSafeBool bStopFadedOut;
		float Pitch;
		float Volume;
		float DistanceAttenuation;
		float Distance;
		float LPFFrequency;
		float HPFFrequency;
		float PitchModBase;
		float VolumeModBase;
		float LPFFrequencyModBase;
		float HPFFrequencyModBase;
		int32 SourceId;
		uint16 bIsPlaying : 1;
		uint16 bIsPaused : 1;
		uint16 bIsActive : 1;
		uint16 bIsBus : 1;
		uint16 bEnableBusSends : 1;
		uint16 bEnableBaseSubmix : 1;
		uint16 bEnableSubmixSends : 1;
		uint16 bHasAttenuationSubmixSends : 1;

		bool IsRenderingToSubmixes() const;
	};

}
