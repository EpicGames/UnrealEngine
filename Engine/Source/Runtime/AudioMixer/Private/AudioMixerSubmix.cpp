// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioMixerSubmix.h"

#include "Async/Async.h"
#include "Audio/AudioDebug.h"
#include "AudioDeviceManager.h"
#include "AudioMixerDevice.h"
#include "AudioMixerSourceVoice.h"
#include "AudioThread.h"
#include "DSP/FloatArrayMath.h"
#include "ISubmixBufferListener.h"
#include "Sound/SoundEffectPreset.h"
#include "Sound/SoundEffectSubmix.h"
#include "Sound/SoundModulationDestination.h"
#include "Sound/SoundSubmix.h"
#include "Sound/SoundSubmixSend.h"
#include "Misc/ScopeTryLock.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "AudioLinkLog.h"
#include "DSP/BufferDiagnostics.h"
#include "Algo/Accumulate.h"

// Link to "Audio" profiling category
CSV_DECLARE_CATEGORY_MODULE_EXTERN(AUDIOMIXERCORE_API, Audio);

static int32 RecoverRecordingOnShutdownCVar = 0;
FAutoConsoleVariableRef CVarRecoverRecordingOnShutdown(
	TEXT("au.RecoverRecordingOnShutdown"),
	RecoverRecordingOnShutdownCVar,
	TEXT("When set to 1, we will attempt to bounce the recording to a wav file if the game is shutdown while a recording is in flight.\n")
	TEXT("0: Disabled, 1: Enabled"),
	ECVF_Default);

static int32 BypassAllSubmixEffectsCVar = 0;
FAutoConsoleVariableRef CVarBypassAllSubmixEffects(
	TEXT("au.BypassAllSubmixEffects"),
	BypassAllSubmixEffectsCVar,
	TEXT("When set to 1, all submix effects will be bypassed.\n")
	TEXT("1: Submix Effects are disabled."),
	ECVF_Default);

static int32 LogSubmixEnablementCVar = 0;
FAutoConsoleVariableRef CVarLogSubmixEnablement(
	TEXT("au.LogSubmixAutoDisable"),
	LogSubmixEnablementCVar,
	TEXT("Enables logging of submix disable and enable state.\n")
	TEXT("1: Submix enablement logging is on. 0: Submix enablement/disablement logging is off."),
	ECVF_Default);

static bool bNeverDisableSubmixes = false;
static FAutoConsoleVariableRef CVarNeverDisableSubmixes(
	TEXT("au.NeverDisableSubmixes"),
	bNeverDisableSubmixes,
	TEXT("Forces submixes to stay enabled and ignore the bAutoDisable value for situations where rendering silence is required instead of no audio rendering\n")
	TEXT("0: Submixes use auto disable as normal\n")
	TEXT("1: Submixes always render silence instead of disabling\n"),
	ECVF_Default
);

namespace MixerSubmixCVars
{
	bool bRemoveModulatorsOnDisabledSubmixes = false;
	FAutoConsoleVariableRef CVarRemoveModulatorsOnDisabledSubmixes(
		TEXT("au.RemoveModulatorsOnDisabledSubmixes"),
		bRemoveModulatorsOnDisabledSubmixes,
		TEXT("When true, modulators are released on a submix while it is auto-disabled and restored from the USoundSubmix asset on re-enable. Note: this discards any ")
		TEXT("runtime modulator overrides applied via UpdateSubmixModulationSettings. Enable only if needed as a workaround for legacy behavior."),
		ECVF_Default);
}

// Define profiling categories for submixes. 
DEFINE_STAT(STAT_AudioMixerEndpointSubmixes);
DEFINE_STAT(STAT_AudioMixerSubmixes);
DEFINE_STAT(STAT_AudioMixerSubmixChildren);
DEFINE_STAT(STAT_AudioMixerSubmixSource);
DEFINE_STAT(STAT_AudioMixerSubmixEffectProcessing);
DEFINE_STAT(STAT_AudioMixerSubmixBufferListeners);
DEFINE_STAT(STAT_AudioMixerSubmixSoundfieldChildren);
DEFINE_STAT(STAT_AudioMixerSubmixSoundfieldSources);
DEFINE_STAT(STAT_AudioMixerSubmixSoundfieldProcessors);

#if UE_AUDIO_PROFILERTRACE_ENABLED

UE_TRACE_EVENT_BEGIN(Audio, SubmixLoaded)
	UE_TRACE_EVENT_FIELD(uint32, DeviceId)
	UE_TRACE_EVENT_FIELD(uint32, SubmixId)
	UE_TRACE_EVENT_FIELD(uint64, Timestamp)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Name)
	UE_TRACE_EVENT_FIELD(int32, NumChannels)
	UE_TRACE_EVENT_FIELD(bool, IsMainSubmix)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Audio, SubmixHasActivity)
	UE_TRACE_EVENT_FIELD(uint32, DeviceId)
	UE_TRACE_EVENT_FIELD(uint32, SubmixId)
	UE_TRACE_EVENT_FIELD(uint64, Timestamp)
	UE_TRACE_EVENT_FIELD(bool, HasActivity)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Audio, SubmixEnvelopeFollowerEnabled)
	UE_TRACE_EVENT_FIELD(uint32, DeviceId)
	UE_TRACE_EVENT_FIELD(uint32, SubmixId)
	UE_TRACE_EVENT_FIELD(uint64, Timestamp)
	UE_TRACE_EVENT_FIELD(bool, EnvelopeFollowerEnabled)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Audio, SubmixEnvelopeValues)
	UE_TRACE_EVENT_FIELD(uint32, DeviceId)
	UE_TRACE_EVENT_FIELD(uint32, SubmixId)
	UE_TRACE_EVENT_FIELD(uint64, Timestamp)
	UE_TRACE_EVENT_FIELD(float[], EnvelopeValues)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Audio, SubmixUnloaded)
	UE_TRACE_EVENT_FIELD(uint32, DeviceId)
	UE_TRACE_EVENT_FIELD(uint32, SubmixId)
	UE_TRACE_EVENT_FIELD(uint64, Timestamp)
UE_TRACE_EVENT_END()

#endif // UE_AUDIO_PROFILERTRACE_ENABLED

namespace Audio
{
	namespace MixerSubmixIntrinsics
	{

		FSpectrumAnalyzerSettings::EFFTSize GetSpectrumAnalyzerFFTSize(EFFTSize InFFTSize)
		{
			switch (InFFTSize)
			{
				case EFFTSize::DefaultSize:
					return FSpectrumAnalyzerSettings::EFFTSize::Default;

				case EFFTSize::Min:
					return FSpectrumAnalyzerSettings::EFFTSize::Min_64;

				case EFFTSize::Small:
					return FSpectrumAnalyzerSettings::EFFTSize::Small_256;

				case EFFTSize::Medium:
					return FSpectrumAnalyzerSettings::EFFTSize::Medium_512;

				case EFFTSize::Large:
					return FSpectrumAnalyzerSettings::EFFTSize::Large_1024;

				case EFFTSize::VeryLarge:
					return FSpectrumAnalyzerSettings::EFFTSize::VeryLarge_2048;

				case EFFTSize::Max:
					return FSpectrumAnalyzerSettings::EFFTSize::TestLarge_4096;

				default:
					return FSpectrumAnalyzerSettings::EFFTSize::Default;
			}
		}

		EWindowType GetWindowType(EFFTWindowType InWindowType)
		{
			switch (InWindowType)
			{
				case EFFTWindowType::None:
					return EWindowType::None;

				case EFFTWindowType::Hamming:
					return EWindowType::Hamming;

				case EFFTWindowType::Hann:
					return EWindowType::Hann;

				case EFFTWindowType::Blackman:
					return EWindowType::Blackman;

				default:
					return EWindowType::None;
			}
		}

		FSpectrumBandExtractorSettings::EMetric GetExtractorMetric(EAudioSpectrumType InSpectrumType)
		{
			using EMetric = FSpectrumBandExtractorSettings::EMetric;

			switch (InSpectrumType)
			{
				case EAudioSpectrumType::MagnitudeSpectrum:
					return EMetric::Magnitude;

				case EAudioSpectrumType::PowerSpectrum:
					return EMetric::Power;

				case EAudioSpectrumType::Decibel:
				default:
					return EMetric::Decibel;
			}
		}

		ISpectrumBandExtractor::EBandType GetExtractorBandType(EFFTPeakInterpolationMethod InMethod)
		{
			using EBandType = ISpectrumBandExtractor::EBandType;

			switch (InMethod)
			{
				case EFFTPeakInterpolationMethod::NearestNeighbor:
					return EBandType::NearestNeighbor;

				case EFFTPeakInterpolationMethod::Linear:
					return EBandType::Lerp;

				case EFFTPeakInterpolationMethod::Quadratic:
					return EBandType::Quadratic;

				case EFFTPeakInterpolationMethod::ConstantQ:
				default:
					return EBandType::ConstantQ;
			}
		}
	}

	namespace FMixerSubmixPrivate
	{
		bool IsAudioBufferSilent(const FAlignedFloatBuffer& InAudioBuffer)
		{
			// Extremely cheap silent buffer detection: as soon as we hit a sample which isn't silent, we flag we're not silent
			bool bIsSilent = true;

			int32 Index = 0;

#if PLATFORM_ENABLE_VECTORINTRINSICS
			const int32 SimdNum = InAudioBuffer.Num() & 0xFFFFFFF0;
			for (; Index < SimdNum; Index += 16)
			{
				const VectorRegister4x4Float Samples = VectorLoad16(&InAudioBuffer[Index]);

				if (VectorAnyGreaterThan(VectorAbs(Samples.val[0]), GlobalVectorConstants::SmallNumber) ||
					VectorAnyGreaterThan(VectorAbs(Samples.val[1]), GlobalVectorConstants::SmallNumber) ||
					VectorAnyGreaterThan(VectorAbs(Samples.val[2]), GlobalVectorConstants::SmallNumber) ||
					VectorAnyGreaterThan(VectorAbs(Samples.val[3]), GlobalVectorConstants::SmallNumber))
				{
					bIsSilent = false;
					Index = INT_MAX;
					break;
				}
			}
#endif

			// Finish to the end of the buffer or check each sample if vector intrinsics are disabled
			for (; Index < InAudioBuffer.Num(); ++Index)
			{
				// As soon as we hit a non-silent sample, we're not silent
				if (FMath::Abs(InAudioBuffer[Index]) > SMALL_NUMBER)
				{
					bIsSilent = false;
					break;
				}
			}

			return bIsSilent;
		}
	}

	// Unique IDs for mixer submixes
	static uint32 GSubmixMixerIDs = 0;

	FMixerSubmix::FMixerSubmix(FMixerDevice* InMixerDevice)
		: Id(GSubmixMixerIDs++)
		, ParentSubmix(nullptr)
		, MixerDevice(InMixerDevice)
		, NumChannels(0)
		, NumSamples(0)
		, CurrentOutputVolume(1.0f)
		, TargetOutputVolume(1.0f)
		, CurrentWetLevel(1.0f)
		, TargetWetLevel(1.0f)
		, CurrentDryLevel(0.0f)
		, TargetDryLevel(0.0f)
		, EnvelopeNumChannels(0)
		, NumSubmixEffects(0)
		, bIsRecording(false)
		, bIsBackgroundMuted(false)
		, bAutoDisable(true)
		, bIsSilent(false)
		, bIsCurrentlyDisabled(false)
		, AutoDisableTime(0.1)
		, SilenceTimeStartSeconds(-1.0)
		, bIsSpectrumAnalyzing(false)
	{
	}

	FMixerSubmix::~FMixerSubmix()
	{
		ClearSoundEffectSubmixes();

		if (RecoverRecordingOnShutdownCVar && OwningSubmixObject.IsValid() && bIsRecording)
		{
			FString InterruptedFileName = "InterruptedRecording.wav";
			UE_LOGF(LogAudioMixer, Warning, "Recording of Submix %ls was interrupted. Saving interrupted recording as %ls.", *(OwningSubmixObject->GetName()), *InterruptedFileName);
			if (const USoundSubmix* SoundSubmix = Cast<const USoundSubmix>(OwningSubmixObject))
			{
				USoundSubmix* MutableSubmix = const_cast<USoundSubmix*>(SoundSubmix);
				MutableSubmix->StopRecordingOutput(MixerDevice, EAudioRecordingExportType::WavFile, InterruptedFileName, FString());
			}
		}

#if UE_AUDIO_PROFILERTRACE_ENABLED
		const bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AudioChannel);
		if (bChannelEnabled)
		{
			UE_TRACE_LOG(Audio, SubmixUnloaded, AudioChannel)
				<< SubmixUnloaded.DeviceId(MixerDevice->DeviceID)
				<< SubmixUnloaded.SubmixId(GetTypeHash(OwningSubmixObject->GetPathName()))
				<< SubmixUnloaded.Timestamp(FPlatformTime::Cycles64());
		}
#endif // UE_AUDIO_PROFILERTRACE_ENABLED
	}

	void FMixerSubmix::Init(const USoundSubmixBase* InSoundSubmix, bool bAllowReInit)
	{
		check(IsInAudioThread());
		if (InSoundSubmix != nullptr)
		{
			SubmixName = InSoundSubmix->GetName();

#if UE_AUDIO_PROFILERTRACE_ENABLED
			SubmixPathName = InSoundSubmix->GetPathName();
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

			// This is a first init and needs to be synchronous
			if (!OwningSubmixObject.IsValid())
			{
				OwningSubmixObject = InSoundSubmix;
				InitInternal();
			}
			// This is a re-init and needs to be thread safe
			else if (bAllowReInit)
			{
				check(OwningSubmixObject == InSoundSubmix);
				SubmixCommand([this]()
				{
					InitInternal();
				});
			}
			else if (const USoundfieldSubmix* SoundfieldSubmix = Cast<const USoundfieldSubmix>(OwningSubmixObject))
			{
				ISoundfieldFactory* SoundfieldFactory = SoundfieldSubmix->GetSoundfieldFactoryForSubmix();
				const USoundfieldEncodingSettingsBase* EncodingSettings = SoundfieldSubmix->GetSoundfieldEncodingSettings();

				TArray<USoundfieldEffectBase*> Effects = SoundfieldSubmix->GetSoundfieldProcessors();
				SetupSoundfieldStreams(EncodingSettings, Effects, SoundfieldFactory);
			}
		}

#if UE_AUDIO_PROFILERTRACE_ENABLED
		const bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AudioChannel);
		if (bChannelEnabled)
		{
			UE_TRACE_LOG(Audio, SubmixLoaded, AudioChannel)
				<< SubmixLoaded.DeviceId(MixerDevice->DeviceID)
				<< SubmixLoaded.SubmixId(GetTypeHash(OwningSubmixObject->GetPathName()))
				<< SubmixLoaded.Timestamp(FPlatformTime::Cycles64())
				<< SubmixLoaded.Name(*OwningSubmixObject->GetPathName())
				<< SubmixLoaded.NumChannels(NumChannels)
				<< SubmixLoaded.IsMainSubmix(InSoundSubmix == &MixerDevice->GetMainSubmixObject());
		}
#endif // UE_AUDIO_PROFILERTRACE_ENABLED
	}

	void FMixerSubmix::InitInternal()
	{
		// Loop through the submix's presets and make new instances of effects in the same order as the presets
		ClearSoundEffectSubmixes();

		// Copy any base data
		bAutoDisable = OwningSubmixObject->bAutoDisable;
		AutoDisableTime = (double)OwningSubmixObject->AutoDisableTime;
		bIsSilent = false;
		bIsCurrentlyDisabled = false;
		SilenceTimeStartSeconds = -1.0;

		if (MixerDevice->IsModulationPluginEnabled() && MixerDevice->ModulationInterface.IsValid())
		{
			VolumeMod.Init(MixerDevice->DeviceID, FName("Volume"), false /* bInIsBuffered */, true /* bInValueLinear */);
			WetLevelMod.Init(MixerDevice->DeviceID, FName("Volume"), false /* bInIsBuffered */, true /* bInValueLinear */);
			DryLevelMod.Init(MixerDevice->DeviceID, FName("Volume"), false /* bInIsBuffered */, true /* bInValueLinear */);
		}

		if (const USoundSubmix* SoundSubmix = Cast<const USoundSubmix>(OwningSubmixObject))
		{
			VolumeModBaseDb = FMath::Clamp(SoundSubmix->OutputVolumeModulation.Value, MIN_VOLUME_DECIBELS, 0.0f);
			WetModBaseDb = FMath::Clamp(SoundSubmix->WetLevelModulation.Value, MIN_VOLUME_DECIBELS, 0.0f);
			DryModBaseDb = FMath::Clamp(SoundSubmix->DryLevelModulation.Value, MIN_VOLUME_DECIBELS, 0.0f);

			TargetOutputVolume = Audio::ConvertToLinear(VolumeModBaseDb);
			TargetWetLevel = Audio::ConvertToLinear(WetModBaseDb);
			TargetDryLevel = Audio::ConvertToLinear(DryModBaseDb);

			CurrentOutputVolume = TargetOutputVolume;
			CurrentDryLevel = TargetDryLevel;
			CurrentWetLevel = TargetWetLevel;

			if (MixerDevice->IsModulationPluginEnabled() && MixerDevice->ModulationInterface.IsValid())
			{
				TSet<TObjectPtr<USoundModulatorBase>> VolumeModulator = SoundSubmix->OutputVolumeModulation.Modulators;
				TSet<TObjectPtr<USoundModulatorBase>> WetLevelModulator = SoundSubmix->WetLevelModulation.Modulators;
				TSet<TObjectPtr<USoundModulatorBase>> DryLevelModulator = SoundSubmix->DryLevelModulation.Modulators;

				// Queue this up to happen after submix init, when the mixer device has finished being added to the device manager
				SubmixCommand([this, VolMod = MoveTemp(VolumeModulator), WetMod = MoveTemp(WetLevelModulator), DryMod = MoveTemp(DryLevelModulator)]() mutable
				{
					UpdateModulationSettings(VolMod, WetMod, DryMod);
				});
			}
			
			// AudioLink send enabled? 
			if (SoundSubmix->bSendToAudioLink)
			{
				// If AudioLink is active, create a link.
				if (IAudioLinkFactory* LinkFactory = MixerDevice->GetAudioLinkFactory())
				{
					check(LinkFactory->GetSettingsClass());
					check(LinkFactory->GetSettingsClass()->GetDefaultObject());

					const UAudioLinkSettingsAbstract* Settings = !SoundSubmix->AudioLinkSettings ?
						GetDefault<UAudioLinkSettingsAbstract>(LinkFactory->GetSettingsClass()) : SoundSubmix->AudioLinkSettings.Get();

					AudioLinkInstance = LinkFactory->CreateSubmixAudioLink({ SoundSubmix, MixerDevice, Settings });
				}
			}
			
			FScopeLock ScopeLock(&EffectChainMutationCriticalSection);
			{
    			NumSubmixEffects = 0;
				EffectChains.Reset();

				if (SoundSubmix->SubmixEffectChain.Num() > 0)
				{
					FSubmixEffectFadeInfo NewEffectFadeInfo;
					NewEffectFadeInfo.FadeVolume = FDynamicParameter(1.0f);
					NewEffectFadeInfo.bIsCurrentChain = true;
					NewEffectFadeInfo.bIsBaseEffect = true;

					for (USoundEffectSubmixPreset* EffectPreset : SoundSubmix->SubmixEffectChain)
					{
						if (EffectPreset)
						{
							++NumSubmixEffects;

							FSoundEffectSubmixInitData InitData;
							InitData.DeviceID = MixerDevice->DeviceID;
							InitData.SampleRate = MixerDevice->GetSampleRate();
							InitData.PresetSettings = nullptr;
							InitData.ParentPresetUniqueId = EffectPreset->GetUniqueID();

							// Create a new effect instance using the preset & enable
							TSoundEffectSubmixPtr SubmixEffect = USoundEffectPreset::CreateInstance<FSoundEffectSubmixInitData, FSoundEffectSubmix>(InitData, *EffectPreset);
							SubmixEffect->SetEnabled(true);

							// Add the effect to this submix's chain
							NewEffectFadeInfo.EffectChain.Add(SubmixEffect);
						}
					}

					EffectChains.Add(NewEffectFadeInfo);
				}
			}

			NumChannels = MixerDevice->GetNumDeviceChannels();
			const int32 NumOutputFrames = MixerDevice->GetNumOutputFrames();
			NumSamples = NumChannels * NumOutputFrames;
		}
		else if (const USoundfieldSubmix* SoundfieldSubmix = Cast<const USoundfieldSubmix>(OwningSubmixObject))
		{
			ISoundfieldFactory* SoundfieldFactory = SoundfieldSubmix->GetSoundfieldFactoryForSubmix();
			const USoundfieldEncodingSettingsBase* EncodingSettings = SoundfieldSubmix->GetSoundfieldEncodingSettings();

			TArray<USoundfieldEffectBase*> Effects = SoundfieldSubmix->GetSoundfieldProcessors();
			SetupSoundfieldStreams(EncodingSettings, Effects, SoundfieldFactory);
		}
		else if (const UEndpointSubmix* EndpointSubmix = Cast<const UEndpointSubmix>(OwningSubmixObject))
		{
			NumChannels = MixerDevice->GetNumDeviceChannels();
			const int32 NumOutputFrames = MixerDevice->GetNumOutputFrames();
			NumSamples = NumChannels * NumOutputFrames;

			IAudioEndpointFactory* EndpointFactory = EndpointSubmix->GetAudioEndpointForSubmix();
			const UAudioEndpointSettingsBase* EndpointSettings = EndpointSubmix->GetEndpointSettings();

			SetupEndpoint(EndpointFactory, EndpointSettings);
		}
		else if (const USoundfieldEndpointSubmix* SoundfieldEndpointSubmix = Cast<const USoundfieldEndpointSubmix>(OwningSubmixObject))
		{
 			ISoundfieldEndpointFactory* SoundfieldFactory = SoundfieldEndpointSubmix->GetSoundfieldEndpointForSubmix();
			const USoundfieldEncodingSettingsBase* EncodingSettings = SoundfieldEndpointSubmix->GetEncodingSettings();

			if (!SoundfieldFactory)
			{
				UE_LOGF(LogAudio, Display, "Wasn't able to set up soundfield format for submix %ls. Sending to default output.", *OwningSubmixObject->GetName());
				return;
			}

			if (!EncodingSettings)
			{
				EncodingSettings = SoundfieldFactory->GetDefaultEncodingSettings();

				if (!ensureMsgf(EncodingSettings, TEXT("Soundfield Endpoint %s did not return default encoding settings! Is ISoundfieldEndpointFactory::GetDefaultEncodingSettings() implemented?"), *SoundfieldFactory->GetEndpointTypeName().ToString()))
				{
					return;
				}
			}

			TArray<USoundfieldEffectBase*> Effects = SoundfieldEndpointSubmix->GetSoundfieldProcessors();

			SetupSoundfieldStreams(EncodingSettings, Effects, SoundfieldFactory);

			if (IsSoundfieldSubmix())
			{
				const USoundfieldEndpointSettingsBase* EndpointSettings = SoundfieldEndpointSubmix->GetEndpointSettings();

				if (!EndpointSettings)
				{
					EndpointSettings = SoundfieldFactory->GetDefaultEndpointSettings();

					if (!ensureMsgf(EndpointSettings, TEXT("Soundfield Endpoint %s did not return default endpoint settings! Is ISoundfieldEndpointFactory::GetDefaultEndpointSettings() implemented?"), *SoundfieldFactory->GetEndpointTypeName().ToString()))
					{
						return;
					}
				}

				SetupEndpoint(SoundfieldFactory, EndpointSettings);
			}
			else
			{
				UE_LOGF(LogAudio, Display, "Wasn't able to set up soundfield format for submix %ls. Sending to default output.", *OwningSubmixObject->GetName());
				SoundfieldStreams.Reset();
			}
		}
		else
		{
			// If we've arrived here, we couldn't identify the type of the submix we're initializing.
			checkNoEntry();
		}
	}

	void FMixerSubmix::DownmixBuffer(const int32 InChannels, const FAlignedFloatBuffer& InBuffer, const int32 OutChannels, FAlignedFloatBuffer& OutNewBuffer)
	{
		Audio::FAlignedFloatBuffer MixdownGainsMap;
		Audio::FMixerDevice::Get2DChannelMap(false, InChannels, OutChannels, false, MixdownGainsMap);
		Audio::DownmixBuffer(InChannels, OutChannels, InBuffer, OutNewBuffer, MixdownGainsMap.GetData());
	}

	void FMixerSubmix::SetParentSubmix(TWeakPtr<FMixerSubmix, ESPMode::ThreadSafe> SubmixWeakPtr)
	{
		if (ParentSubmix == SubmixWeakPtr)
		{
			return;
		}

		TSharedPtr<Audio::FMixerSubmix, ESPMode::ThreadSafe> ParentPtr = ParentSubmix.Pin();
		if (ParentPtr.IsValid())
		{
			const uint32 InChildId = GetId();
			ParentPtr->SubmixCommand([this, InChildId, SubmixWeakPtr]()
			{
				AUDIO_MIXER_CHECK_AUDIO_PLAT_THREAD(MixerDevice);

				ChildSubmixes.Remove(InChildId);
			});
		}

		SubmixCommand([this, SubmixWeakPtr]()
		{
			AUDIO_MIXER_CHECK_AUDIO_PLAT_THREAD(MixerDevice);

			ParentSubmix = SubmixWeakPtr;
			if (IsSoundfieldSubmix())
			{
				SetupSoundfieldStreamForParent();
			}
		});
	}

	void FMixerSubmix::AddChildSubmix(TWeakPtr<FMixerSubmix, ESPMode::ThreadSafe> SubmixWeakPtr)
	{
		SubmixCommand([this, SubmixWeakPtr]()
		{
			AUDIO_MIXER_CHECK_AUDIO_PLAT_THREAD(MixerDevice);

			TSharedPtr<Audio::FMixerSubmix, ESPMode::ThreadSafe> SubmixSharedPtr = SubmixWeakPtr.Pin();
			if (SubmixSharedPtr.IsValid())
			{
				FChildSubmixInfo& ChildSubmixInfo = ChildSubmixes.Emplace(SubmixSharedPtr->GetId(), SubmixWeakPtr);

				if (IsSoundfieldSubmix())
				{
					SetupSoundfieldEncodingForChild(ChildSubmixInfo);
				}
			}
		});
	}

	void FMixerSubmix::RemoveChildSubmix(TWeakPtr<FMixerSubmix, ESPMode::ThreadSafe> SubmixWeakPtr)
	{
		TSharedPtr<FMixerSubmix, ESPMode::ThreadSafe> SubmixStrongPtr = SubmixWeakPtr.Pin();
		if (!SubmixStrongPtr.IsValid())
		{
			return;
		}

		const uint32 OldIdToRemove = SubmixStrongPtr->GetId();
		SubmixCommand([this, OldIdToRemove]()
		{
			AUDIO_MIXER_CHECK_AUDIO_PLAT_THREAD(MixerDevice);

			ChildSubmixes.Remove(OldIdToRemove);
		});
	}

	void FMixerSubmix::RegisterAudioBus(const Audio::FAudioBusKey& InAudioBusKey, Audio::FPatchInput&& InPatchInput)
	{
		check(IsInAudioThread());

		SubmixCommand([this, InAudioBusKey, InPatchInput = MoveTemp(InPatchInput)]()
		{
			if (!AudioBuses.Contains(InAudioBusKey))
			{
				AudioBuses.Emplace(InAudioBusKey, InPatchInput);
			}

			// If we are sending to any audio buses, set up this buffer for potential downmixing
			AudioBusSendBuffer.SetNumUninitialized(NumSamples * static_cast<int32>(EAudioBusChannels::MaxChannelCount));
		});
	}

	void FMixerSubmix::UnregisterAudioBus(const Audio::FAudioBusKey& InAudioBusKey)
	{
		check(IsInAudioThread());

		SubmixCommand([this, InAudioBusKey]()
		{
			if (AudioBuses.Contains(InAudioBusKey))
			{
				AudioBuses.Remove(InAudioBusKey);
			}
		});
	}

	int32 FMixerSubmix::GetSubmixChannels() const
	{
		return NumChannels;
	}

	TWeakPtr<FMixerSubmix, ESPMode::ThreadSafe> FMixerSubmix::GetParentSubmix()
	{
		return ParentSubmix;
	}

	int32 FMixerSubmix::GetNumSourceVoices() const
	{
		return MixerSourceIds.Num();
	}

	int32 FMixerSubmix::GetNumEffects() const
	{
		return NumSubmixEffects;
	}

	int32 FMixerSubmix::GetSizeOfSubmixChain() const
	{
		// Return the base size
		for (const FSubmixEffectFadeInfo& Info : EffectChains)
		{
			if (Info.bIsCurrentChain)
			{
				return Info.EffectChain.Num();
			}
		}
		return 0;
	}

	void FMixerSubmix::AddOrSetSourceVoice(FMixerSourceVoice* InSourceVoice, const float InSendLevel, EMixerSourceSubmixSendStage InSubmixSendStage)
	{
		AddOrSetSourceById(InSourceVoice->GetSourceId(), InSendLevel, InSubmixSendStage);
	}

	void FMixerSubmix::AddOrSetSourceById(const int32 InSourceId, const float InSendLevel, EMixerSourceSubmixSendStage InSubmixSendStage)
	{
		AUDIO_MIXER_CHECK_AUDIO_PLAT_THREAD(MixerDevice);

		FSubmixVoiceData NewVoiceData;
		NewVoiceData.SendLevel = InSendLevel;
		NewVoiceData.SubmixSendStage = InSubmixSendStage;

		MixerSourceIds.Add(InSourceId, NewVoiceData);
	}

	FPatchOutputStrongPtr FMixerSubmix::AddPatch(float InGain)
	{
		if (IsSoundfieldSubmix())
		{
			UE_LOGF(LogAudioMixer, Warning, "Patch listening to SoundfieldSubmixes is not supported.");
			return nullptr;
		}

		return PatchSplitter.AddNewPatch(NumSamples, InGain);
	}

	void FMixerSubmix::RemoveSourceVoice(FMixerSourceVoice* InSourceVoice)
	{
		RemoveSourceById(InSourceVoice->GetSourceId());
	}

	void FMixerSubmix::RemoveSourceById(const int32 InSourceId)
	{
		AUDIO_MIXER_CHECK_AUDIO_PLAT_THREAD(MixerDevice);
		
		// If the source has a corresponding ambisonics encoder, close it out.
		uint32 SourceEncoderID = INDEX_NONE;
		const FSubmixVoiceData* MixerSourceVoiceData = MixerSourceIds.Find(InSourceId);

		// If we did find a valid corresponding FSubmixVoiceData, remove it from the map.
		if (MixerSourceVoiceData)
		{
			int32 NumRemoved = MixerSourceIds.Remove(InSourceId);
			AUDIO_MIXER_CHECK(NumRemoved == 1);
		}
	}

	void FMixerSubmix::AddSoundEffectSubmix(FSoundEffectSubmixPtr InSoundEffectSubmix)
	{
		FScopeLock ScopeLock(&EffectChainMutationCriticalSection);
		AUDIO_MIXER_CHECK_AUDIO_PLAT_THREAD(MixerDevice);

		uint32 SubmixPresetId = InSoundEffectSubmix->GetParentPresetId();

		// Look to see if the submix preset ID is already present
		for (FSubmixEffectFadeInfo& FadeInfo : EffectChains)
		{
			for (FSoundEffectSubmixPtr& Effect : FadeInfo.EffectChain)
			{
				if (Effect.IsValid() && Effect->GetParentPresetId() == SubmixPresetId)
				{
					// Already added.
					return;
				}
			}
		}

		++NumSubmixEffects;
		if (EffectChains.Num() > 0)
		{
			for (FSubmixEffectFadeInfo& FadeInfo : EffectChains)
			{
				if (FadeInfo.bIsCurrentChain)
				{
					FadeInfo.EffectChain.Add(InSoundEffectSubmix);
					return;
				}
			}
		}
		else
		{
			FSubmixEffectFadeInfo& NewSubmixEffectChain = EffectChains.Add_GetRef(FSubmixEffectFadeInfo());
			NewSubmixEffectChain.bIsCurrentChain = true;
			NewSubmixEffectChain.FadeVolume = FDynamicParameter(1.0f);
			NewSubmixEffectChain.EffectChain.Add(InSoundEffectSubmix);
		}
	}

	void FMixerSubmix::RemoveSoundEffectSubmix(uint32 SubmixPresetId)
	{
		FScopeLock ScopeLock(&EffectChainMutationCriticalSection);
		AUDIO_MIXER_CHECK_AUDIO_PLAT_THREAD(MixerDevice);

		for (FSubmixEffectFadeInfo& FadeInfo : EffectChains)
		{
			for (FSoundEffectSubmixPtr& EffectInstance : FadeInfo.EffectChain)
			{
				if (EffectInstance.IsValid())
				{
					if (EffectInstance->GetParentPresetId() == SubmixPresetId)
					{
						EffectInstance.Reset();
						--NumSubmixEffects;
						return;
					}
				}
			}
		}
	}

	void FMixerSubmix::RemoveSoundEffectSubmixAtIndex(int32 InIndex)
	{
		AUDIO_MIXER_CHECK_AUDIO_PLAT_THREAD(MixerDevice);

		for (FSubmixEffectFadeInfo& FadeInfo : EffectChains)
		{
			if (FadeInfo.bIsCurrentChain)
			{
				if (InIndex >= 0 && InIndex < FadeInfo.EffectChain.Num())
				{
					FSoundEffectSubmixPtr& EffectInstance = FadeInfo.EffectChain[InIndex];
					if (EffectInstance.IsValid())
					{
						EffectInstance.Reset();
						--NumSubmixEffects;
					}
				}
				return;
			}
		}
	}

	void FMixerSubmix::ClearSoundEffectSubmixes()
	{
		FScopeLock ScopeLock(&EffectChainMutationCriticalSection);

		TArray<TSoundEffectSubmixPtr> SubmixEffectsToReset;

		for (FSubmixEffectFadeInfo& FadeInfo : EffectChains)
		{
			for (FSoundEffectSubmixPtr& EffectInstance : FadeInfo.EffectChain)
			{
				if (EffectInstance.IsValid())
				{
					SubmixEffectsToReset.Add(EffectInstance);
				}
			}

			FadeInfo.EffectChain.Reset();
		}

		// Unregister these source effect instances from their owning USoundEffectInstance on the audio thread.
		// Have to pass to Game Thread prior to processing on AudioThread to avoid race condition with GC
		// (RunCommandOnAudioThread is not safe to call from any thread other than the GameThread).
		if (!SubmixEffectsToReset.IsEmpty())
		{
			AsyncTask(ENamedThreads::GameThread, [GTSubmixEffects = MoveTemp(SubmixEffectsToReset)]() mutable
			{
				FAudioThread::RunCommandOnAudioThread([ATSubmixEffects = MoveTemp(GTSubmixEffects)]() mutable
				{
					for (const TSoundEffectSubmixPtr& SubmixPtr : ATSubmixEffects)
					{
						USoundEffectPreset::UnregisterInstance(SubmixPtr);
					}
				});
			});
		}

		NumSubmixEffects = 0;
		EffectChains.Reset();
	}

	void FMixerSubmix::SetSubmixEffectChainOverride(const TArray<FSoundEffectSubmixPtr>& InSubmixEffectPresetChain, float InFadeTimeSec)
	{
		FScopeLock ScopeLock(&EffectChainMutationCriticalSection);

		// Set every existing override to NOT be the current override
		for (FSubmixEffectFadeInfo& FadeInfo : EffectChains)
		{
			FadeInfo.bIsCurrentChain = false;
			FadeInfo.FadeVolume.Set(0.0f, InFadeTimeSec);
		}

		FSubmixEffectFadeInfo& NewSubmixEffectChain = EffectChains.Add_GetRef(FSubmixEffectFadeInfo());
		NewSubmixEffectChain.bIsCurrentChain = true;
		NewSubmixEffectChain.FadeVolume = FDynamicParameter(0.0f);
		NewSubmixEffectChain.FadeVolume.Set(1.0f, InFadeTimeSec);
		NewSubmixEffectChain.EffectChain = InSubmixEffectPresetChain; 
	}

	void FMixerSubmix::ClearSubmixEffectChainOverride(float InFadeTimeSec)
	{
		FScopeLock ScopeLock(&EffectChainMutationCriticalSection);

		// Set all non-base submix chains to fading out, set the base submix chain to fading in
		for (FSubmixEffectFadeInfo& FadeInfo : EffectChains)
		{
			if (FadeInfo.bIsBaseEffect)
			{
				FadeInfo.bIsCurrentChain = true;
				FadeInfo.FadeVolume.Set(1.0f, InFadeTimeSec);
			}
			else
			{
				FadeInfo.bIsCurrentChain = false;
				FadeInfo.FadeVolume.Set(0.0f, InFadeTimeSec);
			}
		}
	}

	void FMixerSubmix::ReplaceSoundEffectSubmix(int32 InIndex, FSoundEffectSubmixPtr InEffectInstance)
	{
		FScopeLock ScopeLock(&EffectChainMutationCriticalSection);

		for (FSubmixEffectFadeInfo& FadeInfo : EffectChains)
		{
			if (FadeInfo.bIsCurrentChain)
			{
				if (FadeInfo.EffectChain.IsValidIndex(InIndex))
				{
					FadeInfo.EffectChain[InIndex] = InEffectInstance;
				}
				break;
			}
		}
	}

	void FMixerSubmix::SetBackgroundMuted(bool bInMuted)
	{
		SubmixCommand([this, bInMuted]()
		{
			bIsBackgroundMuted = bInMuted;
		});
	}

	void FMixerSubmix::MixBufferDownToMono(const FAlignedFloatBuffer& InBuffer, int32 NumInputChannels, FAlignedFloatBuffer& OutBuffer)
	{
		check(NumInputChannels > 0);

		int32 NumFrames = InBuffer.Num() / NumInputChannels;
		OutBuffer.Reset();
		OutBuffer.AddZeroed(NumFrames);

		const float* InData = InBuffer.GetData();
		float* OutData = OutBuffer.GetData();

		const float GainFactor = 1.0f / FMath::Sqrt((float) NumInputChannels);

		for (int32 FrameIndex = 0; FrameIndex < NumFrames; FrameIndex++)
		{
			for (int32 ChannelIndex = 0; ChannelIndex < NumInputChannels; ChannelIndex++)
			{
				const int32 InputIndex = FrameIndex * NumInputChannels + ChannelIndex;
				OutData[FrameIndex] += InData[InputIndex] * GainFactor;
			}
		}
	}

	void FMixerSubmix::SetupSoundfieldEncodersForChildren()
	{
		check(SoundfieldStreams.Factory);
		check(SoundfieldStreams.Settings.IsValid());

		//Here we scan all child submixes to see which submixes need to be reencoded.
		for (auto& Iter : ChildSubmixes)
		{
			FChildSubmixInfo& ChildSubmix = Iter.Value;
			SetupSoundfieldEncodingForChild(ChildSubmix);
		}
	}

	void FMixerSubmix::SetupSoundfieldEncodingForChild(FChildSubmixInfo& InChild)
	{
		TSharedPtr<Audio::FMixerSubmix, ESPMode::ThreadSafe> SubmixPtr = InChild.SubmixPtr.Pin();

		if (SubmixPtr.IsValid())
		{
			check(SoundfieldStreams.Factory && SoundfieldStreams.Settings.IsValid());

			// If this child submix is not a soundfield submix and we need to encode every child submix independently, set up an encoder.
			if (!SubmixPtr->IsSoundfieldSubmix() && SoundfieldStreams.Factory->ShouldEncodeAllStreamsIndependently(*SoundfieldStreams.Settings))
			{
				FAudioPluginInitializationParams InitParams = GetInitializationParamsForSoundfieldStream();
				InChild.Encoder = SoundfieldStreams.Factory->CreateEncoderStream(InitParams, *SoundfieldStreams.Settings);
			}
			else if(SubmixPtr->IsSoundfieldSubmix())
			{
				// If the child submix is of a soundfield format that needs to be transcoded, set up a transcoder.
				InChild.Transcoder = GetTranscoderForChildSubmix(SubmixPtr);
			}

			// see if we need to initialize our child submix encoder
			const bool bNeedsDownmixedChildEncoder = !SoundfieldStreams.DownmixedChildrenEncoder.IsValid();
			const bool bUsingUniqueEncoderForEachChildSubmix = !SoundfieldStreams.Factory->ShouldEncodeAllStreamsIndependently(*SoundfieldStreams.Settings);

			if (bNeedsDownmixedChildEncoder && bUsingUniqueEncoderForEachChildSubmix)
			{
				FAudioPluginInitializationParams InitParams = GetInitializationParamsForSoundfieldStream();
				SoundfieldStreams.DownmixedChildrenEncoder = SoundfieldStreams.Factory->CreateEncoderStream(InitParams, *SoundfieldStreams.Settings);
			}

			// If neither of these are true, either we are downmixing all child audio and encoding it once, or
			// this submix can handle the child's soundfield audio packet directly, so no encoder nor transcoder is needed.
		}
	}

	void FMixerSubmix::SetupSoundfieldStreamForParent()
	{
		TSharedPtr<FMixerSubmix, ESPMode::ThreadSafe> ParentSubmixSharedPtr = ParentSubmix.Pin();

		if (ParentSubmixSharedPtr.IsValid() && !ParentSubmixSharedPtr->IsSoundfieldSubmix())
		{
			// If the submix we're plugged into isn't a soundfield submix, we need to decode our soundfield for it.
			SetUpSoundfieldPositionalData(ParentSubmixSharedPtr);

			FAudioPluginInitializationParams InitParams = GetInitializationParamsForSoundfieldStream();
			SoundfieldStreams.ParentDecoder = SoundfieldStreams.Factory->CreateDecoderStream(InitParams, *SoundfieldStreams.Settings);
		}
	}

	void FMixerSubmix::SetUpSoundfieldPositionalData(const TSharedPtr<Audio::FMixerSubmix, ESPMode::ThreadSafe>& InParentSubmix)
	{
		// If there is a parent and we are not passing it this submix's ambisonics audio, retrieve that submix's channel format.
		check(InParentSubmix.IsValid());

		const int32 NumParentChannels = InParentSubmix->GetSubmixChannels();
		SoundfieldStreams.CachedPositionalData.NumChannels = NumParentChannels;
		SoundfieldStreams.CachedPositionalData.ChannelPositions = MixerDevice->GetDefaultPositionMap(NumParentChannels);
		
		// For now we don't actually do any sort of rotation for decoded audio.
		SoundfieldStreams.CachedPositionalData.Rotation = FQuat::Identity;
	}

	void FMixerSubmix::MixInSource(const ISoundfieldAudioPacket& InAudio, const ISoundfieldEncodingSettingsProxy& InSettings, ISoundfieldAudioPacket& PacketToSumTo)
	{
		check(SoundfieldStreams.Mixer.IsValid());

		FSoundfieldMixerInputData InputData =
		{
			InAudio, // InputPacket
			InSettings, // EncodingSettings
			1.0f // SendLevel
		};

		SoundfieldStreams.Mixer->MixTogether(InputData, PacketToSumTo);
	}

	void FMixerSubmix::UpdateListenerRotation(const FQuat& InRotation)
	{
		SoundfieldStreams.CachedPositionalData.Rotation = InRotation;
	}

	void FMixerSubmix::MixInChildSubmix(FChildSubmixInfo& Child, ISoundfieldAudioPacket& PacketToSumTo)
	{
		check(IsSoundfieldSubmix());

		// We only either encode, transcode input, and never both. If we have both for this child, something went wrong in initialization.
		check(!(Child.Encoder.IsValid() && Child.Transcoder.IsValid()));

		TSharedPtr<FMixerSubmix, ESPMode::ThreadSafe> ChildSubmixSharedPtr = Child.SubmixPtr.Pin();
		if (ChildSubmixSharedPtr.IsValid())
		{
			if (!ChildSubmixSharedPtr->IsSoundfieldSubmix())
			{
				// Reset the output scratch buffer so that we can call ProcessAudio on the ChildSubmix with it:
				ScratchBuffer.Reset(ChildSubmixSharedPtr->NumSamples);
				ScratchBuffer.AddZeroed(ChildSubmixSharedPtr->NumSamples);
				

				// If this is true, the Soundfield Factory explicitly requested that a separate encoder stream was set up for every
				// non-soundfield child submix.
				if (Child.Encoder.IsValid())
				{
					ChildSubmixSharedPtr->ProcessAudio(ScratchBuffer);

					// Encode the resulting audio and mix it in.
					FSoundfieldEncoderInputData InputData = {
						ScratchBuffer, /* AudioBuffer */
						ChildSubmixSharedPtr->NumChannels, /* NumChannels */
						*SoundfieldStreams.Settings, /** InputSettings */
						SoundfieldStreams.CachedPositionalData /** PosititonalData */
					};

					Child.Encoder->EncodeAndMixIn(InputData, PacketToSumTo);
				}
				else
				{
					// Otherwise, process and mix in the submix's audio to the scratch buffer, and we will encode ScratchBuffer later.
					ChildSubmixSharedPtr->ProcessAudio(ScratchBuffer);
				}
			}
			else if (Child.Transcoder.IsValid())
			{
				// Make sure our packet that we call process on is zeroed out:
				if (!Child.IncomingPacketToTranscode.IsValid())
				{
					Child.IncomingPacketToTranscode = ChildSubmixSharedPtr->SoundfieldStreams.Factory->CreateEmptyPacket();
				}
				else
				{
					Child.IncomingPacketToTranscode->Reset();
				}

				check(Child.IncomingPacketToTranscode.IsValid());

				ChildSubmixSharedPtr->ProcessAudio(*Child.IncomingPacketToTranscode);

				Child.Transcoder->TranscodeAndMixIn(*Child.IncomingPacketToTranscode, ChildSubmixSharedPtr->GetSoundfieldSettings(), PacketToSumTo, *SoundfieldStreams.Settings);
			}
			else
			{
				// No conversion necessary.
				ChildSubmixSharedPtr->ProcessAudio(PacketToSumTo);
			}

			//Propogate listener rotation down to this submix.
			// This is required if this submix doesn't have any sources sending to it, but does have at least one child submix.
			UpdateListenerRotation(ChildSubmixSharedPtr->SoundfieldStreams.CachedPositionalData.Rotation);
		}
	}

	bool FMixerSubmix::IsSoundfieldSubmix() const
	{
		return SoundfieldStreams.Factory != nullptr;
	}

	bool FMixerSubmix::IsDefaultEndpointSubmix() const
	{
		return !ParentSubmix.IsValid() && !(EndpointData.SoundfieldEndpoint.IsValid() || EndpointData.NonSoundfieldEndpoint.IsValid());
	}

	bool FMixerSubmix::IsExternalEndpointSubmix() const
	{
		return !ParentSubmix.IsValid() && (EndpointData.SoundfieldEndpoint.IsValid() || EndpointData.NonSoundfieldEndpoint.IsValid());
	}

	bool FMixerSubmix::IsSoundfieldEndpointSubmix() const
	{
		return !ParentSubmix.IsValid() && IsSoundfieldSubmix();
	}

	bool FMixerSubmix::IsDummyEndpointSubmix() const
	{
		if (EndpointData.NonSoundfieldEndpoint.IsValid())
		{
			return !EndpointData.NonSoundfieldEndpoint->IsImplemented();
		}
		else
		{
			return false;
		}
	}

	FName FMixerSubmix::GetSoundfieldFormat() const
	{
		if (IsSoundfieldSubmix())
		{
			return SoundfieldStreams.Factory->GetSoundfieldFormatName();
		}
		else
		{
			return ISoundfieldFactory::GetFormatNameForNoEncoding();
		}
	}

	ISoundfieldEncodingSettingsProxy& FMixerSubmix::GetSoundfieldSettings()
	{
		check(IsSoundfieldSubmix());
		check(SoundfieldStreams.Settings.IsValid());

		return *SoundfieldStreams.Settings;
	}

	FAudioPluginInitializationParams FMixerSubmix::GetInitializationParamsForSoundfieldStream()
	{
		FAudioPluginInitializationParams InitializationParams;
		InitializationParams.AudioDevicePtr = MixerDevice;
		InitializationParams.BufferLength = MixerDevice ? MixerDevice->GetNumOutputFrames() : 0;
		InitializationParams.NumOutputChannels = MixerDevice ? MixerDevice->GetNumDeviceChannels() : 0;
		InitializationParams.SampleRate = MixerDevice ? MixerDevice->SampleRate : 0.0f;

		// We only use one soundfield stream per source.
		InitializationParams.NumSources = 1;

		return InitializationParams;
	}

	FSoundfieldSpeakerPositionalData FMixerSubmix::GetDefaultPositionalDataForAudioDevice()
	{
		FSoundfieldSpeakerPositionalData PositionalData;
		PositionalData.NumChannels = MixerDevice->GetNumDeviceChannels();
		PositionalData.ChannelPositions = MixerDevice->GetDefaultPositionMap(PositionalData.NumChannels);

		// For now we don't actually do any sort of rotation for decoded audio.
		PositionalData.Rotation = FQuat::Identity;

		return PositionalData;
	}

	TUniquePtr<ISoundfieldTranscodeStream> FMixerSubmix::GetTranscoderForChildSubmix(const TSharedPtr<Audio::FMixerSubmix, ESPMode::ThreadSafe>& InChildSubmix)
	{
		check(InChildSubmix.IsValid());
		check(IsSoundfieldSubmix() && InChildSubmix->IsSoundfieldSubmix());
		check(SoundfieldStreams.Settings.IsValid() && InChildSubmix->SoundfieldStreams.Settings.IsValid());

		if (GetSoundfieldFormat() != InChildSubmix->GetSoundfieldFormat())
		{
			ISoundfieldFactory* ChildFactory = InChildSubmix->GetSoundfieldFactory();

			if (SoundfieldStreams.Factory->CanTranscodeFromSoundfieldFormat(InChildSubmix->GetSoundfieldFormat(), *InChildSubmix->SoundfieldStreams.Settings))
			{
				FAudioPluginInitializationParams InitParams = GetInitializationParamsForSoundfieldStream();
				return SoundfieldStreams.Factory->CreateTranscoderStream(InChildSubmix->GetSoundfieldFormat(), InChildSubmix->GetSoundfieldSettings(), SoundfieldStreams.Factory->GetSoundfieldFormatName(), GetSoundfieldSettings(), InitParams);
			}
			else if (ChildFactory->CanTranscodeToSoundfieldFormat(GetSoundfieldFormat(), GetSoundfieldSettings()))
			{
				FAudioPluginInitializationParams InitParams = GetInitializationParamsForSoundfieldStream();
				return ChildFactory->CreateTranscoderStream(InChildSubmix->GetSoundfieldFormat(), InChildSubmix->GetSoundfieldSettings(), SoundfieldStreams.Factory->GetSoundfieldFormatName(), GetSoundfieldSettings(), InitParams);
			}
			else
			{
				return nullptr;
			}
		}
		else
		{
			if (SoundfieldStreams.Factory->IsTranscodeRequiredBetweenSettings(*InChildSubmix->SoundfieldStreams.Settings, *SoundfieldStreams.Settings))
			{
				FAudioPluginInitializationParams InitParams = GetInitializationParamsForSoundfieldStream();
				return SoundfieldStreams.Factory->CreateTranscoderStream(InChildSubmix->GetSoundfieldFormat(), InChildSubmix->GetSoundfieldSettings(), SoundfieldStreams.Factory->GetSoundfieldFormatName(), GetSoundfieldSettings(), InitParams);
			}
			else
			{
				return nullptr;
			}
		}
	}

	void FMixerSubmix::PumpCommandQueue()
	{
		TFunction<void()> Command;
		while (CommandQueue.Dequeue(Command))
		{
			Command();
		}
	}

	void FMixerSubmix::SubmixCommand(TFunction<void()> Command)
	{
		CommandQueue.Enqueue(MoveTemp(Command));
	}

	bool FMixerSubmix::IsOwnerValid() const
	{
		return OwningSubmixObject.IsValid();
	}

	bool FMixerSubmix::IsValid() const
	{
		return OwningSubmixObject.IsValid();
	}

	bool FMixerSubmix::IsRenderingAudio() const
	{
		// If we're told to not auto-disable we act as if we're always rendering audio
		if (!bAutoDisable)
		{
			return true;
		}
		
		{
			// query the SubmixBufferListeners to see if they plan to render audio into this buffer
			FScopeLock Lock(&BufferListenerCriticalSection);
			for (const TWeakPtr<ISubmixBufferListener>& BufferListenerWeakPtr : BufferListenerPtrs)	
			{
				if (TSharedPtr<ISubmixBufferListener> BufferListener = BufferListenerWeakPtr.Pin())
				{
					if (BufferListener->IsRenderingAudio())
					{
						return true;
					}
				}
			}
		}

		// If this submix is not rendering any sources directly and silence has been detected, we need to check it's children submixes
		if (MixerSourceIds.Num() == 0 && SilenceTimeStartSeconds >= 0.0)
		{
			// Use the audio clock to check against the disablement timeout
			double AudioClock = MixerDevice->GetAudioClock();
			double TimeSinceSilent = AudioClock - SilenceTimeStartSeconds;

			// If we're past the threshold for disablement, do one last check of any child submixes. Maybe they are now rendering audio.
			if (TimeSinceSilent > AutoDisableTime)
			{
				for (auto& ChildSubmixEntry : ChildSubmixes)
				{
					TSharedPtr<Audio::FMixerSubmix, ESPMode::ThreadSafe> ChildSubmix = ChildSubmixEntry.Value.SubmixPtr.Pin();

					// If any of the submix's children are rendering audio then this submix is also rendering audio
					if (ChildSubmix && ChildSubmix->IsRenderingAudio())
					{
						return true;
					}
				}

				// We're past the auto-disablement threshold and no child submixes are rendering audio, we're now silent
				return false;
			}
		}
		return true;
	}

	void FMixerSubmix::SetAutoDisable(bool bInAutoDisable)
	{
		bAutoDisable = bInAutoDisable;
	}

	void FMixerSubmix::SetAutoDisableTime(float InAutoDisableTime)
	{
		AutoDisableTime = InAutoDisableTime;
	}

	void FMixerSubmix::UpdateDebugVolume()
	{
		check(IsInAudioThread());

#if ENABLE_AUDIO_DEBUG
		FAudioDeviceManager* const DeviceManager = FAudioDeviceManager::Get();
		if (DeviceManager)
		{
			const Audio::FAudioDebugger& Debugger = DeviceManager->GetDebugger();

			if (!Debugger.IsSubmixMuteOrSoloActive())
			{
				DebugVolume.store(1.0f, std::memory_order_relaxed);
				return;
			}

			bool bIsSoloed = false;
			bool bIsMuted = false;
			FString Reason;

			Debugger.QuerySoloMuteSubmix(SubmixName, bIsSoloed, bIsMuted, Reason);

			DebugVolume.store(bIsMuted ? 0.0f : 1.0f, std::memory_order_relaxed);
			return;
		}
#endif // ENABLE_AUDIO_DEBUG

		DebugVolume.store(1.0f, std::memory_order_relaxed);
	}

	void FMixerSubmix::ProcessAudio(FAlignedFloatBuffer& OutAudioBuffer)
	{
		AUDIO_MIXER_CHECK_AUDIO_PLAT_THREAD(MixerDevice);

		// Handle channel count change.
		if (NumChannels != MixerDevice->GetNumDeviceChannels())
		{
			// Device format may change channels if device is hot swapped
			NumChannels = MixerDevice->GetNumDeviceChannels();

			if (IsSoundfieldSubmix())
			{
				// Update decoder to match parent stream format.
				SetupSoundfieldStreamForParent();
			}
		}

		// If we hit this, it means that platform info gave us an invalid NumChannel count.
		if (!ensure(NumChannels != 0 && NumChannels <= AUDIO_MIXER_MAX_OUTPUT_CHANNELS))
		{
			return;
		}


		// If this is a Soundfield Submix, process our soundfield and decode it to a OutAudioBuffer.
		if (IsSoundfieldSubmix())
		{
			FScopeLock ScopeLock(&SoundfieldStreams.StreamsLock);

			// Initialize or clear the mixed down audio packet.
			if (!SoundfieldStreams.MixedDownAudio.IsValid())
			{
				SoundfieldStreams.MixedDownAudio = SoundfieldStreams.Factory->CreateEmptyPacket();
			}
			else
			{
				SoundfieldStreams.MixedDownAudio->Reset();
			}

			check(SoundfieldStreams.MixedDownAudio.IsValid());

			ProcessAudio(*SoundfieldStreams.MixedDownAudio);

			if (!SoundfieldStreams.ParentDecoder.IsValid())
			{
				return;
			}

			//Decode soundfield to interleaved float audio.
			FSoundfieldDecoderInputData DecoderInput =
			{
				*SoundfieldStreams.MixedDownAudio, /* SoundfieldBuffer */
				SoundfieldStreams.CachedPositionalData, /* PositionalData */
				MixerDevice ? MixerDevice->GetNumOutputFrames() : 0, /* NumFrames */
				MixerDevice ? MixerDevice->GetSampleRate() : 0.0f /* SampleRate */
			};

			FSoundfieldDecoderOutputData DecoderOutput = { OutAudioBuffer };

			SoundfieldStreams.ParentDecoder->DecodeAndMixIn(DecoderInput, DecoderOutput);
			return;
		}
		else
		{
			// Pump pending command queues. For Soundfield Submixes this occurs in ProcessAudio(ISoundfieldAudioPacket&).
			PumpCommandQueue();
		}
	
		const int32 NumOutputFrames = OutAudioBuffer.Num() / NumChannels;
		NumSamples = NumChannels * NumOutputFrames;

 		InputBuffer.Reset(NumSamples);
 		InputBuffer.AddZeroed(NumSamples);

		float* BufferPtr = InputBuffer.GetData();

		// As an optimization, early out if we're set to auto-disable and we're not rendering audio
		if (bAutoDisable && !IsRenderingAudio() && !bNeverDisableSubmixes)
		{
			if (!bIsCurrentlyDisabled)
			{
				bIsCurrentlyDisabled = true;

				if (MixerSubmixCVars::bRemoveModulatorsOnDisabledSubmixes)
				{
					VolumeMod.ClearModulators();
					WetLevelMod.ClearModulators();
					DryLevelMod.ClearModulators();
				}
				
#if UE_AUDIO_PROFILERTRACE_ENABLED
				const bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AudioMixerChannel);
				if (bChannelEnabled && OwningSubmixObject.IsValid())
				{
					TArray<float> SilentEnvelopeValues;
					SilentEnvelopeValues.Init(0.0f, NumChannels);

					UE_TRACE_LOG(Audio, SubmixEnvelopeValues, AudioMixerChannel)
						<< SubmixEnvelopeValues.DeviceId(MixerDevice->DeviceID)
						<< SubmixEnvelopeValues.SubmixId(GetTypeHash(OwningSubmixObject->GetPathName()))
						<< SubmixEnvelopeValues.Timestamp(FPlatformTime::Cycles64())
						<< SubmixEnvelopeValues.EnvelopeValues(SilentEnvelopeValues.GetData(), NumChannels);
				}
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

				UE_CLOGF(LogSubmixEnablementCVar == 1, LogAudioMixer, Verbose,
					"Submix Disabled. Num Sources: %d, Time Silent: %.2f, Disablement Threshold: %.2f, Submix Name: %ls", 
					MixerSourceIds.Num(),
					(float)(MixerDevice->GetAudioClock() - SilenceTimeStartSeconds),
					AutoDisableTime,
					*SubmixName);
			}

			// Even though we're silent, broadcast the buffer to any listeners (will be a silent buffer)
			SendAudioToSubmixBufferListeners(InputBuffer);
			SendAudioToRegisteredAudioBuses(InputBuffer);
			return;
		}

		if (bIsCurrentlyDisabled)
		{
			if (MixerSubmixCVars::bRemoveModulatorsOnDisabledSubmixes)
			{
				if (const USoundSubmix* SoundSubmix = Cast<const USoundSubmix>(OwningSubmixObject))
				{
					const TSet<TObjectPtr<USoundModulatorBase>>& VolumeModulator = SoundSubmix->OutputVolumeModulation.Modulators;
					const TSet<TObjectPtr<USoundModulatorBase>>& WetLevelModulator = SoundSubmix->WetLevelModulation.Modulators;
					const TSet<TObjectPtr<USoundModulatorBase>>& DryLevelModulator = SoundSubmix->DryLevelModulation.Modulators;
					UpdateModulationSettings(VolumeModulator, WetLevelModulator, DryLevelModulator);
				}
			}

			bIsCurrentlyDisabled = false;
			UE_CLOGF(LogSubmixEnablementCVar == 1, LogAudioMixer, Verbose, "Submix Re-Enabled: %ls", *SubmixName);
		}

		// Mix all submix audio into this submix's input scratch buffer
		{
			CSV_SCOPED_TIMING_STAT(Audio, SubmixChildren);
			CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_AudioMixerSubmixChildren, (ChildSubmixes.Num() > 0));

			// First loop this submix's child submixes mixing in their output into this submix's dry/wet buffers.
			TArray<uint32> ToRemove;
			for (auto& ChildSubmixEntry : ChildSubmixes)
			{
				TSharedPtr<Audio::FMixerSubmix, ESPMode::ThreadSafe> ChildSubmix = ChildSubmixEntry.Value.SubmixPtr.Pin();

				// Owning submix can become invalid prior to BeginDestroy being called if object is
				// forcibly deleted in editor, so submix validity (in addition to pointer validity) is checked before processing
				if (ChildSubmix.IsValid() && ChildSubmix->IsOwnerValid())
				{
					ChildSubmix->ProcessAudio(InputBuffer);

					// Check the buffer after processing to catch any bad values.
					AUDIO_CHECK_BUFFER_NAMED_MSG(InputBuffer, TEXT("Submix Chidren"), TEXT("Submix: %s"), *ChildSubmix->GetName());
				}
				else
				{
					ToRemove.Add(ChildSubmixEntry.Key);
				}
			}

			for (uint32 Key : ToRemove)
			{
				ChildSubmixes.Remove(Key);
			}
		}

		{
			CSV_SCOPED_TIMING_STAT(Audio, SubmixSource);
			CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_AudioMixerSubmixSource, (MixerSourceIds.Num() > 0));

			// Pre-size the scratch buffer to match InputBuffer so it never needs to grow mid-loop.
			// FilterScratchBuffer is a member, so its backing allocation persists across MixBuffer()
			// calls; SetNumUninitialized only reallocates if the current capacity is insufficient.
			FilterScratchBuffer.SetNumUninitialized(InputBuffer.Num());

			// Loop through this submix's sound sources
			for (const auto& MixerSourceVoiceIter : MixerSourceIds)
			{
				const int32 SourceId = MixerSourceVoiceIter.Key;
				const float SendLevel = MixerSourceVoiceIter.Value.SendLevel;
				const EMixerSourceSubmixSendStage SubmixSendStage = MixerSourceVoiceIter.Value.SubmixSendStage;

				MixerDevice->GetSourceManager()->MixOutputBuffers(SourceId, NumChannels, SendLevel, SubmixSendStage, GetId(), FilterScratchBuffer, InputBuffer);

				// Check the buffer after each voice mix to catch any bad values.
				AUDIO_CHECK_BUFFER_NAMED_MSG(InputBuffer, TEXT("Submix SourceMix"), TEXT("Submix: %s"), *GetName());
			}
		}

		DryChannelBuffer.Reset();

		// Update Dry Level using modulator
		if (MixerDevice->IsModulationPluginEnabled() && MixerDevice->ModulationInterface.IsValid())
		{
			DryLevelMod.ProcessControl(DryModBaseDb);
			TargetDryLevel = DryLevelMod.GetValue();
		}
		else
		{
			TargetDryLevel = Audio::ConvertToLinear(DryModBaseDb);
		}

		TargetDryLevel *= DryLevelModifier;

		// Check if we need to allocate a dry buffer. This is stored here before effects processing. We mix in with wet buffer after effects processing.
		if (!FMath::IsNearlyEqual(TargetDryLevel, CurrentDryLevel) || !FMath::IsNearlyZero(CurrentDryLevel))
		{
			DryChannelBuffer.Append(InputBuffer);
		}

		{
			FScopeLock ScopeLock(&EffectChainMutationCriticalSection);

			if (!BypassAllSubmixEffectsCVar && EffectChains.Num() > 0)
			{		
				CSV_SCOPED_TIMING_STAT(Audio, SubmixEffectProcessing);
				SCOPE_CYCLE_COUNTER(STAT_AudioMixerSubmixEffectProcessing);

				float SampleRate = MixerDevice->GetSampleRate();
				check(SampleRate > 0.0f);
				float DeltaTimeSec = NumOutputFrames / SampleRate;

				// Setup the input data buffer
				FSoundEffectSubmixInputData InputData;
				InputData.AudioClock = MixerDevice->GetAudioTime();

				// Compute the number of frames of audio. This will be independent of if we downmix our wet buffer.
				InputData.NumFrames = NumSamples / NumChannels;
				InputData.NumChannels = NumChannels;
				InputData.NumDeviceChannels = MixerDevice->GetNumDeviceChannels();
				InputData.ListenerTransforms = MixerDevice->GetListenerTransforms();
				InputData.AudioClock = MixerDevice->GetAudioClock();

				bool bProcessedAnEffect = false;
				float TotalStartFadeVolume = 0.0f;
				float TotalEndFadeVolume = 0.0f;

				// Zero and resize buffer holding mix of current effect chains
				SubmixChainMixBuffer.Reset(NumSamples);
				SubmixChainMixBuffer.SetNumZeroed(NumSamples);

				for (int32 EffectChainIndex = EffectChains.Num() - 1; EffectChainIndex >= 0; --EffectChainIndex)
				{
					FSubmixEffectFadeInfo& FadeInfo = EffectChains[EffectChainIndex];

					if (!FadeInfo.EffectChain.Num())
					{
						continue;
					}

					// If we're not the current chain and we've finished fading out, lets remove it from the effect chains
					if (!FadeInfo.bIsCurrentChain && FadeInfo.FadeVolume.IsDone())
					{
						// only remove effect chain if it's not the base effect chain
						if (!FadeInfo.bIsBaseEffect)
						{
							EffectChains.RemoveAtSwap(EffectChainIndex, EAllowShrinking::Yes);
						}
						continue;
					}

					// Prepare the scratch buffer for effect chain processing
					EffectChainOutputBuffer.Reset();
					EffectChainOutputBuffer.AddZeroed(NumSamples);

					const bool bResult = GenerateEffectChainAudio(InputData, InputBuffer, FadeInfo.EffectChain, EffectChainOutputBuffer);
					bProcessedAnEffect |= bResult;
					const FAlignedFloatBuffer* OutBuffer = bResult ? &EffectChainOutputBuffer : &InputBuffer;

					// Mix effect chain output into SubmixChainMixBuffer
					float StartFadeVolume = FadeInfo.FadeVolume.GetValue();
					FadeInfo.FadeVolume.Update(DeltaTimeSec);
					float EndFadeVolume = FadeInfo.FadeVolume.GetValue();

					TotalStartFadeVolume += StartFadeVolume;
					TotalEndFadeVolume += EndFadeVolume;

					// Mix this effect chain with other effect chains.
					ArrayMixIn(*OutBuffer, SubmixChainMixBuffer, StartFadeVolume, EndFadeVolume);
				}

				// If we processed any effects, write over the old input buffer vs mixing into it. This is basically the "wet channel" audio in a submix.
				if (bProcessedAnEffect)
				{
					// When the active effect-chain fade volumes don't sum to 1.0 (e.g. an override is fading in/out
					// with no base chain to absorb the gap), mix the unprocessed InputBuffer back in at the deficit
					// so the dry signal crossfades against the effect chains instead of cutting out. UE-378714.
					const float StartDryGain = FMath::Clamp(1.0f - TotalStartFadeVolume, 0.0f, 1.0f);
					const float EndDryGain = FMath::Clamp(1.0f - TotalEndFadeVolume, 0.0f, 1.0f);
					if (StartDryGain > 0.0f || EndDryGain > 0.0f)
					{
						ArrayMixIn(InputBuffer, SubmixChainMixBuffer, StartDryGain, EndDryGain);
					}

					FMemory::Memcpy((void*)BufferPtr, (void*)SubmixChainMixBuffer.GetData(), sizeof(float)* NumSamples);
				}

				// Update Wet Level using modulator
				if (MixerDevice->IsModulationPluginEnabled() && MixerDevice->ModulationInterface.IsValid())
				{
					WetLevelMod.ProcessControl(WetModBaseDb);
					TargetWetLevel = WetLevelMod.GetValue();
					CachedWetLevelModValue = TargetWetLevel;
				}
				else
				{
					TargetWetLevel = Audio::ConvertToLinear(WetModBaseDb);
					CachedWetLevelModValue = TargetWetLevel;
				}

				TargetWetLevel *= WetLevelModifier;

				// Apply the wet level here after processing effects. 
				if (!FMath::IsNearlyEqual(TargetWetLevel, CurrentWetLevel) || !FMath::IsNearlyEqual(CurrentWetLevel, 1.0f))
				{
					if (FMath::IsNearlyEqual(TargetWetLevel, CurrentWetLevel))
					{
						ArrayMultiplyByConstantInPlace(InputBuffer, TargetWetLevel);
					}
					else
					{
						ArrayFade(InputBuffer, CurrentWetLevel, TargetWetLevel);
						CurrentWetLevel = TargetWetLevel;
					}
				}
			}
		}

		// Mix in the dry channel buffer
		if (DryChannelBuffer.Num() > 0)
		{
			// If we've already set the volume, only need to multiply by constant
			if (FMath::IsNearlyEqual(TargetDryLevel, CurrentDryLevel))
			{
				ArrayMultiplyByConstantInPlace(DryChannelBuffer, TargetDryLevel);
			}
			else
			{
				// To avoid popping, we do a fade on the buffer to the target volume
				ArrayFade(DryChannelBuffer, CurrentDryLevel, TargetDryLevel);
				CurrentDryLevel = TargetDryLevel;
			}
			ArrayMixIn(DryChannelBuffer, InputBuffer);
		}

		// If we're muted, memzero the buffer. Note we are still doing all the work to maintain buffer state between mutings.
		if (bIsBackgroundMuted)
		{
			FMemory::Memzero((void*)BufferPtr, sizeof(float) * NumSamples);
		}
	
		// If we are recording, Add out buffer to the RecordingData buffer:
		{
			FScopeLock ScopedLock(&RecordingCriticalSection);
			if (bIsRecording)
			{
				// TODO: Consider a scope lock between here and OnStopRecordingOutput.
				RecordingData.Append((float*)BufferPtr, NumSamples);
			}
		}

		// If spectrum analysis is enabled for this submix, downmix the resulting audio
		// and push it to the spectrum analyzer.
		{
			FScopeTryLock TryLock(&SpectrumAnalyzerCriticalSection);

			if (TryLock.IsLocked() && SpectrumAnalyzer.IsValid())
			{
				MixBufferDownToMono(InputBuffer, NumChannels, MonoMixBuffer);
				SpectrumAnalyzer->PushAudio(MonoMixBuffer.GetData(), MonoMixBuffer.Num());
				SpectrumAnalyzer->PerformAsyncAnalysisIfPossible(true);
			}
		}

		// Perform any envelope following if we're told to do so
		if (bIsEnvelopeFollowing)
		{
			const int32 BufferSamples = InputBuffer.Num();
			const float* AudioBufferPtr = InputBuffer.GetData();

			// Perform envelope following per channel
			FScopeLock EnvelopeScopeLock(&EnvelopeCriticalSection);
			FMemory::Memset(EnvelopeValues, sizeof(float) * AUDIO_MIXER_MAX_OUTPUT_CHANNELS);

			if (NumChannels > 0)
			{
				const int32 NumFrames = BufferSamples / NumChannels;
				if (EnvelopeFollower.GetNumChannels() != NumChannels)
				{
					EnvelopeFollower.SetNumChannels(NumChannels);
				}

				EnvelopeFollower.ProcessAudio(AudioBufferPtr, NumFrames);
				const TArray<float>& EnvValues = EnvelopeFollower.GetEnvelopeValues();

				check(EnvValues.Num() == NumChannels);

				FMemory::Memcpy(EnvelopeValues, EnvValues.GetData(), sizeof(float) * NumChannels);
				Audio::ArrayClampInPlace(MakeArrayView(EnvelopeValues, NumChannels), 0.f, 1.f);
			}

			EnvelopeNumChannels = NumChannels;
		}

		// Update output volume using modulator
		if (MixerDevice->IsModulationPluginEnabled() && MixerDevice->ModulationInterface.IsValid())
		{
			VolumeMod.ProcessControl(VolumeModBaseDb);
			TargetOutputVolume = VolumeMod.GetValue();
			CachedVolumeModValue = TargetOutputVolume;
		}
		else
		{
			TargetOutputVolume = Audio::ConvertToLinear(VolumeModBaseDb);
			CachedVolumeModValue = TargetOutputVolume;
		}

		TargetOutputVolume *= VolumeModifier;
		TargetOutputVolume *= GetDebugVolume();

		// Now apply the output volume
		if (!FMath::IsNearlyEqual(TargetOutputVolume, CurrentOutputVolume) || !FMath::IsNearlyEqual(CurrentOutputVolume, 1.0f))
		{
			// If we've already set the output volume, only need to multiply by constant
			if (FMath::IsNearlyEqual(TargetOutputVolume, CurrentOutputVolume))
			{
				Audio::ArrayMultiplyByConstantInPlace(InputBuffer, TargetOutputVolume);
			}
			else
			{
				// To avoid popping, we do a fade on the buffer to the target volume
				Audio::ArrayFade(InputBuffer, CurrentOutputVolume, TargetOutputVolume);
				CurrentOutputVolume = TargetOutputVolume;
			}
		}

#if UE_AUDIO_PROFILERTRACE_ENABLED
		// Trace envelope values scaled by the final output volume so Audio Insights reflects the audible signal (and not raw pre-volume envelope).
		if (bIsEnvelopeFollowing)
		{
			TraceSubmixEnvelopeValues(NumChannels, TargetOutputVolume);
		}
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

		SendAudioToSubmixBufferListeners(InputBuffer);
		SendAudioToRegisteredAudioBuses(InputBuffer);

		// Mix the audio buffer of this submix with the audio buffer of the output buffer (i.e. with other submixes)
		Audio::ArrayMixIn(InputBuffer, OutAudioBuffer);

		// Once we've finished rendering submix audio, check if the output buffer is silent if we are auto-disabling
		bool bIsNowSilent = false;

#if UE_AUDIO_PROFILERTRACE_ENABLED
		const bool bShouldTraceHasActivityMsg = UE_TRACE_CHANNELEXPR_IS_ENABLED(AudioChannel) && OwningSubmixObject.IsValid();
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

		if (bAutoDisable && !bNeverDisableSubmixes)
		{
			bIsNowSilent = FMixerSubmixPrivate::IsAudioBufferSilent(OutAudioBuffer);

			if (bIsNowSilent)
			{
				// If this is the first time we're silent track when it happens
				if (!bIsSilent)
				{
					bIsSilent = true;
					SilenceTimeStartSeconds = MixerDevice->GetAudioClock();
				}
			}
			else
			{
				bIsSilent = false;
				SilenceTimeStartSeconds = -1.0;
			}
		}
		else
		{
#if UE_AUDIO_PROFILERTRACE_ENABLED
			if (bShouldTraceHasActivityMsg)
			{
				bIsNowSilent = FMixerSubmixPrivate::IsAudioBufferSilent(OutAudioBuffer);
			}
#endif // UE_AUDIO_PROFILERTRACE_ENABLED
		}

#if UE_AUDIO_PROFILERTRACE_ENABLED
		if (bShouldTraceHasActivityMsg)
		{
			UE_TRACE_LOG(Audio, SubmixHasActivity, AudioChannel)
				<< SubmixHasActivity.DeviceId(MixerDevice->DeviceID)
				<< SubmixHasActivity.SubmixId(GetTypeHash(OwningSubmixObject->GetPathName()))
				<< SubmixHasActivity.Timestamp(FPlatformTime::Cycles64())
				<< SubmixHasActivity.HasActivity(!bIsNowSilent);
		}
#endif // UE_AUDIO_PROFILERTRACE_ENABLED
	}

	void FMixerSubmix::SendAudioToSubmixBufferListeners(FAlignedFloatBuffer& OutAudioBuffer)
	{
		// Now loop through any buffer listeners and feed the listeners the result of this audio callback
		if (const USoundSubmix* SoundSubmix = Cast<const USoundSubmix>(OwningSubmixObject))
		{
			CSV_SCOPED_TIMING_STAT(Audio, SubmixBufferListeners);
			SCOPE_CYCLE_COUNTER(STAT_AudioMixerSubmixBufferListeners);

			double AudioClock = MixerDevice->GetAudioTime();
			float SampleRate = MixerDevice->GetSampleRate();
			
			FScopeLock Lock(&BufferListenerCriticalSection);
			for (TWeakPtr<ISubmixBufferListener, ESPMode::ThreadSafe>& BufferListenerWeakPtr : BufferListenerPtrs)
			{
				if (TSharedPtr<ISubmixBufferListener> BufferListener = BufferListenerWeakPtr.Pin())
				{
					BufferListener->OnNewSubmixBuffer(SoundSubmix, OutAudioBuffer.GetData(), OutAudioBuffer.Num(), NumChannels, SampleRate, AudioClock);
				}
			}

			PatchSplitter.PushAudio(OutAudioBuffer.GetData(), OutAudioBuffer.Num());
			PruneSubmixBufferListeners();
		}
	}

	void FMixerSubmix::SendAudioToRegisteredAudioBuses(FAlignedFloatBuffer& OutAudioBuffer)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FMixerSubmix::SendAudioToRegisteredAudioBuses);

		if (FMixerSourceManager* AudioMixerSourceManager = MixerDevice->GetSourceManager())
		{
			int32 NumFrames = OutAudioBuffer.Num() / NumChannels;

			for (auto& [AudioBusKey, PatchInput] : AudioBuses)
			{
				// Submix and Audio Bus do not necessarily have the same channel configuration, so we must downmix first
				int32 BusChannelCount = AudioMixerSourceManager->GetAudioBusNumChannels(AudioBusKey);
				// It's possible for the buses referenced by this submix to be GC'd/destroyed, and in those cases the channel count will return 0.
				// Todo: remove buses from the list of registered buses when they are destroyed.
				if (BusChannelCount > 0)
				{
					DownmixBuffer(NumChannels, OutAudioBuffer, BusChannelCount, AudioBusSendBuffer);
					PatchInput.PushAudio(AudioBusSendBuffer.GetData(), NumFrames * BusChannelCount);
				}
			}
		}
	}

	void FMixerSubmix::UnregisterBufferListenerInternal(UPTRINT ListenerBufferPtr) 
	{
		FScopeLock Lock(&BufferListenerCriticalSection);
		BufferListenerPtrs.RemoveAll(
			[=](const TWeakPtr<ISubmixBufferListener>& Listener)
			{
				Listener.GetWeakPtrTypeHash();
				return reinterpret_cast<UPTRINT>(Listener.Pin().Get()) == ListenerBufferPtr;
			});
	}

	void FMixerSubmix::PruneSubmixBufferListeners()
	{
		FScopeLock Lock(&BufferListenerCriticalSection);
		BufferListenerPtrs.RemoveAll(
			[](const TWeakPtr<ISubmixBufferListener>& Listener)
			{
				return !Listener.IsValid();
			});
	}

	bool FMixerSubmix::GenerateEffectChainAudio(FSoundEffectSubmixInputData& InputData, const FAlignedFloatBuffer& InAudioBuffer, TArray<FSoundEffectSubmixPtr>& InEffectChain, FAlignedFloatBuffer& OutBuffer)
	{
		checkf(InAudioBuffer.Num() == OutBuffer.Num(), TEXT("Processing effect chain audio must not alter the number of samples in a buffer. Buffer size mismatch InAudioBuffer[%d] != OutBuffer[%d]"), InAudioBuffer.Num(), OutBuffer.Num());

		// Use the scratch buffer to hold the wet audio. For the first effect in the effect chain, this is the InAudioBuffer. 
		ScratchBuffer = InAudioBuffer;

		FSoundEffectSubmixOutputData OutputData;
		OutputData.AudioBuffer = &OutBuffer;
		OutputData.NumChannels = NumChannels;

		const int32 NumOutputFrames = OutBuffer.Num() / NumChannels;
		bool bProcessedAnEffect = false;

		for (FSoundEffectSubmixPtr& SubmixEffect : InEffectChain)
		{
			// SubmixEffectInfo.EffectInstance will be null if FMixerSubmix::RemoveSoundEffectSubmix was called earlier.
			if (!SubmixEffect.IsValid())
			{
				continue;
			}

			// Check to see if we need to down-mix our audio before sending to the submix effect
			const uint32 ChannelCountOverride = SubmixEffect->GetDesiredInputChannelCountOverride();
			if (ChannelCountOverride != INDEX_NONE && ChannelCountOverride != NumChannels)
			{
				// Perform the down-mix operation with the down-mixed scratch buffer
				DownmixedBuffer.SetNumUninitialized(NumOutputFrames * ChannelCountOverride);
				DownmixBuffer(NumChannels, ScratchBuffer, ChannelCountOverride, DownmixedBuffer);

				InputData.NumChannels = ChannelCountOverride;
				InputData.AudioBuffer = &DownmixedBuffer;
			}
			else
			{
				// If we're not down-mixing, then just pass in the current wet buffer and our channel count is the same as the output channel count
				InputData.NumChannels = NumChannels;
				InputData.AudioBuffer = &ScratchBuffer;
			}

			if (SubmixEffect->ProcessAudio(InputData, OutputData))
			{
				AUDIO_CHECK_BUFFER_NAMED_MSG(*OutputData.AudioBuffer,TEXT("Submix Effects"), TEXT("FxPreset=%s, Submix=%s"), 
					*GetNameSafe(SubmixEffect->GetPreset()), *GetName());
				
				// Mix in the dry signal directly
				const float DryLevel = SubmixEffect->GetDryLevel();
				if (DryLevel > 0.0f)
				{
					ArrayMixIn(ScratchBuffer, *OutputData.AudioBuffer, DryLevel);
				}
				// Copy the output to the input
				FMemory::Memcpy((void*)ScratchBuffer.GetData(), (void*)OutputData.AudioBuffer->GetData(), sizeof(float) * NumSamples);

				bProcessedAnEffect = true;
			}
		}

		return bProcessedAnEffect;
	}

	void FMixerSubmix::ProcessAudio(ISoundfieldAudioPacket& OutputAudio)
	{
		check(IsSoundfieldSubmix());
		PumpCommandQueue();

		FMixerSourceManager* SourceManager = MixerDevice->GetSourceManager();

		// Mix all submix audio into OutputAudio.
		{
			CSV_SCOPED_TIMING_STAT(Audio, SubmixSoundfieldChildren);
			SCOPE_CYCLE_COUNTER(STAT_AudioMixerSubmixSoundfieldChildren);

			// If we are mixing down all non-soundfield child submixes,
			// Set up the scratch buffer so that we can sum all non-soundfield child submixes to it.
			if (SoundfieldStreams.DownmixedChildrenEncoder.IsValid())
			{
				ScratchBuffer.Reset();
				ScratchBuffer.AddZeroed(MixerDevice->GetNumOutputFrames() * MixerDevice->GetNumDeviceChannels());
			}

			// First loop this submix's child submixes that are soundfields mixing in their output into this submix's dry/wet buffers.
			for (auto& ChildSubmixEntry : ChildSubmixes)
			{
				MixInChildSubmix(ChildSubmixEntry.Value, OutputAudio);
			}

			// If we mixed down all non-soundfield child submixes,
			// We encode and mix in here.
			if (ChildSubmixes.Num() && SoundfieldStreams.DownmixedChildrenEncoder.IsValid())
			{
				FSoundfieldSpeakerPositionalData PositionalData = GetDefaultPositionalDataForAudioDevice();

				FSoundfieldEncoderInputData InputData = {
						ScratchBuffer, /* AudioBuffer */
						MixerDevice->GetNumDeviceChannels(), /* NumChannels */
						*SoundfieldStreams.Settings, /** InputSettings */
						PositionalData /** PosititonalData */
				};

				SoundfieldStreams.DownmixedChildrenEncoder->EncodeAndMixIn(InputData, OutputAudio);
			}
		}

		// Mix all source sends into OutputAudio.
		{
			CSV_SCOPED_TIMING_STAT(Audio, SubmixSoundfieldSources);
			CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_AudioMixerSubmixSoundfieldSources, (MixerSourceIds.Num() > 0));

			check(SoundfieldStreams.Mixer.IsValid());

			// Loop through this submix's sound sources
			for (const auto& MixerSourceVoiceIter : MixerSourceIds)
			{
				const int32 SourceId = MixerSourceVoiceIter.Key;
				const float SendLevel = MixerSourceVoiceIter.Value.SendLevel;

				// if this voice has a valid encoded packet, mix it in.
				const ISoundfieldAudioPacket* Packet = SourceManager->GetEncodedOutput(SourceId, GetKeyForSubmixEncoding());
				UpdateListenerRotation(SourceManager->GetListenerRotation(SourceId));

				if (Packet)
				{
					FSoundfieldMixerInputData InputData =
					{
						*Packet,
						*SoundfieldStreams.Settings,
						SendLevel
					};

					SoundfieldStreams.Mixer->MixTogether(InputData, OutputAudio);
				}
			}
		}

		// Run soundfield processors.
		{
			CSV_SCOPED_TIMING_STAT(Audio, SubmixSoundfieldProcessors);
			CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_AudioMixerSubmixSoundfieldProcessors, (SoundfieldStreams.EffectProcessors.Num() > 0));

			for (auto& EffectData : SoundfieldStreams.EffectProcessors)
			{
				check(EffectData.Processor.IsValid());
				check(EffectData.Settings.IsValid());

				EffectData.Processor->ProcessAudio(OutputAudio, *SoundfieldStreams.Settings, *EffectData.Settings);
			}
		}
	}

	void FMixerSubmix::ProcessAudioAndSendToEndpoint()
	{
		check(MixerDevice);

		//If this endpoint should no-op, just set the buffer to zero and return
		if (IsDummyEndpointSubmix())
		{
			EndpointData.AudioBuffer.Reset();
			EndpointData.AudioBuffer.AddZeroed(MixerDevice->GetNumOutputFrames() * MixerDevice->GetNumDeviceChannels());
			return;
		}

		if (IsSoundfieldSubmix())
		{
			if (!EndpointData.AudioPacket.IsValid())
			{
				EndpointData.AudioPacket = SoundfieldStreams.Factory->CreateEmptyPacket();
			}
			else
			{
				EndpointData.AudioPacket->Reset();
			}

			// First, process the audio chain for this submix.
			check(EndpointData.AudioPacket);
			ProcessAudio(*EndpointData.AudioPacket);

			if (EndpointData.SoundfieldEndpoint->GetRemainderInPacketBuffer() > 0)
			{
				EndpointData.SoundfieldEndpoint->PushAudio(MoveTemp(EndpointData.AudioPacket));
			}
			else
			{
				ensureMsgf(false, TEXT("Buffer overrun in Soundfield endpoint! %s may need to override ISoundfieldEndpoint::EndpointRequiresCallback() to return true."), *SoundfieldStreams.Factory->GetSoundfieldFormatName().ToString());
			}

			EndpointData.SoundfieldEndpoint->ProcessAudioIfNecessary();
		}
		else
		{
			// First, process the chain for this submix.
			EndpointData.AudioBuffer.Reset();
			EndpointData.AudioBuffer.AddZeroed(MixerDevice->GetNumOutputFrames() * MixerDevice->GetNumDeviceChannels());
			ProcessAudio(EndpointData.AudioBuffer);

			if (!EndpointData.Input.IsOutputStillActive())
			{
				// Either this is our first time pushing audio or we were disconnected.
				const float DurationPerCallback = MixerDevice->GetNumOutputFrames() / MixerDevice->GetSampleRate();

				EndpointData.Input = EndpointData.NonSoundfieldEndpoint->PatchNewInput(DurationPerCallback, EndpointData.SampleRate, EndpointData.NumChannels);

				if (!FMath::IsNearlyEqual(EndpointData.SampleRate, MixerDevice->GetSampleRate()))
				{
					// Initialize the resampler.
					float SampleRateRatio = EndpointData.SampleRate / MixerDevice->GetSampleRate();

					EndpointData.Resampler.Init(EResamplingMethod::Linear, SampleRateRatio, NumChannels);
					EndpointData.bShouldResample = true;

					// Add a little slack at the end of the resampled audio buffer in case we have roundoff jitter.
					EndpointData.ResampledAudioBuffer.Reset();
					EndpointData.ResampledAudioBuffer.AddUninitialized(EndpointData.AudioBuffer.Num() * SampleRateRatio + 16);
				}
			}

			// Resample if necessary.
			int32 NumResampledFrames = EndpointData.AudioBuffer.Num() / NumChannels;
			if (EndpointData.bShouldResample)
			{
				EndpointData.Resampler.ProcessAudio(EndpointData.AudioBuffer.GetData(), EndpointData.AudioBuffer.Num(), false, EndpointData.ResampledAudioBuffer.GetData(), EndpointData.ResampledAudioBuffer.Num(), NumResampledFrames);
			}
			else
			{
				EndpointData.ResampledAudioBuffer = MoveTemp(EndpointData.AudioBuffer);
			}

			// Downmix if necessary.
			const bool bShouldDownmix = EndpointData.NumChannels != NumChannels;
			if (bShouldDownmix)
			{
				EndpointData.DownmixedResampledAudioBuffer.Reset();
				EndpointData.DownmixedResampledAudioBuffer.AddUninitialized(NumResampledFrames * EndpointData.NumChannels);

				EndpointData.DownmixChannelMap.Reset();
				FMixerDevice::Get2DChannelMap(false, NumChannels, EndpointData.NumChannels, false, EndpointData.DownmixChannelMap);
				DownmixBuffer(NumChannels, EndpointData.ResampledAudioBuffer, EndpointData.NumChannels, EndpointData.DownmixedResampledAudioBuffer);
			}
			else
			{
				EndpointData.DownmixedResampledAudioBuffer = MoveTemp(EndpointData.ResampledAudioBuffer);
			}

			EndpointData.Input.PushAudio(EndpointData.DownmixedResampledAudioBuffer.GetData(), EndpointData.DownmixedResampledAudioBuffer.Num());
			EndpointData.NonSoundfieldEndpoint->ProcessAudioIfNeccessary();

			// If we did any pointer passing to bypass downmixing or resampling, pass the pointer back to avoid reallocating ResampledAudioBuffer or AudioBuffer.

			if (!bShouldDownmix)
			{
				EndpointData.ResampledAudioBuffer = MoveTemp(EndpointData.DownmixedResampledAudioBuffer);
			}

			if (!EndpointData.bShouldResample)
			{
				EndpointData.AudioBuffer = MoveTemp(EndpointData.ResampledAudioBuffer);
			}
		}
	}

	int32 FMixerSubmix::GetSampleRate() const
	{
		return MixerDevice->GetDeviceSampleRate();
	}

	int32 FMixerSubmix::GetNumOutputChannels() const
	{
		return MixerDevice->GetNumDeviceChannels();
	}

	int32 FMixerSubmix::GetNumChainEffects()
	{
		FScopeLock ScopeLock(&EffectChainMutationCriticalSection);
		for (const FSubmixEffectFadeInfo& FadeInfo : EffectChains)
		{
			if (FadeInfo.bIsCurrentChain)
			{
				return FadeInfo.EffectChain.Num();
			}
		}
		return 0;
	}

	FSoundEffectSubmixPtr FMixerSubmix::GetSubmixEffect(const int32 InIndex)
	{
		FScopeLock ScopeLock(&EffectChainMutationCriticalSection);
		for (const FSubmixEffectFadeInfo& FadeInfo : EffectChains)
		{
			if (FadeInfo.bIsCurrentChain)
			{
				if (InIndex < FadeInfo.EffectChain.Num())
				{
					return FadeInfo.EffectChain[InIndex];
				}
			}
		}
		return nullptr;
	}

	void FMixerSubmix::SetSoundfieldFactory(ISoundfieldFactory* InSoundfieldFactory)
	{
		SoundfieldStreams.Factory = InSoundfieldFactory;
	}

	void FMixerSubmix::SetupSoundfieldStreams(const USoundfieldEncodingSettingsBase* InAmbisonicsSettings, TArray<USoundfieldEffectBase*>& Processors, ISoundfieldFactory* InSoundfieldFactory)
	{
		FScopeLock ScopeLock(&SoundfieldStreams.StreamsLock);

		// SoundfieldStreams.Factory should have already been set by our first pass around the submix graph, since 
		// we use the soundfield factory
		check(SoundfieldStreams.Factory == InSoundfieldFactory);

		if (!InSoundfieldFactory)
		{
			return;
		}

		check(InAmbisonicsSettings != nullptr);
		// As a santity check, we ensure the passed in soundfield stream is what was used for the initial SetSoundfieldFactory call.
		check(InSoundfieldFactory == SoundfieldStreams.Factory);


		SoundfieldStreams.Reset();
		SoundfieldStreams.Factory = InSoundfieldFactory;

		// If this submix is encoded to a soundfield, 
		// Explicitly set NumChannels and NumSamples to 0 since they are technically irrelevant.
		NumChannels = 0;
		NumSamples = 0;

		SoundfieldStreams.Settings = InAmbisonicsSettings->GetProxy();

		// Check to see if this implementation of GetProxy failed.
		if (!ensureAlwaysMsgf(SoundfieldStreams.Settings.IsValid(), TEXT("Soundfield Format %s failed to create a settings proxy for settings asset %s."), *InSoundfieldFactory->GetSoundfieldFormatName().ToString(), *InAmbisonicsSettings->GetName()))
		{
			
			TeardownSoundfieldStreams();
			return;
		}
		
		SoundfieldStreams.Mixer = InSoundfieldFactory->CreateMixerStream(*SoundfieldStreams.Settings);

		if (!ensureAlwaysMsgf(SoundfieldStreams.Mixer.IsValid(), TEXT("Soundfield Format %s failed to create a settings proxy for settings asset %s."), *InSoundfieldFactory->GetSoundfieldFormatName().ToString(), *InAmbisonicsSettings->GetName()))
		{
			TeardownSoundfieldStreams();
			return;
		}

		// Create new processor proxies.
		for (USoundfieldEffectBase* Processor : Processors)
		{
			if (Processor != nullptr)
			{
				SoundfieldStreams.EffectProcessors.Emplace(SoundfieldStreams.Factory, *SoundfieldStreams.Settings, Processor);
			}
		}

		SetupSoundfieldEncodersForChildren();
		SetupSoundfieldStreamForParent();
	}

	void FMixerSubmix::TeardownSoundfieldStreams()
	{
		SoundfieldStreams.Reset();

		for (auto& ChildSubmix : ChildSubmixes)
		{
			ChildSubmix.Value.Encoder.Reset();
			ChildSubmix.Value.Transcoder.Reset();
		}
	}

	void FMixerSubmix::SetupEndpoint(IAudioEndpointFactory* InFactory, const UAudioEndpointSettingsBase* InSettings)
	{
		checkf(!IsSoundfieldSubmix(), TEXT("Soundfield Endpoint Submixes called with non-soundfield arguments."));
		check(!ParentSubmix.IsValid());
		EndpointData.Reset();

		if (!InFactory)
		{
			return;
		}

		TUniquePtr<IAudioEndpointSettingsProxy> SettingsProxy;
		if (InSettings)
		{
			SettingsProxy = InSettings->GetProxy();
		}
		else
		{
			InSettings = InFactory->GetDefaultSettings();
			ensureMsgf(InSettings, TEXT("The audio endpoint factory %s failed to generate default settings!"), *InFactory->GetEndpointTypeName().ToString());

			if (InSettings)
			{
				SettingsProxy = InSettings->GetProxy();
			}
		}

		if (SettingsProxy)
		{
			FAudioPluginInitializationParams InitParams = GetInitializationParamsForSoundfieldStream();
			EndpointData.NonSoundfieldEndpoint = InFactory->CreateNewEndpointInstance(InitParams, *SettingsProxy);
		}
		else
		{
			ensureMsgf(false, TEXT("Settings object %s failed to create a proxy object. Likely an error in the implementation of %s::GetProxy()."), *InSettings->GetName(), *InSettings->GetClass()->GetName());
		}
	}

	void FMixerSubmix::SetupEndpoint(ISoundfieldEndpointFactory* InFactory, const USoundfieldEndpointSettingsBase* InSettings)
	{
		checkf(IsSoundfieldSubmix(), TEXT("Non-Soundfield Endpoint Submixes called with soundfield arguments."));
		check(SoundfieldStreams.Factory == InFactory);
		check(!ParentSubmix.IsValid());

		EndpointData.Reset();

		if (!InFactory)
		{
			return;
		}

		TUniquePtr<ISoundfieldEndpointSettingsProxy> SettingsProxy;
		if (InSettings)
		{
			SettingsProxy = InSettings->GetProxy();
		}
		else
		{
			InSettings = InFactory->GetDefaultEndpointSettings();
			ensureMsgf(InSettings, TEXT("The audio endpoint factory %s failed to generate default settings!"), *InFactory->GetEndpointTypeName().ToString());

			if (InSettings)
			{
				SettingsProxy = InSettings->GetProxy();
			}
		}

		if (SettingsProxy)
		{
			FAudioPluginInitializationParams InitParams = GetInitializationParamsForSoundfieldStream();
			EndpointData.SoundfieldEndpoint = InFactory->CreateNewEndpointInstance(InitParams, *SettingsProxy);
		}
		else
		{
			ensureMsgf(false, TEXT("Settings object %s failed to create a proxy object. Likely an error in the implementation of %s::GetProxy()."), *InSettings->GetName(), *InSettings->GetClass()->GetName());
		}
	}

	void FMixerSubmix::UpdateEndpointSettings(TUniquePtr<IAudioEndpointSettingsProxy>&& InSettings)
	{
		checkf(!IsSoundfieldSubmix(), TEXT("UpdateEndpointSettings for a soundfield submix was called with an IAudioEndpointSettingsProxy rather than an ISoundfieldEndpointSettingsProxy."));
		if (ensureMsgf(EndpointData.NonSoundfieldEndpoint.IsValid(), TEXT("UpdateEndpointSettings called on an object that is not an endpoint submix.")))
		{
			EndpointData.NonSoundfieldEndpoint->SetNewSettings(MoveTemp(InSettings));
		}
	}

	void FMixerSubmix::UpdateEndpointSettings(TUniquePtr<ISoundfieldEndpointSettingsProxy>&& InSettings)
	{
		checkf(IsSoundfieldSubmix(), TEXT("UpdateEndpointSettings for a non-soundfield submix was called with an ISoundfieldEndpointSettingsProxy rather than an IAudioEndpointSettingsProxy."));
		if (ensureMsgf(EndpointData.SoundfieldEndpoint.IsValid(), TEXT("UpdateEndpointSettings called on an object that is not an endpoint submix.")))
		{
			EndpointData.SoundfieldEndpoint->SetNewSettings(MoveTemp(InSettings));
		}
	}

	void FMixerSubmix::OnStartRecordingOutput(float ExpectedDuration)
	{
		RecordingData.Reset();
		RecordingData.Reserve(ExpectedDuration * GetSampleRate());
		bIsRecording = true;
	}

	FAlignedFloatBuffer& FMixerSubmix::OnStopRecordingOutput(float& OutNumChannels, float& OutSampleRate)
	{
		FScopeLock ScopedLock(&RecordingCriticalSection);
		bIsRecording = false;
		OutNumChannels = NumChannels;
		OutSampleRate = GetSampleRate();
		return RecordingData;
	}

	void FMixerSubmix::PauseRecordingOutput()
	{
		if (!RecordingData.Num())
		{
			UE_LOGF(LogAudioMixer, Warning, "Cannot pause recording output as no recording is in progress.");
			return;
		}
		
		bIsRecording = false;
	}

	void FMixerSubmix::ResumeRecordingOutput()
	{
		if (!RecordingData.Num())
		{
			UE_LOGF(LogAudioMixer, Warning, "Cannot resume recording output as no recording is in progress.");
			return;
		}
		bIsRecording = true;
	}

	void FMixerSubmix::RegisterBufferListener(TSharedRef<ISubmixBufferListener, ESPMode::ThreadSafe> BufferListener)
	{
		FScopeLock Lock(&BufferListenerCriticalSection);
		BufferListenerPtrs.AddUnique(BufferListener);
	}

	void FMixerSubmix::UnregisterBufferListener(TSharedRef<ISubmixBufferListener, ESPMode::ThreadSafe> BufferListener)
	{
		FScopeLock Lock(&BufferListenerCriticalSection);
		BufferListenerPtrs.Remove(BufferListener);
	}

	void FMixerSubmix::StartEnvelopeFollowing(int32 AttackTime, int32 ReleaseTime)
	{
		EnvelopeFollowerRequestsRefCount.Increment();

		if (!bIsEnvelopeFollowing)
		{
			FEnvelopeFollowerInitParams EnvelopeFollowerInitParams;
			EnvelopeFollowerInitParams.SampleRate = GetSampleRate(); 
			EnvelopeFollowerInitParams.NumChannels = NumChannels; 
			EnvelopeFollowerInitParams.AttackTimeMsec = static_cast<float>(AttackTime);
			EnvelopeFollowerInitParams.ReleaseTimeMsec = static_cast<float>(ReleaseTime);
			EnvelopeFollower.Init(EnvelopeFollowerInitParams);

			// Zero out any previous envelope values which may have been in the array before starting up
			for (int32 ChannelIndex = 0; ChannelIndex < AUDIO_MIXER_MAX_OUTPUT_CHANNELS; ++ChannelIndex)
			{
				EnvelopeValues[ChannelIndex] = 0.0f;
			}

			bIsEnvelopeFollowing = true;

#if UE_AUDIO_PROFILERTRACE_ENABLED
			const bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AudioMixerChannel);
			if (bChannelEnabled && OwningSubmixObject.IsValid())
			{
				UE_TRACE_LOG(Audio, SubmixEnvelopeFollowerEnabled, AudioMixerChannel)
					<< SubmixEnvelopeFollowerEnabled.DeviceId(MixerDevice->DeviceID)
					<< SubmixEnvelopeFollowerEnabled.SubmixId(GetTypeHash(OwningSubmixObject->GetPathName()))
					<< SubmixEnvelopeFollowerEnabled.Timestamp(FPlatformTime::Cycles64())
					<< SubmixEnvelopeFollowerEnabled.EnvelopeFollowerEnabled(true);
			}
#endif // UE_AUDIO_PROFILERTRACE_ENABLED
		}
	}

	void FMixerSubmix::StopEnvelopeFollowing()
	{
		if (!bIsEnvelopeFollowing)
		{
			return;
		}

		EnvelopeFollowerRequestsRefCount.Decrement();
		if (EnvelopeFollowerRequestsRefCount.GetValue() <= 0)
		{
			EnvelopeFollowerRequestsRefCount.Reset();
			bIsEnvelopeFollowing = false;

#if UE_AUDIO_PROFILERTRACE_ENABLED
			const bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(AudioMixerChannel);
			if (bChannelEnabled && OwningSubmixObject.IsValid())
			{
				UE_TRACE_LOG(Audio, SubmixEnvelopeFollowerEnabled, AudioMixerChannel)
				<< SubmixEnvelopeFollowerEnabled.DeviceId(MixerDevice->DeviceID)
				<< SubmixEnvelopeFollowerEnabled.SubmixId(GetTypeHash(OwningSubmixObject->GetPathName()))
				<< SubmixEnvelopeFollowerEnabled.Timestamp(FPlatformTime::Cycles64())
				<< SubmixEnvelopeFollowerEnabled.EnvelopeFollowerEnabled(false);
			}
#endif // UE_AUDIO_PROFILERTRACE_ENABLED
		}
	}

	void FMixerSubmix::AddEnvelopeFollowerDelegate(const FOnSubmixEnvelopeBP& OnSubmixEnvelopeBP)
	{
		OnSubmixEnvelope.AddUnique(OnSubmixEnvelopeBP);
	}

	void FMixerSubmix::RemoveEnvelopeFollowerDelegate(const FOnSubmixEnvelopeBP& OnSubmixEnvelopeBP)
	{
		OnSubmixEnvelope.Remove(OnSubmixEnvelopeBP);
	}

	void FMixerSubmix::AddSpectralAnalysisDelegate(const FSoundSpectrumAnalyzerDelegateSettings& InDelegateSettings, const FOnSubmixSpectralAnalysisBP& OnSubmixSpectralAnalysisBP)
	{
		FSpectrumAnalysisDelegateInfo NewDelegateInfo;
	
		NewDelegateInfo.LastUpdateTime = -1.0f;
		NewDelegateInfo.DelegateSettings = InDelegateSettings;
		NewDelegateInfo.DelegateSettings.UpdateRate = FMath::Clamp(NewDelegateInfo.DelegateSettings.UpdateRate, 1.0f, 30.0f);
		NewDelegateInfo.UpdateDelta = 1.0f / NewDelegateInfo.DelegateSettings.UpdateRate;

		NewDelegateInfo.OnSubmixSpectralAnalysis.AddUnique(OnSubmixSpectralAnalysisBP);

		{
			FScopeLock SpectrumAnalyzerLock(&SpectrumAnalyzerCriticalSection);

			SpectralAnalysisDelegates.Add(MoveTemp(NewDelegateInfo));
		}
	}

	void FMixerSubmix::RemoveSpectralAnalysisDelegate(const FOnSubmixSpectralAnalysisBP& OnSubmixSpectralAnalysisBP)
	{
		FScopeLock SpectrumAnalyzerLock(&SpectrumAnalyzerCriticalSection);

		for (FSpectrumAnalysisDelegateInfo& Info : SpectralAnalysisDelegates)
		{
			if (Info.OnSubmixSpectralAnalysis.Contains(OnSubmixSpectralAnalysisBP))
			{
				Info.OnSubmixSpectralAnalysis.Remove(OnSubmixSpectralAnalysisBP);
			}
		}

		SpectralAnalysisDelegates.RemoveAllSwap([](FSpectrumAnalysisDelegateInfo& Info) {
			return !Info.OnSubmixSpectralAnalysis.IsBound();
		});
	}

	void FMixerSubmix::StartSpectrumAnalysis(const FSoundSpectrumAnalyzerSettings& InSettings)
	{
		ensure(IsInAudioThread());

		using namespace MixerSubmixIntrinsics;
		using EMetric = FSpectrumBandExtractorSettings::EMetric;
		using EBandType = ISpectrumBandExtractor::EBandType;

		bIsSpectrumAnalyzing = true;

		SpectrumAnalyzerSettings = InSettings;

		FSpectrumAnalyzerSettings AudioSpectrumAnalyzerSettings;

		AudioSpectrumAnalyzerSettings.FFTSize = GetSpectrumAnalyzerFFTSize(SpectrumAnalyzerSettings.FFTSize);
		AudioSpectrumAnalyzerSettings.WindowType = GetWindowType(SpectrumAnalyzerSettings.WindowType);
		AudioSpectrumAnalyzerSettings.HopSize = SpectrumAnalyzerSettings.HopSize;

		EMetric Metric = GetExtractorMetric(SpectrumAnalyzerSettings.SpectrumType);
		EBandType BandType = GetExtractorBandType(SpectrumAnalyzerSettings.InterpolationMethod);

		{
			FScopeLock SpectrumAnalyzerLock(&SpectrumAnalyzerCriticalSection);
			SpectrumAnalyzer = MakeShared<FAsyncSpectrumAnalyzer, ESPMode::ThreadSafe>(AudioSpectrumAnalyzerSettings, MixerDevice->GetSampleRate());


			for (FSpectrumAnalysisDelegateInfo& DelegateInfo : SpectralAnalysisDelegates)
			{
				FSpectrumBandExtractorSettings ExtractorSettings;

				ExtractorSettings.Metric = Metric;
				ExtractorSettings.DecibelNoiseFloor = DelegateInfo.DelegateSettings.DecibelNoiseFloor;
				ExtractorSettings.bDoNormalize = DelegateInfo.DelegateSettings.bDoNormalize;
				ExtractorSettings.bDoAutoRange = DelegateInfo.DelegateSettings.bDoAutoRange;
				ExtractorSettings.AutoRangeReleaseTimeInSeconds = DelegateInfo.DelegateSettings.AutoRangeReleaseTime;
				ExtractorSettings.AutoRangeAttackTimeInSeconds = DelegateInfo.DelegateSettings.AutoRangeAttackTime;

				DelegateInfo.SpectrumBandExtractor = ISpectrumBandExtractor::CreateSpectrumBandExtractor(ExtractorSettings);

				if (DelegateInfo.SpectrumBandExtractor.IsValid())
				{
					for (const FSoundSubmixSpectralAnalysisBandSettings& BandSettings : DelegateInfo.DelegateSettings.BandSettings)
					{
						ISpectrumBandExtractor::FBandSettings NewExtractorBandSettings;
						NewExtractorBandSettings.Type = BandType;
						NewExtractorBandSettings.CenterFrequency = BandSettings.BandFrequency;
						NewExtractorBandSettings.QFactor = BandSettings.QFactor;

						DelegateInfo.SpectrumBandExtractor->AddBand(NewExtractorBandSettings);

						FSpectralAnalysisBandInfo NewBand;

						FInlineEnvelopeFollowerInitParams EnvelopeFollowerInitParams;
						EnvelopeFollowerInitParams.SampleRate = DelegateInfo.DelegateSettings.UpdateRate;
						EnvelopeFollowerInitParams.AttackTimeMsec = static_cast<float>(BandSettings.AttackTimeMsec);
						EnvelopeFollowerInitParams.ReleaseTimeMsec = static_cast<float>(BandSettings.ReleaseTimeMsec);
						NewBand.EnvelopeFollower.Init(EnvelopeFollowerInitParams);
					
						DelegateInfo.SpectralBands.Add(NewBand);
					}
				} 
			}
		}
	}

	void FMixerSubmix::StopSpectrumAnalysis()
	{
		ensure(IsInAudioThread());

		FScopeLock SpectrumAnalyzerLock(&SpectrumAnalyzerCriticalSection);
		bIsSpectrumAnalyzing = false;
		SpectrumAnalyzer.Reset();
	}

	void FMixerSubmix::GetMagnitudeForFrequencies(const TArray<float>& InFrequencies, TArray<float>& OutMagnitudes)
	{
		FScopeLock SpectrumAnalyzerLock(&SpectrumAnalyzerCriticalSection);

		if (SpectrumAnalyzer.IsValid())
		{
			using EMethod = FSpectrumAnalyzer::EPeakInterpolationMethod;

			EMethod Method;	

			switch (SpectrumAnalyzerSettings.InterpolationMethod)
			{
				case EFFTPeakInterpolationMethod::NearestNeighbor:
					Method = EMethod::NearestNeighbor;
					break;

				case EFFTPeakInterpolationMethod::Linear:
					Method = EMethod::Linear;
					break;

				case EFFTPeakInterpolationMethod::Quadratic:
					Method = EMethod::Quadratic;
					break;

				default:
					Method = EMethod::Linear;
					break;
			}

			OutMagnitudes.Reset();
			OutMagnitudes.AddUninitialized(InFrequencies.Num());

			SpectrumAnalyzer->LockOutputBuffer();
			for (int32 Index = 0; Index < InFrequencies.Num(); Index++)
			{
				OutMagnitudes[Index] = SpectrumAnalyzer->GetMagnitudeForFrequency(InFrequencies[Index], Method);
			}
			SpectrumAnalyzer->UnlockOutputBuffer();
		}
		else
		{
			UE_LOGF(LogAudioMixer, Warning, "Call StartSpectrumAnalysis before calling GetMagnitudeForFrequencies.");
		}
	}

	void FMixerSubmix::GetPhaseForFrequencies(const TArray<float>& InFrequencies, TArray<float>& OutPhases)
	{
		FScopeLock SpectrumAnalyzerLock(&SpectrumAnalyzerCriticalSection);

		if (SpectrumAnalyzer.IsValid())
		{
			using EMethod = FSpectrumAnalyzer::EPeakInterpolationMethod;

			EMethod Method;	

			switch (SpectrumAnalyzerSettings.InterpolationMethod)
			{
				case EFFTPeakInterpolationMethod::NearestNeighbor:
					Method = EMethod::NearestNeighbor;
					break;

				case EFFTPeakInterpolationMethod::Linear:
					Method = EMethod::Linear;
					break;

				case EFFTPeakInterpolationMethod::Quadratic:
					Method = EMethod::Quadratic;
					break;

				default:
					Method = EMethod::Linear;
					break;
			}

			OutPhases.Reset();
			OutPhases.AddUninitialized(InFrequencies.Num());

			SpectrumAnalyzer->LockOutputBuffer();
			for (int32 Index = 0; Index < InFrequencies.Num(); Index++)
			{
				OutPhases[Index] = SpectrumAnalyzer->GetPhaseForFrequency(InFrequencies[Index], Method);
			}
			SpectrumAnalyzer->UnlockOutputBuffer();
		}
		else
		{
			UE_LOGF(LogAudioMixer, Warning, "Call StartSpectrumAnalysis before calling GetPhaseForFrequencies.");
		}
	}

	void FMixerSubmix::SetOutputVolume(float InOutputVolume)
	{
		VolumeModifier = FMath::Clamp(InOutputVolume, 0.0f, 1.0f);
	}

	void FMixerSubmix::SetDryLevel(float InDryLevel)
	{
		DryLevelModifier = FMath::Clamp(InDryLevel, 0.0f, 1.0f);
	}

	void FMixerSubmix::SetWetLevel(float InWetLevel)
	{
		WetLevelModifier = FMath::Clamp(InWetLevel, 0.0f, 1.0f);
	}

	void FMixerSubmix::UpdateModulationSettings(const TSet<TObjectPtr<USoundModulatorBase>>& InOutputModulators, const TSet<TObjectPtr<USoundModulatorBase>>& InWetLevelModulators, const TSet<TObjectPtr<USoundModulatorBase>>& InDryLevelModulators)
	{
		VolumeMod.SetModulators(InOutputModulators);
		WetLevelMod.SetModulators(InWetLevelModulators);
		DryLevelMod.SetModulators(InDryLevelModulators);
	}

	void FMixerSubmix::SetModulationBaseLevels(float InVolumeModBaseDb, float InWetModBaseDb, float InDryModBaseDb)
	{
		VolumeModBaseDb = InVolumeModBaseDb;
		WetModBaseDb = InWetModBaseDb;
		DryModBaseDb = InDryModBaseDb;
	}

	float FMixerSubmix::GetOutputVolumeModulationValueThreadSafe() const
	{
		return CachedVolumeModValue;
	}

	float FMixerSubmix::GetWetVolumeModulationValueThreadSafe() const
	{
		return CachedWetLevelModValue;
	}

	void FMixerSubmix::BroadcastDelegates()
	{
		if (bIsEnvelopeFollowing)
		{
			// Get the envelope data
			TArray<float> EnvelopeData;

			{
				// Make the copy of the envelope values using a critical section
				FScopeLock EnvelopeScopeLock(&EnvelopeCriticalSection);

				if (EnvelopeNumChannels > 0)
				{
					EnvelopeData.AddUninitialized(EnvelopeNumChannels);
					FMemory::Memcpy(EnvelopeData.GetData(), EnvelopeValues, sizeof(float)*EnvelopeNumChannels);
				}
			}

			// Broadcast to any bound delegates
			if (OnSubmixEnvelope.IsBound())
			{
				OnSubmixEnvelope.Broadcast(EnvelopeData);
			}

		}
		
		// If we're analyzing spectra and if we've got delegates setup
		if (bIsSpectrumAnalyzing) 
		{
			FScopeLock SpectrumLock(&SpectrumAnalyzerCriticalSection);

			if (SpectralAnalysisDelegates.Num() > 0)
			{
				if (ensureMsgf(SpectrumAnalyzer.IsValid(), TEXT("Analyzing spectrum with invalid spectrum analyzer")))
				{
					TArray<TPair<FSpectrumAnalysisDelegateInfo*, TArray<float>>> ResultsPerDelegate;
					
					{
						// This lock ensures that the spectrum analyzer's analysis buffer doesn't
						// change in this scope. 
						Audio::FAsyncSpectrumAnalyzerScopeLock AnalyzerLock(SpectrumAnalyzer.Get());

						for (FSpectrumAnalysisDelegateInfo& DelegateInfo : SpectralAnalysisDelegates)
						{
							TArray<float> SpectralResults;
							SpectralResults.Reset();
							const float CurrentTime = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64());

							// Don't update the spectral band until it's time since the last tick.
							if (DelegateInfo.LastUpdateTime > 0.0f && ((CurrentTime - DelegateInfo.LastUpdateTime) < DelegateInfo.UpdateDelta))
							{
								continue;
							}

							DelegateInfo.LastUpdateTime = CurrentTime;

							if (ensure(DelegateInfo.SpectrumBandExtractor.IsValid()))
							{
								ISpectrumBandExtractor* Extractor = DelegateInfo.SpectrumBandExtractor.Get();

								SpectrumAnalyzer->GetBands(*Extractor, SpectralResults);
							}
							ResultsPerDelegate.Emplace(&DelegateInfo, MoveTemp(SpectralResults));
						}
					}

					for (TPair<FSpectrumAnalysisDelegateInfo*, TArray<float>> Results : ResultsPerDelegate)
					{
						FSpectrumAnalysisDelegateInfo* DelegateInfo = Results.Key;
						// Feed the results through the band envelope followers
						for (int32 ResultIndex = 0; ResultIndex < Results.Value.Num(); ++ResultIndex)
						{
							if (ensure(ResultIndex < DelegateInfo->SpectralBands.Num()))
							{
								FSpectralAnalysisBandInfo& BandInfo = DelegateInfo->SpectralBands[ResultIndex];

								Results.Value[ResultIndex] = BandInfo.EnvelopeFollower.ProcessSample(Results.Value[ResultIndex]);
							}
						}

						if (DelegateInfo->OnSubmixSpectralAnalysis.IsBound())
						{
							DelegateInfo->OnSubmixSpectralAnalysis.Broadcast(MoveTemp(Results.Value));
						}
					}
				}
			}
		}
	}

	FSoundfieldEncodingKey FMixerSubmix::GetKeyForSubmixEncoding()
	{
		check(IsSoundfieldSubmix() && SoundfieldStreams.Settings.IsValid());
		return FSoundfieldEncodingKey(SoundfieldStreams.Factory, *SoundfieldStreams.Settings);
	}

	ISoundfieldFactory* FMixerSubmix::GetSoundfieldFactory()
	{
		return SoundfieldStreams.Factory;
	}

#if UE_AUDIO_PROFILERTRACE_ENABLED
	void FMixerSubmix::TraceSubmixEnvelopeValues(int32 InNumChannels, float InOutputVolume)
	{
		if (!UE_TRACE_CHANNELEXPR_IS_ENABLED(AudioMixerChannel) || !OwningSubmixObject.IsValid() || InNumChannels <= 0)
		{
			return;
		}

		check(InNumChannels <= AUDIO_MIXER_MAX_OUTPUT_CHANNELS);

		float ScaledEnvelopeValues[AUDIO_MIXER_MAX_OUTPUT_CHANNELS];
		FMemory::Memcpy(ScaledEnvelopeValues, EnvelopeValues, sizeof(float) * InNumChannels);

		Audio::ArrayMultiplyByConstantInPlace(MakeArrayView(ScaledEnvelopeValues, InNumChannels), InOutputVolume);
		Audio::ArrayClampInPlace(MakeArrayView(ScaledEnvelopeValues, InNumChannels), 0.0f, 1.0f);

		UE_TRACE_LOG(Audio, SubmixEnvelopeValues, AudioMixerChannel)
			<< SubmixEnvelopeValues.DeviceId(MixerDevice->DeviceID)
			<< SubmixEnvelopeValues.SubmixId(GetTypeHash(OwningSubmixObject->GetPathName()))
			<< SubmixEnvelopeValues.Timestamp(FPlatformTime::Cycles64())
			<< SubmixEnvelopeValues.EnvelopeValues(ScaledEnvelopeValues, InNumChannels);
	}
#endif // UE_AUDIO_PROFILERTRACE_ENABLED
}
