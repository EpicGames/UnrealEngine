// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioRenderScheduler.h"

#include "AudioMixerDevice.h"
#include "AudioMixerTrace.h"
#include "IAudioMixerRenderStep.h"
#include "Logging/StructuredLog.h"
#include "Tasks/Task.h"

DEFINE_LOG_CATEGORY(LogAudioRenderScheduler);

// If MixerDevice is null, don't check the thread. This is used in unit tests.
#define AUDIO_RENDER_SCHEDULER_CHECK_THREAD() ensure(MixerDevice == nullptr || MixerDevice->IsAudioRenderingThread())

namespace Audio
{
	FAudioRenderStepId FAudioRenderStepId::FromWaveInstancePlayOrder(const uint32 PlayOrder)
	{
		return PlayOrder | (uint64(1) << 63);
	}

	FAudioRenderStepId FAudioRenderStepId::FromAudioBusKey(const FAudioBusKey BusKey)
	{
		// BusKey.InstanceId is usually -1. We add 1 just to make this equal to the object id in that usual case.
		return (uint64(uint32(BusKey.InstanceId + 1)) << 32) | BusKey.ObjectId;
	}

	FAudioRenderScheduler::FAudioRenderScheduler(FMixerDevice* InMixerDevice) :
		MixerDevice(InMixerDevice)
	{
#if !WITH_DEV_AUTOMATION_TESTS
		// A null mixer device is only allowed in unit tests.
		check(MixerDevice != nullptr);
#endif
	}

	FAudioRenderScheduler::FInternalIndex FAudioRenderScheduler::GetInternalIndex(const FAudioRenderStepId Id)
	{
		FInternalIndex* IndexInMap = IndexMap.Find(Id);
		if (IndexInMap)
		{
			return *IndexInMap;
		}

		FInternalIndex Index;
		if (FirstFreeIndex != INDEX_NONE)
		{
			// Pop a step entry off the free list if available
			Index = FirstFreeIndex;
			FirstFreeIndex = Steps[Index].NextFreeIndex;
		}
		else
		{
			// Otherwise grow the steps array
			check(Steps.Num() < TNumericLimits<FInternalIndex>::Max());
			Steps.AddDefaulted(1);
			Index = Steps.Num() - 1;
		}

		Steps[Index].StepId = Id;
		IndexMap.Add(Id, Index);
		ActiveSteps.Add(Index);

		return Index;
	}

	void FAudioRenderScheduler::RemoveReference(FAudioRenderScheduler::FInternalIndex Index)
	{
		FStepEntry& Entry = Steps[Index];
		check(Entry.ReferenceCount >= 1 && Entry.StepId.IsValid());
		if (--Entry.ReferenceCount == 0)
		{
			IndexMap.Remove(Entry.StepId);
			ActiveSteps.Remove(Index);
			Entry = FStepEntry{};
			Entry.NextFreeIndex = FirstFreeIndex;
			FirstFreeIndex = Index;
		}
	}

	void FAudioRenderScheduler::AddStep(const FAudioRenderStepId Id, IAudioMixerRenderStep* Step)
	{
		AUDIO_RENDER_SCHEDULER_CHECK_THREAD();

		check(Id.IsValid() && Step != nullptr);
		FInternalIndex Index = GetInternalIndex(Id);
		FStepEntry& Entry = Steps[Index];
		if (Entry.StepInterface == nullptr)
		{
			UE_LOGFMT(LogAudioRenderScheduler, Verbose, "Adding render step '{Name}' with id {Id}",
				("Name", Step->GetRenderStepName()),
				("Id", Id.GetRawValue()));
			Entry.StepInterface = Step;
			Entry.ReferenceCount++;
			bDirty = true;
		}
		else
		{
			UE_LOGFMT(LogAudioRenderScheduler, Error, "Tried to add duplicate render step '{Name}' with id {Id}",
				("Name", Step->GetRenderStepName()),
				("Id", Id.GetRawValue()));
#if ENABLE_AUDIO_DEBUG
			DumpOnError();
#endif
		}
	}

	void FAudioRenderScheduler::RemoveStep(const FAudioRenderStepId Id)
	{
		AUDIO_RENDER_SCHEDULER_CHECK_THREAD();

		check(Id.IsValid());
		FInternalIndex Index = GetInternalIndex(Id);
		FStepEntry& Entry = Steps[Index];
		if (Entry.StepInterface != nullptr)
		{
			UE_LOGFMT(LogAudioRenderScheduler, Verbose, "Removing render step '{Name}' with id {Id}",
				("Name", Entry.StepInterface->GetRenderStepName()),
				("Id", Id.GetRawValue()));
			Entry.StepInterface = nullptr;
			RemoveReference(Index);
			bDirty = true;
		}
		else
		{
			UE_LOGFMT(LogAudioRenderScheduler, Error, "Tried to remove unknown render step with id {Id}",
				("Id", Id.GetRawValue()));
#if ENABLE_AUDIO_DEBUG
			DumpOnError();
#endif
		}
	}

	void FAudioRenderScheduler::AddDependency(const FAudioRenderStepId FirstStep, const FAudioRenderStepId SecondStep)
	{
		AUDIO_RENDER_SCHEDULER_CHECK_THREAD();

		check(FirstStep.IsValid() && SecondStep.IsValid());
		if (FirstStep == SecondStep)
		{
			// We allow this, but ignore it because this dependency will always need to be broken anyway.
			return;
		}

		const FInternalIndex FirstIndex = GetInternalIndex(FirstStep);
		const FInternalIndex SecondIndex = GetInternalIndex(SecondStep);

		Steps[SecondIndex].Prerequisites.Add(FirstIndex);
		Steps[SecondIndex].ReferenceCount++;
		Steps[FirstIndex].ReferenceCount++;

		bDirty = true;
	}

	void FAudioRenderScheduler::RemoveDependency(const FAudioRenderStepId FirstStep, const FAudioRenderStepId SecondStep)
	{
		AUDIO_RENDER_SCHEDULER_CHECK_THREAD();

		check(FirstStep.IsValid() && SecondStep.IsValid());
		if (FirstStep == SecondStep)
		{
			return;
		}

		const FInternalIndex FirstIndex = GetInternalIndex(FirstStep);
		const FInternalIndex SecondIndex = GetInternalIndex(SecondStep);

		if (Steps[SecondIndex].Prerequisites.RemoveSingleSwap(FirstIndex))
		{
			RemoveReference(FirstIndex);
			RemoveReference(SecondIndex);
		}
		else
		{
			UE_LOGFMT(LogAudioRenderScheduler, Error, "Unknown dependency removed from audio render scheduler: {Id1} -> {Id2}",
				("Id1", FirstStep.GetRawValue()),
				("Id2", SecondStep.GetRawValue()));
#if ENABLE_AUDIO_DEBUG
			DumpOnError();
#endif
		}

		bDirty = true;
	}

	void FAudioRenderScheduler::RenderBlock(const bool bSingleThreaded)
	{
		AUDIO_RENDER_SCHEDULER_CHECK_THREAD();
		AUDIO_MIXER_TRACE_CPUPROFILER_EVENT_SCOPE(FAudioRenderScheduler::RenderBlock);

		if (bSingleThreaded)
		{
			ScheduleAll(true);
		}
		else
		{
			UE::Tasks::FTask Task = UE::Tasks::Launch(
				TEXT("Scheduled audio render steps"),
				[this]() { ScheduleAll(false); },
				LowLevelTasks::ETaskPriority::High);

			Task.Wait();
		}

#if ENABLE_AUDIO_DEBUG
		if (bShouldDumpSteps)
		{
			DumpStepsInternal();
			bShouldDumpSteps = false;
		}
#endif
	}

	void FAudioRenderScheduler::ScheduleAll(const bool bSingleThreaded)
	{
		AUDIO_MIXER_TRACE_CPUPROFILER_EVENT_SCOPE(FAudioRenderScheduler::ScheduleAll);

		TArray<FInternalIndex> OrderedSteps;
		OrderedSteps.Reserve(ActiveSteps.Num());
		TArray<UE::Tasks::FTask> PrerequisiteHandles;

		// If we need multiple passes, we avoid having to start from the beginning each time.
		int FirstPendingStepIndex = 0;

		// Check if any of the previously broken links need to be broken again
		CheckBrokenLinks();

		while (OrderedSteps.Num() < ActiveSteps.Num())
		{
			bool bStuck = true;

			// Go through the steps array, launching any steps that have no prerequisites or prerequisites that have already been launched.
			for (int ActiveStepIndex = FirstPendingStepIndex; ActiveStepIndex < ActiveSteps.Num(); ++ActiveStepIndex)
			{
				FStepEntry& Entry = Steps[ActiveSteps[ActiveStepIndex]];
				if (Entry.bLaunched)
				{
					if (ActiveStepIndex == FirstPendingStepIndex)
					{
						++FirstPendingStepIndex;
					}
					continue;
				}

				bool bCanLaunch = true;
				if (bDirty)
				{
					for (FInternalIndex PrerequisiteIndex : Entry.Prerequisites)
					{
						if (!Steps[PrerequisiteIndex].bLaunched)
						{
							bCanLaunch = false;
							break;
						}
					}
				}

				if (bCanLaunch)
				{
					if (bSingleThreaded)
					{
						if (Entry.StepInterface != nullptr)
						{
							Entry.StepInterface->DoRenderStep();
						}
					}
					else
					{
						// Gather prerequisite task handles
						PrerequisiteHandles.Reset();
						for (FInternalIndex PrerequisiteIndex : Entry.Prerequisites)
						{
							check(Steps[PrerequisiteIndex].TaskHandle.IsValid());
							PrerequisiteHandles.Add(Steps[PrerequisiteIndex].TaskHandle);
						}

						if (Entry.bCheckBrokenLinks)
						{
							// If this step was one end of a broken link, we prevent it from running concurrently with the other
							//  end of the link.
							for (const TPair<FInternalIndex, FInternalIndex>& BrokenLink : BrokenLinks)
							{
								if (BrokenLink.Get<0>() == ActiveSteps[ActiveStepIndex] && Steps[BrokenLink.Get<1>()].bLaunched)
								{
									PrerequisiteHandles.Add(Steps[BrokenLink.Get<1>()].TaskHandle);
								}
								if (BrokenLink.Get<1>() == ActiveSteps[ActiveStepIndex] && Steps[BrokenLink.Get<0>()].bLaunched)
								{
									PrerequisiteHandles.Add(Steps[BrokenLink.Get<0>()].TaskHandle);
								}
							}
						}

						auto TaskLambda = [&Entry]()
							{
								if (Entry.StepInterface != nullptr)
								{
									Entry.StepInterface->DoRenderStep();
								}
							};

						// Launch the task
						Entry.TaskHandle = UE::Tasks::Launch(
							Entry.StepInterface ? Entry.StepInterface->GetRenderStepName() : TEXT("Empty audio render step"),
							MoveTemp(TaskLambda),
							PrerequisiteHandles,
							LowLevelTasks::ETaskPriority::High);
						UE::Tasks::AddNested(Entry.TaskHandle);
					}

					OrderedSteps.Add(ActiveSteps[ActiveStepIndex]);
					if (ActiveStepIndex == FirstPendingStepIndex)
					{
						++FirstPendingStepIndex;
					}
					Entry.bLaunched = true;
					bStuck = false;
				}
			}

			if (bStuck)
			{
				// If we go through the whole list and find no steps that are launchable, it means there must be a cycle that needs to be broken.
				BreakCycle(FirstPendingStepIndex);
			}
		}

		// After scheduling everything, restore the broken links
		for (TPair<FInternalIndex, FInternalIndex> Link : BrokenLinks)
		{
			Steps[Link.Get<1>()].Prerequisites.Add(Link.Get<0>());
		}

		// Reset the transient state for all steps
		for (int i = 0; i < Steps.Num(); ++i)
		{
			FStepEntry& Entry = Steps[i];
			Entry.TaskHandle = {};
			Entry.bLaunched = false;
			Entry.bCheckBrokenLinks = false;
		}

		// Remember the ordering of steps for next block
		ActiveSteps = MoveTemp(OrderedSteps);

		// If no changes happen before next block, we can skip lots of the checks.
		bDirty = false;
	}

	void FAudioRenderScheduler::BreakCycle(const int FirstPendingStepIndex)
	{
		FInternalIndex CurrentStep = ActiveSteps[FirstPendingStepIndex];
		TArray<FInternalIndex> VisitedIndices;
		VisitedIndices.Add(CurrentStep);

		while (true)
		{
			FStepEntry& Entry = Steps[CurrentStep];
			check(!Entry.bLaunched);
			FInternalIndex PrerequisiteStep = INDEX_NONE;

			// Find a pending step that is a dependency of CurrentStep
			for (FInternalIndex PrerequisiteIndex : Entry.Prerequisites)
			{
				if (!Steps[PrerequisiteIndex].bLaunched)
				{
					PrerequisiteStep = PrerequisiteIndex;
					break;
				}
			}

			// If this fails, BreakCycle() was called unnecessarily (there was a step with no pending prerequisites.)
			check(PrerequisiteStep != INDEX_NONE);

			if (VisitedIndices.Contains(PrerequisiteStep))
			{
				// The link between CurrentStep and PrerequisiteStep is part of a cycle, so we'll break it.
				UE_LOGFMT(LogAudioRenderScheduler, Display, "Breaking cycle between step {Id1} and step {Id2}",
					("Id1", Steps[CurrentStep].StepId.GetRawValue()),
					("Id2", Steps[PrerequisiteStep].StepId.GetRawValue()));
				BreakLink(PrerequisiteStep, CurrentStep);
				BrokenLinks.Add(TPair<FInternalIndex, FInternalIndex>{PrerequisiteStep, CurrentStep});
				return;
			}

			CurrentStep = PrerequisiteStep;
			VisitedIndices.Add(CurrentStep);
		}

		// It shouldn't be possible to get here; each time through the loop we extend VisitedIndices and check for duplicates.
		checkNoEntry();
	}

	void FAudioRenderScheduler::CheckBrokenLinks()
	{
		for (int LinkIndex = 0; LinkIndex < BrokenLinks.Num(); )
		{
			const TPair<FInternalIndex, FInternalIndex>& Link = BrokenLinks[LinkIndex];
			if (!bDirty || (Steps[Link.Get<1>()].Prerequisites.Contains(Link.Get<0>()) && FindIndirectLink(Link.Get<1>(), Link.Get<0>())))
			{
				// This link is still part of a cycle; break it again and leave it in BrokenLinks
				BreakLink(Link.Get<0>(), Link.Get<1>());
				++LinkIndex;
			}
			else
			{
				// This link is no longer part of a cycle; remove it and continue.
				BrokenLinks.RemoveAt(LinkIndex);
			}
		}
	}

	void FAudioRenderScheduler::BreakLink(FInternalIndex FirstStep, FInternalIndex SecondStep)
	{
		Steps[SecondStep].Prerequisites.RemoveSingleSwap(FirstStep);
		Steps[FirstStep].bCheckBrokenLinks = true;
		Steps[SecondStep].bCheckBrokenLinks = true;
	}

	bool FAudioRenderScheduler::FindIndirectLink(FInternalIndex FirstStep, FInternalIndex SecondStep)
	{
		TArray<FInternalIndex> StepsToSkip;
		return FindIndirectLinkSkipping(FirstStep, SecondStep, StepsToSkip);
	}

	bool FAudioRenderScheduler::FindIndirectLinkSkipping(FInternalIndex FirstStep, FInternalIndex SecondStep, TArray<FInternalIndex>& StepsToSkip)
	{
		StepsToSkip.Add(SecondStep);
		for (FInternalIndex Prereq : Steps[SecondStep].Prerequisites)
		{
			if (Prereq == FirstStep)
			{
				return true;
			}

			if (StepsToSkip.Contains(Prereq))
			{
				continue;
			}

			if (FindIndirectLinkSkipping(FirstStep, Prereq, StepsToSkip))
			{
				return true;
			}
		}

		return false;
	}

#if ENABLE_AUDIO_DEBUG
	void FAudioRenderScheduler::DumpSteps()
	{
		bShouldDumpSteps = true;
	}

	void FAudioRenderScheduler::DumpStepsInternal()
	{
		for (const FStepEntry& Step : Steps)
		{
			if (Step.ReferenceCount == 0)
			{
				continue;
			}

			UE_LOGFMT(LogAudioRenderScheduler, Display, "Index: {Index} Id: {Id} Name: '{Name}' Prerequisites: {Prerequisites}",
				("Index", int(&Step - &Steps[0])),
				("Id", Step.StepId.GetRawValue()),
				("Name", Step.StepInterface ? Step.StepInterface->GetRenderStepName() : TEXT("(Empty)")),
				("Prerequisites", Step.Prerequisites));
		}

		UE_LOGFMT(LogAudioRenderScheduler, Display, "Step ordering: {Steps}", ("Steps", ActiveSteps));
	}

	void FAudioRenderScheduler::DumpOnError()
	{
		// Only do this once so we don't spam in a persistent error state.
		if (!bDidDumpOnError)
		{
			bDidDumpOnError = true;
			bShouldDumpSteps = true;
		}
	}
#endif
}
