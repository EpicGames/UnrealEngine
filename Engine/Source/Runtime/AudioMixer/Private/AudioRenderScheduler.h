// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioBusSubsystem.h"
#include "Containers/Map.h"
#include "Tasks/Task.h"

DECLARE_LOG_CATEGORY_EXTERN(LogAudioRenderScheduler, Display, All);

namespace Audio
{
	// Forward declarations
	struct FAudioBusKey;
	class FMixerDevice;
	struct IAudioMixerRenderStep;

	// An identifier that uniquely identifies a given render step to the scheduler.
	class FAudioRenderStepId
	{
	public:
		FAudioRenderStepId() :
			Value(INDEX_NONE)
		{
		}

		static FAudioRenderStepId FromWaveInstancePlayOrder(const uint32 PlayOrder);
		static FAudioRenderStepId FromAudioBusKey(const FAudioBusKey BusKey);

		uint64 GetRawValue() const
		{
			return Value;
		}

		friend uint32 GetTypeHash(FAudioRenderStepId Id)
		{
			return GetTypeHash(Id.Value);
		}

		friend bool operator==(const FAudioRenderStepId InLHS, const FAudioRenderStepId InRHS)
		{
			return InLHS.Value == InRHS.Value;
		}

		friend bool operator!=(const FAudioRenderStepId InLHS, const FAudioRenderStepId InRHS)
		{
			return InLHS.Value != InRHS.Value;
		}

		bool IsValid() const
		{
			return Value != INDEX_NONE;
		}

	protected:
		FAudioRenderStepId(uint64 InValue) :
			Value(InValue)
		{
		}

		uint64 Value;
	};

	// FAudioRenderScheduler allows individual audio render work items ("steps") to be added to be run as part of each rendering block.
	//  Dependencies can be declared to enforce ordering and synchronization between steps when needed. There can be cyclical
	//  dependencies, in which case the scheduler will do its best to consistently break the cycles in the same way every block;
	//  but cycles should be avoided when possible.
	//
	// Render steps are executed on worker threads using the UE Tasks system, unless single-threaded rendering is requested. Currently
	//  only source rendering uses the scheduler.
	//
	// All methods of this class are to be called from the audio render thread only.

	class FAudioRenderScheduler
	{
	public:
		UE_NONCOPYABLE(FAudioRenderScheduler);

		FAudioRenderScheduler(FMixerDevice* InMixerDevice);

		// Add/remove a render step.
		void AddStep(const FAudioRenderStepId Id, IAudioMixerRenderStep* Step);
		void RemoveStep(const FAudioRenderStepId Id);

		// Declare that one step is dependent on another. It's possible to add a dependency without
		//  having previously called AddStep; in this case the dependency will still be respected with
		//  an empty placeholder step.
		void AddDependency(const FAudioRenderStepId FirstStep, const FAudioRenderStepId SecondStep);

		// All dependencies declared with AddDependency() must be removed by a corresponding call to RemoveDependency().
		void RemoveDependency(const FAudioRenderStepId FirstStep, const FAudioRenderStepId SecondStep);

		// Execute all steps and block until they're done.
		void RenderBlock(const bool bSingleThreaded);

#if ENABLE_AUDIO_DEBUG
		// Dump the state of the scheduler after next run
		void DumpSteps();
#endif
	private:
		// Internally the step id's are mapped to smaller internal indices to reduce
		//  memory footprint.
		using FInternalIndex = int16;

		void ScheduleAll(const bool bSingleThreaded);

		FInternalIndex GetInternalIndex(const FAudioRenderStepId Id);
		void RemoveReference(FAudioRenderScheduler::FInternalIndex Index);

		void BreakCycle(const int FirstPendingStepIndex);
		void CheckBrokenLinks();
		void BreakLink(FInternalIndex FirstStep, FInternalIndex SecondStep);

		// Helpers to verify that a link is still part of a cycle
		bool FindIndirectLink(const FInternalIndex FirstStep, const FInternalIndex SecondStep);
		bool FindIndirectLinkSkipping(const FInternalIndex FirstStep, const FInternalIndex SecondStep, TArray<FInternalIndex>& StepsToSkip);

#if ENABLE_AUDIO_DEBUG
		void DumpStepsInternal();
		void DumpOnError();

		bool bShouldDumpSteps = false;
		bool bDidDumpOnError = false;
#endif

		struct FStepEntry
		{
			FAudioRenderStepId StepId;
			IAudioMixerRenderStep* StepInterface = nullptr;
			TArray<FInternalIndex, TInlineAllocator<4>> Prerequisites;
			int32 ReferenceCount = 0;
			bool bLaunched = false;
			bool bCheckBrokenLinks = false;
			UE::Tasks::FTask TaskHandle;

			// Unused step entries are added to a free list.
			FInternalIndex NextFreeIndex = INDEX_NONE;
		};

		// Only used to check the current thread; may be null.
		FMixerDevice* MixerDevice;

		TMap<FAudioRenderStepId, FInternalIndex> IndexMap;
		TArray<FStepEntry> Steps;
		FInternalIndex FirstFreeIndex = INDEX_NONE;

		// Array of all used step indices. Ends up sorted at the end of RenderBlock().
		TArray<FInternalIndex> ActiveSteps;

		// Have any changes happened since the previous call to RenderBlock()?
		bool bDirty = true;

		// Links that have been broken to eliminate cycles. These are retained between
		//  blocks to improve stability of step ordering.
		TArray<TPair<FInternalIndex, FInternalIndex>> BrokenLinks;
	};
}
