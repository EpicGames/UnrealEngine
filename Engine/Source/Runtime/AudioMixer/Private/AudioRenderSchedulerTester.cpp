// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS

#include "AudioRenderSchedulerTester.h"

#include "AudioRenderScheduler.h"
#include "IAudioMixerRenderStep.h"

namespace Audio
{
	// Simple test harness for the audio render scheduler. You can add steps identified by number and dependencies
	//  between them. Running the scheduler returns an array that shows the order the steps were executed in.
	class FAudioRenderSchedulerTesterImpl : public IAudioRenderSchedulerTester
	{
		class FTestStep : public IAudioMixerRenderStep
		{
		public:
			FTestStep(FAudioRenderSchedulerTesterImpl& InOwner, int InId) :
				Owner(InOwner),
				Id(InId)
			{
			}

		protected:
			virtual void DoRenderStep() override
			{
				FScopeLock Lock(&Owner.ResultsLock);
				Owner.Results.Add(Id);
			}

			virtual const TCHAR* GetRenderStepName() override
			{
				return TEXT("");
			}

			FAudioRenderSchedulerTesterImpl& Owner;
			int Id;
		};

		TMap<int, TUniquePtr<FTestStep>> Steps;
		FAudioRenderScheduler Scheduler{ /*MixerDevice=*/ nullptr};

		FCriticalSection ResultsLock;
		TArray<int> Results;

		virtual void AddStep(int Id) override
		{
			if (!Steps.Contains(Id))
			{
				Steps.Emplace(Id, MakeUnique<FTestStep>(*this, Id));
				Scheduler.AddStep(FAudioRenderStepId::FromWaveInstancePlayOrder(Id), Steps[Id].Get());
			}
		}

		virtual void AddDependency(int FirstStep, int SecondStep) override
		{
			Scheduler.AddDependency(FAudioRenderStepId::FromWaveInstancePlayOrder(FirstStep), FAudioRenderStepId::FromWaveInstancePlayOrder(SecondStep));
		}

		virtual void RemoveDependency(int FirstStep, int SecondStep) override
		{
			Scheduler.RemoveDependency(FAudioRenderStepId::FromWaveInstancePlayOrder(FirstStep), FAudioRenderStepId::FromWaveInstancePlayOrder(SecondStep));
		}

		virtual TArray<int> Run() override
		{
			Results.Reset();
			Scheduler.RenderBlock(false);
			return Results;
		}
	};
}

TUniquePtr<IAudioRenderSchedulerTester> IAudioRenderSchedulerTester::Create()
{
	return MakeUnique<Audio::FAudioRenderSchedulerTesterImpl>();
}

#endif //WITH_DEV_AUTOMATION_TESTS
