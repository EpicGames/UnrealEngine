// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/SynthComponent.h"

#include "AudioDevice.h"
#include "AudioMixerLog.h"
#include "Engine/World.h"
#include "Sound/AudioSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SynthComponent)


USynthSound::USynthSound(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void USynthSound::Init(USynthComponent* InSynthComponent, const int32 InNumChannels, const int32 InSampleRate, const int32 InCallbackSize)
{
	check(InSynthComponent);

	OwningSynthComponent = InSynthComponent;
	VirtualizationMode = EVirtualizationMode::PlayWhenSilent;
	NumChannels = InNumChannels;
	NumSamplesToGeneratePerCallback = InCallbackSize;
	bCanProcessAsync = true;

	Duration = INDEFINITELY_LOOPING_DURATION;
	bLooping = true;
	SampleRate = InSampleRate;
}

void USynthSound::StartOnAudioDevice(FAudioDevice* InAudioDevice)
{
}

void USynthSound::OnBeginGenerate()
{
	if (ensure(OwningSynthComponent.IsValid()))
	{
		OwningSynthComponent->OnBeginGenerate();
	}
}

int32 USynthSound::OnGeneratePCMAudio(TArray<uint8>& OutAudio, int32 NumSamples)
{
	LLM_SCOPE(ELLMTag::AudioSynthesis);

	OutAudio.Reset();

	// If running with audio mixer, the output audio buffer will be in floats already
	OutAudio.AddZeroed(NumSamples * sizeof(float));

	// Mark pending kill can null this out on the game thread in rare cases.
	if (!OwningSynthComponent.IsValid())
	{
		return 0;
	}

	return OwningSynthComponent->OnGeneratePCMAudio((float*)OutAudio.GetData(), NumSamples);

}	

void USynthSound::OnEndGenerate()
{
	// Mark pending kill can null this out on the game thread in rare cases.
	if (OwningSynthComponent.IsValid())
	{
		OwningSynthComponent->OnEndGenerate();
	}
}

ISoundGeneratorPtr USynthSound::CreateSoundGenerator(const FSoundGeneratorInitParams& InParams)
{
	if (OwningSynthComponent.IsValid())
	{
		return OwningSynthComponent->CreateSoundGeneratorInternal(InParams);
	}
	return nullptr;
}

Audio::EAudioMixerStreamDataFormat::Type USynthSound::GetGeneratedPCMDataFormat() const
{
	// Only audio mixer supports return float buffers
	return Audio::EAudioMixerStreamDataFormat::Float;
}

USynthComponent::USynthComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bAutoActivate = false;

	bStopWhenOwnerDestroyed = true;

	bEnableBusSends = true;
	bEnableBaseSubmix = true;
	bEnableSubmixSends = true;

	bNeverNeedsRenderUpdate = true;
	bUseAttachParentBound = true; // Avoid CalcBounds() when transform changes.

	bIsSynthPlaying = false;
	bIsInitialized = false;
	bIsUISound = false;
	bAlwaysPlay = true;
	Synth = nullptr;

	PreferredBufferLength = DEFAULT_PROCEDURAL_SOUNDWAVE_BUFFER_SIZE;

#if WITH_EDITORONLY_DATA
	bVisualizeComponent = false;
#endif
}

void USynthComponent::OnAudioComponentEnvelopeValue(const UAudioComponent* InAudioComponent, const USoundWave* SoundWave, const float EnvelopeValue)
{
	if (OnAudioEnvelopeValue.IsBound())
	{
		OnAudioEnvelopeValue.Broadcast(EnvelopeValue);
	}

	if (OnAudioEnvelopeValueNative.IsBound())
	{
		OnAudioEnvelopeValueNative.Broadcast(InAudioComponent, EnvelopeValue);
	}
}

void USynthComponent::BeginDestroy()
{
	Super::BeginDestroy();
	Stop();
}

void USynthComponent::Activate(bool bReset)
{
	if (bReset || ShouldActivate())
	{
		Start();
		if (IsActive())
		{
			OnComponentActivated.Broadcast(this, bReset);
		}
	}
}

void USynthComponent::Deactivate()
{
	if (ShouldActivate() == false)
	{
		Stop();

		if (!IsActive())
		{
			OnComponentDeactivated.Broadcast(this);
		}
	}
}

void USynthComponent::Initialize(int32 SampleRateOverride)
{
	// This will try to create the audio component if it hasn't yet been created
	CreateAudioComponent();

	// Try to get a proper sample rate
	int32 SampleRate = SampleRateOverride;
	if (SampleRate == INDEX_NONE)
	{
		// Check audio device if we've not explicitly been told what sample rate to use
		FAudioDevice* AudioDevice = GetAudioDevice();
		if (AudioDevice)
		{
			SampleRate = AudioDevice->SampleRate;
		}
	}

	// Only allow initialization if we have a proper sample rate
	if (SampleRate != INDEX_NONE)
	{
#if SYNTH_GENERATOR_TEST_TONE
		NumChannels = 2;
		TestSineLeft.Init(SampleRate, 440.0f, 0.5f);
		TestSineRight.Init(SampleRate, 220.0f, 0.5f);
#else
		// Initialize the synth component
		Init(SampleRate);

		if (NumChannels < 0 || NumChannels > 8)
		{
			UE_LOGF(LogAudioMixer, Error, "Synthesis component '%ls' has set an invalid channel count '%d'.", *GetName(), NumChannels);
		}

		NumChannels = FMath::Clamp(NumChannels, 1, 8);
#endif

		if (!Synth)
		{
			Synth = NewObject<USynthSound>();
		}

		// Copy sound base data to the sound
		Synth->SourceEffectChain = SourceEffectChain;
		Synth->SoundSubmixObject = SoundSubmix;
		Synth->SoundSubmixSends = SoundSubmixSends;
		Synth->BusSends = BusSends;
		Synth->PreEffectBusSends = PreEffectBusSends;
		Synth->bEnableBusSends = bEnableBusSends;
		Synth->bEnableBaseSubmix = bEnableBaseSubmix;
		Synth->bEnableSubmixSends = bEnableSubmixSends;

		Synth->Init(this, NumChannels, SampleRate, PreferredBufferLength);

		// Retrieve the synth component's audio device vs the audio component's
		if (FAudioDevice* AudioDevice = GetAudioDevice())
		{
			Synth->StartOnAudioDevice(AudioDevice);
		}
	}
}

UAudioComponent* USynthComponent::GetAudioComponent()
{
	return AudioComponent;
}

void USynthComponent::CreateAudioComponent()
{
	if (!AudioComponent)
	{
		// Create the audio component which will be used to play the procedural sound wave
		AudioComponent = NewObject<UAudioComponent>(this, NAME_None, RF_Transactional | RF_Transient | RF_TextExportTransient);
		AudioComponent->CreationMethod = CreationMethod;

		AudioComponent->OnAudioSingleEnvelopeValueNative.AddUObject(this, &USynthComponent::OnAudioComponentEnvelopeValue);

		if (!AudioComponent->GetAttachParent() && !AudioComponent->IsAttachedTo(this))
		{
			AActor* Owner = GetOwner();

			// If the media component has no owner or the owner doesn't have a world
			if (!Owner || !Owner->GetWorld())
			{
				// Attempt to retrieve the synth component's world and register the audio component with it
				// This ensures that the synth component plays on the correct world in cases where there isn't an owner
				if (UWorld* World = GetWorld())
				{
					AudioComponent->RegisterComponentWithWorld(World);
					AudioComponent->AttachToComponent(this, FAttachmentTransformRules::KeepRelativeTransform);
				}
				else
				{
					AudioComponent->SetupAttachment(this);
				}
			}
			else
			{
				AudioComponent->AttachToComponent(this, FAttachmentTransformRules::KeepRelativeTransform);
				AudioComponent->RegisterComponent();
			}
		}
	}

	if (AudioComponent)
	{
		AudioComponent->bAutoActivate = false;
		AudioComponent->bStopWhenOwnerDestroyed = true;
		AudioComponent->bShouldRemainActiveIfDropped = true;
		AudioComponent->Mobility = EComponentMobility::Movable;

#if WITH_EDITORONLY_DATA
		AudioComponent->bVisualizeComponent = false;
#endif

		// Set defaults to be the same as audio component defaults
		AudioComponent->EnvelopeFollowerAttackTime = EnvelopeFollowerAttackTime;
		AudioComponent->EnvelopeFollowerReleaseTime = EnvelopeFollowerReleaseTime;
		AudioComponent->bAlwaysPlay = bAlwaysPlay;
	}
}

void USynthComponent::OnRegister()
{
	CreateAudioComponent();

	Super::OnRegister();
}

void USynthComponent::OnUnregister()
{
	// Route OnUnregister event.
	Super::OnUnregister();

	// Don't stop audio and clean up component if owner has been destroyed (default behaviour). This function gets
	// called from AActor::ClearComponents when an actor gets destroyed which is not usually what we want for one-
	// shot sounds.
	AActor* Owner = GetOwner();
	if (!Owner || bStopWhenOwnerDestroyed)
	{
		Stop();
	}

	// Make sure the audio component is destroyed during unregister
	if (AudioComponent && !AudioComponent->IsBeingDestroyed())
	{
		if (Owner && Owner->GetWorld())
		{
			AudioComponent->DetachFromComponent(FDetachmentTransformRules::KeepRelativeTransform);
			AudioComponent->UnregisterComponent();
		}
		AudioComponent->DestroyComponent();
		AudioComponent = nullptr;
	}

	// Clear out the synth component's reference to the sound generator or it will leak until it gets GC'd
	// Normally this is ok to wait till GC but some derived synths might need for the handle to be released
	SoundGenerator.Reset();
}

void USynthComponent::EndPlay(const EEndPlayReason::Type Reason) 
{	
	Super::EndPlay(Reason);

	if (GetOwner() && (Reason == EEndPlayReason::LevelTransition || Reason == EEndPlayReason::RemovedFromWorld || Reason == EEndPlayReason::Destroyed))
	{
		// If our world or sublevel is going away, stop immediately to prevent the containing world/level from being leaked via hard references from the audio device.
		Stop();
	}
}

USoundClass* USynthComponent::GetSoundClass()
{
	if (SoundClass)
	{
		return SoundClass;
	}

	const UAudioSettings* AudioSettings = GetDefault<UAudioSettings>();
	if (ensure(AudioSettings))
	{
		return AudioSettings->GetDefaultSoundClass();
	}

	return nullptr;
}

bool USynthComponent::IsReadyForOwnerToAutoDestroy() const
{
	const bool bIsAudioComponentReadyForDestroy = !AudioComponent || (AudioComponent && !AudioComponent->IsPlaying());
	const bool bIsSynthSoundReadyForDestroy = !Synth || !Synth->IsGeneratingAudio();
	return bIsAudioComponentReadyForDestroy && bIsSynthSoundReadyForDestroy;
}

#if WITH_EDITOR
void USynthComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (IsActive())
	{
		// If this is an auto destroy component we need to prevent it from being auto-destroyed since we're really just restarting it
		const bool bWasAutoDestroy = bAutoDestroy;
		bAutoDestroy = false;
		Stop();
		bAutoDestroy = bWasAutoDestroy;
		Start();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif //WITH_EDITOR

#if WITH_EDITORONLY_DATA
void USynthComponent::PostLoad()
{
	Super::PostLoad();
	if (bOutputToBusOnly_DEPRECATED)
	{
		bEnableBusSends = true;
		bEnableBaseSubmix = !bOutputToBusOnly_DEPRECATED;
		bEnableSubmixSends = !bOutputToBusOnly_DEPRECATED;
		bOutputToBusOnly_DEPRECATED = false;
	}
}
#endif //WITH_EDITORONLY_DATA

void USynthComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

#if WITH_EDITORONLY_DATA
	if (Ar.IsLoading())
	{
		if (ConcurrencySettings_DEPRECATED != nullptr)
		{
			ConcurrencySet.Add(ConcurrencySettings_DEPRECATED);
			ConcurrencySettings_DEPRECATED = nullptr;
		}
	}
#endif // WITH_EDITORONLY_DATA

}

void USynthComponent::PumpPendingMessages()
{
	TFunction<void()> Command;
	while (CommandQueue.Dequeue(Command))
	{
		Command();
	}
}

FAudioDevice* USynthComponent::GetAudioDevice() const
{
	// If the synth component has a world, that means it was already registed with that world
	if (UWorld* World = GetWorld())
	{
		return World->AudioDeviceHandle.GetAudioDevice();
	}

	// Otherwise, retrieve the audio component's audio device (probably from it's owner)
	if (AudioComponent)
	{
		return AudioComponent->GetAudioDevice();
	}
	
	// No audio device
	return nullptr;
}

int32 USynthComponent::OnGeneratePCMAudio(float* GeneratedPCMData, int32 NumSamples)
{
	LLM_SCOPE(ELLMTag::AudioSynthesis);

	PumpPendingMessages();

	check(NumSamples > 0);

	// Only call into the synth if we're actually playing, otherwise, we'll write out zero's
	if (bIsSynthPlaying)
	{
		return OnGenerateAudio(GeneratedPCMData, NumSamples);
	}
	return NumSamples;
}

void USynthComponent::Start()
{
	// Only need to start if we're not already active
	if (IsActive())
	{
		return;
	}

	// We will also ensure that this synth was initialized before attempting to play.
	Initialize();

	// If there is no Synth USoundBase, we can't start. This can happen if start is called in a cook, a server, or
	// if the audio engine is set to "noaudio".
	// TODO: investigate if this should be handled elsewhere before this point
	if (!Synth)
	{
		return;
	}

	if (AudioComponent)
	{
		// Copy the attenuation and concurrency data from the synth component to the audio component
		AudioComponent->AttenuationSettings = AttenuationSettings;
		AudioComponent->bOverrideAttenuation = bOverrideAttenuation;
		AudioComponent->bIsUISound = bIsUISound;
		AudioComponent->bIsPreviewSound = bIsPreviewSound;
		AudioComponent->bAllowSpatialization = bAllowSpatialization;
		AudioComponent->ConcurrencySet = ConcurrencySet;
		AudioComponent->AttenuationOverrides = AttenuationOverrides;
		AudioComponent->SoundClassOverride = SoundClass;
		AudioComponent->EnvelopeFollowerAttackTime = EnvelopeFollowerAttackTime;
		AudioComponent->EnvelopeFollowerReleaseTime = EnvelopeFollowerReleaseTime;
		AudioComponent->ModulationRouting = ModulationRouting;

		// Copy sound base data to the sound
		Synth->AttenuationSettings = AttenuationSettings;
		Synth->SourceEffectChain = SourceEffectChain;
		Synth->SoundSubmixObject = SoundSubmix;
		Synth->SoundSubmixSends = SoundSubmixSends;

		// Set the audio component's sound to be our procedural sound wave
		AudioComponent->SetSound(Synth);
		AudioComponent->Play(0);

		SetActiveFlag(AudioComponent->IsActive());

		bIsSynthPlaying = true;
	}
}

void USynthComponent::Stop()
{
	if (IsActive())
	{
		if (AudioComponent)
		{
			AudioComponent->Stop();		
			
			if (FAudioDevice* AudioDevice = AudioComponent->GetAudioDevice())
			{
				AudioDevice->StopSoundsUsingResource(Synth);
			}
		}

		SetActiveFlag(false);
	}
}

bool USynthComponent::IsPlaying() const
{
	return AudioComponent && AudioComponent->IsPlaying();
}

void USynthComponent::SetVolumeMultiplier(float VolumeMultiplier)
{
	if (AudioComponent)
	{
		AudioComponent->SetVolumeMultiplier(VolumeMultiplier);
	}
}

void USynthComponent::SetSubmixSend(USoundSubmixBase* Submix, float SendLevel)
{
	if (AudioComponent)
	{
		AudioComponent->SetSubmixSend(Submix, SendLevel);
	}
}

void USynthComponent::SetSourceBusSendPreEffect(USoundSourceBus* SoundSourceBus, float SourceBusSendLevel)
{
	if (AudioComponent)
	{
		AudioComponent->SetSourceBusSendPreEffect(SoundSourceBus, SourceBusSendLevel);
	}
}

void USynthComponent::SetSourceBusSendPostEffect(USoundSourceBus* SoundSourceBus, float SourceBusSendLevel)
{
	if (AudioComponent)
	{
		AudioComponent->SetSourceBusSendPostEffect(SoundSourceBus, SourceBusSendLevel);
	}
}

void USynthComponent::SetAudioBusSendPreEffect(UAudioBus* AudioBus, float AudioBusSendLevel)
{
	if (AudioComponent)
	{
		AudioComponent->SetAudioBusSendPreEffect(AudioBus, AudioBusSendLevel);
	}
}

void USynthComponent::SetAudioBusSendPostEffect(UAudioBus* AudioBus, float AudioBusSendLevel)
{
	if (AudioComponent)
	{
		AudioComponent->SetAudioBusSendPostEffect(AudioBus, AudioBusSendLevel);
	}
}

void USynthComponent::SetLowPassFilterEnabled(bool InLowPassFilterEnabled)
{
	if (AudioComponent)
	{
		AudioComponent->SetLowPassFilterEnabled(InLowPassFilterEnabled);
	}
}

void USynthComponent::SetLowPassFilterFrequency(float InLowPassFilterFrequency)
{
	if (AudioComponent)
	{
		AudioComponent->SetLowPassFilterFrequency(InLowPassFilterFrequency);
	}
}

void USynthComponent::SetOutputToBusOnly(bool bInOutputToBusOnly)
{
	if (AudioComponent)
	{
		AudioComponent->SetOutputToBusOnly(bInOutputToBusOnly);
	}
}

void USynthComponent::FadeIn(float FadeInDuration, float FadeVolumeLevel/* = 1.0f*/, float StartTime/* = 0.0f*/, const EAudioFaderCurve FadeCurve/* = EAudioFaderCurve::Linear*/) const
{
	if(AudioComponent)
	{
		AudioComponent->FadeIn(FadeInDuration, FadeVolumeLevel, StartTime, FadeCurve);
	}
}

void USynthComponent::FadeOut(float FadeOutDuration, float FadeVolumeLevel, const EAudioFaderCurve FadeCurve/* = EAudioFaderCurve::Linear*/) const
{
	if(AudioComponent)
	{
		AudioComponent->FadeOut(FadeOutDuration, FadeVolumeLevel, FadeCurve);
	}
}

void USynthComponent::AdjustVolume(float AdjustVolumeDuration, float AdjustVolumeLevel, const EAudioFaderCurve FadeCurve/* = EAudioFaderCurve::Linear*/) const
{
	if(AudioComponent)
	{
		AudioComponent->AdjustVolume(AdjustVolumeDuration, AdjustVolumeLevel, FadeCurve);
	}
}

void USynthComponent::SetModulationRouting(const TSet<USoundModulatorBase*>& Modulators, const EModulationDestination Destination, const EModulationRouting RoutingMethod)
{
	if (AudioComponent)
	{
		AudioComponent->SetModulationRouting(Modulators, Destination, RoutingMethod);
	}
}

TSet<USoundModulatorBase*> USynthComponent::GetModulators(const EModulationDestination Destination)
{
	if (AudioComponent)
	{
		return AudioComponent->GetModulators(Destination);
	}

	return TSet<USoundModulatorBase*>();
}

void USynthComponent::SynthCommand(TFunction<void()> Command)
{
	if (SoundGenerator.IsValid())
	{
		UE_LOGF(LogAudioMixer, Error, "Synthesis component '%ls' has implemented a sound generator interface. Do not call SynthCommand on the USynthComponent).", *GetName());
	}
	else
	{
		CommandQueue.Enqueue(MoveTemp(Command));
	}
}

ISoundGeneratorPtr USynthComponent::CreateSoundGeneratorInternal(const FSoundGeneratorInitParams& InParams)
{
	LLM_SCOPE(ELLMTag::AudioSynthesis);	
	return SoundGenerator = CreateSoundGenerator(InParams);
}

