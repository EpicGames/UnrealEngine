// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubmixEffects/AudioMixerSubmixEffectDynamicsProcessor.h"

#include "AudioBusSubsystem.h"
#include "AudioDeviceManager.h"
#include "AudioMixerDevice.h"
#include "AudioMixerSubmix.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioMixerSubmixEffectDynamicsProcessor)

// Link to "Audio" profiling category
CSV_DECLARE_CATEGORY_MODULE_EXTERN(AUDIOMIXERCORE_API, Audio);

DEFINE_STAT(STAT_AudioMixerSubmixDynamics);

static int32 bBypassSubmixDynamicsProcessor = 0;
FAutoConsoleVariableRef CVarBypassDynamicsProcessor(
	TEXT("au.Submix.Effects.DynamicsProcessor.Bypass"),
	bBypassSubmixDynamicsProcessor,
	TEXT("If non-zero, bypasses all submix dynamics processors currently active.\n"),
	ECVF_Default);

FSubmixEffectDynamicsProcessor::FSubmixEffectDynamicsProcessor()
{
	DeviceCreatedHandle = FAudioDeviceManagerDelegates::OnAudioDeviceCreated.AddRaw(this, &FSubmixEffectDynamicsProcessor::OnDeviceCreated);
	DeviceDestroyedHandle = FAudioDeviceManagerDelegates::OnAudioDeviceDestroyed.AddRaw(this, &FSubmixEffectDynamicsProcessor::OnDeviceDestroyed);
}

FSubmixEffectDynamicsProcessor::~FSubmixEffectDynamicsProcessor()
{
	ResetKey();

	FAudioDeviceManagerDelegates::OnAudioDeviceCreated.Remove(DeviceCreatedHandle);
	FAudioDeviceManagerDelegates::OnAudioDeviceDestroyed.Remove(DeviceDestroyedHandle);
}

Audio::FDeviceId FSubmixEffectDynamicsProcessor::GetDeviceId() const
{
	return DeviceId;
}

void FSubmixEffectDynamicsProcessor::Init(const FSoundEffectSubmixInitData& InitData)
{
	static const int32 ProcessorScratchNumChannels = 8;

	DynamicsProcessor.Init(InitData.SampleRate, ProcessorScratchNumChannels);

	DeviceId = InitData.DeviceID;

	if (USubmixEffectDynamicsProcessorPreset* ProcPreset = Cast<USubmixEffectDynamicsProcessorPreset>(Preset.Get()))
	{
		switch (ProcPreset->Settings.KeySource)
		{
			case ESubmixEffectDynamicsKeySource::AudioBus:
			{
				if (UAudioBus* AudioBus = ProcPreset->Settings.ExternalAudioBus)
				{
					KeySource.Update(ESubmixEffectDynamicsKeySource::AudioBus, AudioBus->GetUniqueID(), static_cast<int32>(AudioBus->AudioBusChannels) + 1);
				}
			}
			break;

			case ESubmixEffectDynamicsKeySource::Submix:
			{
				if (USoundSubmix* Submix = ProcPreset->Settings.ExternalSubmix)
				{
					KeySource.Update(ESubmixEffectDynamicsKeySource::Submix, Submix->GetUniqueID());
				}
			}
			break;

			default:
			{
				// KeySource is this effect's submix/input, so do nothing
			}
			break;
		}
	}
}

void FSubmixEffectDynamicsProcessor::ResetKey()
{
	KeySource.Reset();
}

void FSubmixEffectDynamicsProcessor::OnPresetChanged()
{
	GET_EFFECT_SETTINGS(SubmixEffectDynamicsProcessor);

	bBypass = Settings.bBypass;

	switch (Settings.DynamicsProcessorType)
	{
	default:
	case ESubmixEffectDynamicsProcessorType::Compressor:
		DynamicsProcessor.SetProcessingMode(Audio::EDynamicsProcessingMode::Compressor);
		break;

	case ESubmixEffectDynamicsProcessorType::Limiter:
		DynamicsProcessor.SetProcessingMode(Audio::EDynamicsProcessingMode::Limiter);
		break;

	case ESubmixEffectDynamicsProcessorType::Expander:
		DynamicsProcessor.SetProcessingMode(Audio::EDynamicsProcessingMode::Expander);
		break;

	case ESubmixEffectDynamicsProcessorType::Gate:
		DynamicsProcessor.SetProcessingMode(Audio::EDynamicsProcessingMode::Gate);
		break;

	case ESubmixEffectDynamicsProcessorType::UpwardsCompressor:
		DynamicsProcessor.SetProcessingMode(Audio::EDynamicsProcessingMode::UpwardsCompressor);
		break;
	}

	switch (Settings.PeakMode)
	{
	default:
	case ESubmixEffectDynamicsPeakMode::MeanSquared:
		DynamicsProcessor.SetPeakMode(Audio::EPeakMode::MeanSquared);
		break;

	case ESubmixEffectDynamicsPeakMode::RootMeanSquared:
		DynamicsProcessor.SetPeakMode(Audio::EPeakMode::RootMeanSquared);
		break;

	case ESubmixEffectDynamicsPeakMode::Peak:
		DynamicsProcessor.SetPeakMode(Audio::EPeakMode::Peak);
		break;
	}

	DynamicsProcessor.SetLookaheadMsec(Settings.LookAheadMsec);
	DynamicsProcessor.SetAttackTime(Settings.AttackTimeMsec);
	DynamicsProcessor.SetReleaseTime(Settings.ReleaseTimeMsec);
	DynamicsProcessor.SetThreshold(Settings.ThresholdDb);
	DynamicsProcessor.SetRatio(Settings.Ratio);
	DynamicsProcessor.SetKneeBandwidth(Settings.KneeBandwidthDb);
	DynamicsProcessor.SetInputGain(Settings.InputGainDb);
	DynamicsProcessor.SetOutputGain(Settings.OutputGainDb);
	DynamicsProcessor.SetAnalogMode(Settings.bAnalogMode);

	DynamicsProcessor.SetKeyAudition(Settings.bKeyAudition);
	DynamicsProcessor.SetKeyGain(Settings.KeyGainDb);
	DynamicsProcessor.SetKeyHighshelfCutoffFrequency(Settings.KeyHighshelf.Cutoff);
	DynamicsProcessor.SetKeyHighshelfEnabled(Settings.KeyHighshelf.bEnabled);
	DynamicsProcessor.SetKeyHighshelfGain(Settings.KeyHighshelf.GainDb);
	DynamicsProcessor.SetKeyLowshelfCutoffFrequency(Settings.KeyLowshelf.Cutoff);
	DynamicsProcessor.SetKeyLowshelfEnabled(Settings.KeyLowshelf.bEnabled);
	DynamicsProcessor.SetKeyLowshelfGain(Settings.KeyLowshelf.GainDb);

	static_assert(static_cast<int32>(ESubmixEffectDynamicsChannelLinkMode::Count) == static_cast<int32>(Audio::EDynamicsProcessorChannelLinkMode::Count), "Enumerations must match");
	DynamicsProcessor.SetChannelLinkMode(static_cast<Audio::EDynamicsProcessorChannelLinkMode>(Settings.LinkMode));

	UpdateKeyFromSettings(Settings);
}

Audio::FMixerDevice* FSubmixEffectDynamicsProcessor::GetMixerDevice()
{
	if (FAudioDeviceManager* DeviceManager = FAudioDeviceManager::Get())
	{
		return static_cast<Audio::FMixerDevice*>(DeviceManager->GetAudioDeviceRaw(DeviceId));
	}

	return nullptr;
}

bool FSubmixEffectDynamicsProcessor::UpdateKeySourcePatch()
{
	using namespace Audio;
	
	// Default (input as key) does not use source patch, so don't
	// continue checking or updating state.
	if (KeySource.GetType() == ESubmixEffectDynamicsKeySource::Default)
	{
		return false;
	}

	// Valid object
	const uint32 ObjectId = KeySource.GetObjectId();
	if (ObjectId == INDEX_NONE)
	{
		return false;
	}
	
	// Retrieving/mutating the MixerDevice is only safe during OnProcessAudio calls if
	// it is not called during Teardown.  The DynamicsProcessor should be Reset via
	// the OnDeviceDestroyed callback (prior to FAudioDevice::Teardown), so this call
	// should never be hit during Teardown.
	FMixerDevice* MixerDevice = GetMixerDevice();
	if (!MixerDevice)
	{
		return false;
	}

	// Need to determine if our current patch is still valid.
	// This is handled differently per-type. We return true for valid.
	// If it's not-valid, we must try and (re)create a patch
	
	switch (KeySource.GetType())
	{
		case ESubmixEffectDynamicsKeySource::AudioBus:
		{
			if (UAudioBusSubsystem* AudioBusSubsystem = MixerDevice->GetSubsystem<UAudioBusSubsystem>())
			{
				// Check patch is still valid and not stale.
				// AudioBusses can't change format under us (yet), so just check its validity.
				if (KeySource.Patch.IsValid() && !KeySource.Patch->IsInputStale())
				{
					return true;
				}

				const int32 NumFrames = MixerDevice->GetNumOutputFrames();
				const int32 NumChannels = KeySource.GetNumChannels();
				const FAudioBusKey BusKey(ObjectId);

				// Make sure the bus is started. (if it's started already this does nothing).
				const FString SubmixEffectDynamicsKeySourceAudioBusName = FString::Format(TEXT("_SubmixEffectDynamicsKeySourceId_{0}"), { ObjectId });
				AudioBusSubsystem->StartAudioBus(BusKey, SubmixEffectDynamicsKeySourceAudioBusName, NumChannels, /*bInIsAutomatic=*/false);

				// Add/Recreate a new patch to this bus
				KeySource.Patch = AudioBusSubsystem->AddPatchOutputForAudioBus(BusKey, NumFrames, NumChannels);

				// Always set the keyed channels even if we fail to create the patch.
				// As the nop case requires these to be set.
				DynamicsProcessor.SetKeyNumChannels(NumChannels);

				// If the patch is created and got valid inputs (non-stale) we consider it valid.
				return KeySource.Patch.IsValid() && !KeySource.Patch->IsInputStale();				
			}
		}
		break;

		case ESubmixEffectDynamicsKeySource::Submix:
		{
			if (const FMixerSubmixPtr SubmixPtr = MixerDevice->FindSubmixInstanceByObjectId(ObjectId); SubmixPtr.IsValid())
			{
				const int32 NumSubmixChannels = SubmixPtr->GetNumOutputChannels();					

				// Is patch still valid? Check that we haven't changed channel count.
				// This can happen after we've swapped devices.
				if (KeySource.Patch.IsValid() && !KeySource.Patch->IsInputStale() && KeySource.GetNumChannels() == NumSubmixChannels)
				{
					// Looks good.
					return true;
				}

				// We store the num channels the submix key source is running at.
				KeySource.SetNumChannels(NumSubmixChannels);
				
				// Add/Recreate a new patch to this submix.
				KeySource.Patch = MixerDevice->AddPatchForSubmix(ObjectId, 1.0f /* PatchGain */);

				// Always set the keyed channels even if we fail to create the patch.
				// As the nop case requires these to be set.
				DynamicsProcessor.SetKeyNumChannels(NumSubmixChannels);

				// If the patch is created and got valid inputs (non-stale) we consider it valid.
				return KeySource.Patch.IsValid() && !KeySource.Patch->IsInputStale();				
			}
		}
		break;

		case ESubmixEffectDynamicsKeySource::Default:
		default:
		{
			checkNoEntry();
		}
		break;
	}
	
	// KeySource invalid.
	return false;
}

void FSubmixEffectDynamicsProcessor::OnProcessAudio(const FSoundEffectSubmixInputData& InData, FSoundEffectSubmixOutputData& OutData)
{
	CSV_SCOPED_TIMING_STAT(Audio, SubmixDynamics);
	SCOPE_CYCLE_COUNTER(STAT_AudioMixerSubmixDynamics);
	TRACE_CPUPROFILER_EVENT_SCOPE(FSubmixEffectDynamicsProcessor::OnProcessAudio);

	ensure(InData.NumChannels == OutData.NumChannels);

	const Audio::FAlignedFloatBuffer& InBuffer = *InData.AudioBuffer;
	Audio::FAlignedFloatBuffer& OutBuffer = *OutData.AudioBuffer;

	if (bBypassSubmixDynamicsProcessor || bBypass)
	{
		FMemory::Memcpy(OutBuffer.GetData(), InBuffer.GetData(), sizeof(float) * InBuffer.Num());
		return;
	}

	AudioExternal.Reset();

	// Update the output channels if necessary.
	if (InData.NumChannels != DynamicsProcessor.GetNumChannels())
	{
		DynamicsProcessor.SetNumChannels(InData.NumChannels);
	}
		
	// If set to default, enforce num key channels to always be number of input channels.
	// If either unset or KeySource was changed between frames back to 'Default', NumKeyChannels
	// could be stale or left as initialized number of scratch channels.
	if (KeySource.GetType() == ESubmixEffectDynamicsKeySource::Default)
	{
		if (InData.NumChannels != DynamicsProcessor.GetKeyNumChannels())
		{
			DynamicsProcessor.SetKeyNumChannels(InData.NumChannels);
		}
	}	
	else 	
	{
		// Update our key source. (this validates the existing patch and create new one if necessary)
		// NOTE: This call can adjust the DynamicsProcessor.NumKeyChannels, so query that after updating.
		const bool bPatchIsValid = UpdateKeySourcePatch(); 
		if (!bPatchIsValid)
		{
			// If the patch was not created, we emulate a patch by pumping in 
			// silence with an equal number of channels as the input audio.
			// The key channel count needs to get updated here as it may not have
			// been updated to a valid value inside of `UpdateKeySourcePatch()`
			if (InData.NumChannels != DynamicsProcessor.GetKeyNumChannels())
			{
				DynamicsProcessor.SetKeyNumChannels(InData.NumChannels);
			}
		}

		const int32 NumKeyChannels = DynamicsProcessor.GetKeyNumChannels();
		const int32 NumKeySamples = InData.NumFrames * NumKeyChannels;		
		
		// Make enough space to handle all input samples that we're expecting.
		// These zeros will act as input if we don't get enough samples.
		AudioExternal.AddZeroed(NumKeySamples);	

		if (bPatchIsValid)
		{
			// There's a chance of a patch racing with us here and becoming stale, which would return INDEX_NONE (-1) samples here.
			// This shouldn't matter, but we should be careful of ensuring against >= 0 samples popped.
			const int32 NumSamplesPopped = KeySource.Patch->PopAudio(AudioExternal.GetData(), AudioExternal.Num(), true /* bUseLatestAudio */);
			ensure(NumSamplesPopped <= AudioExternal.Num());
		}
	}

	// No key assigned (Uses input buffer as key)
	if (KeySource.GetType() == ESubmixEffectDynamicsKeySource::Default)
	{
		DynamicsProcessor.ProcessAudio(InBuffer.GetData(), InData.NumChannels * InData.NumFrames, OutBuffer.GetData());
	}
	// Key assigned
	else
	{
		DynamicsProcessor.ProcessAudio(InBuffer.GetData(), InData.NumChannels * InData.NumFrames, OutBuffer.GetData(), AudioExternal.GetData());
	}
}

void FSubmixEffectDynamicsProcessor::UpdateKeyFromSettings(const FSubmixEffectDynamicsProcessorSettings& InSettings)
{
	uint32 ObjectId = INDEX_NONE;
	int32 SourceNumChannels = 0;
	switch (InSettings.KeySource)
	{
		case ESubmixEffectDynamicsKeySource::AudioBus:
		{
			if (InSettings.ExternalAudioBus)
			{
				ObjectId = InSettings.ExternalAudioBus->GetUniqueID();
				SourceNumChannels = static_cast<int32>(InSettings.ExternalAudioBus->AudioBusChannels) + 1;
			}
		}
		break;

		case ESubmixEffectDynamicsKeySource::Submix:
		{
			if (InSettings.ExternalSubmix)
			{
				ObjectId = InSettings.ExternalSubmix->GetUniqueID();
			}
		}
		break;

		default:
		{
		}
		break;
	}

	KeySource.Update(InSettings.KeySource, ObjectId, SourceNumChannels);
}

void FSubmixEffectDynamicsProcessor::OnDeviceCreated(Audio::FDeviceId InDeviceId)
{
	if (InDeviceId == DeviceId)
	{
		GET_EFFECT_SETTINGS(SubmixEffectDynamicsProcessor);
		UpdateKeyFromSettings(Settings);

		FAudioDeviceManagerDelegates::OnAudioDeviceCreated.Remove(DeviceCreatedHandle);
	}
}

void FSubmixEffectDynamicsProcessor::OnDeviceDestroyed(Audio::FDeviceId InDeviceId)
{
	if (InDeviceId == DeviceId)
	{
		// Reset the key on device destruction to avoid reinitializing
		// it during FAudioDevice::Teardown via ProcessAudio.
		ResetKey();
		FAudioDeviceManagerDelegates::OnAudioDeviceDestroyed.Remove(DeviceDestroyedHandle);
	}
}

void USubmixEffectDynamicsProcessorPreset::OnInit()
{
	switch (Settings.KeySource)
	{
		case ESubmixEffectDynamicsKeySource::AudioBus:
		{
			SetAudioBus(Settings.ExternalAudioBus);
		}
		break;

		case ESubmixEffectDynamicsKeySource::Submix:
		{
			SetExternalSubmix(Settings.ExternalSubmix);
		}
		break;

		default:
		{
		}
		break;
	}
}

#if WITH_EDITOR
void USubmixEffectDynamicsProcessorPreset::PostEditChangeChainProperty(struct FPropertyChangedChainEvent& InChainEvent)
{
	if (InChainEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FSubmixEffectDynamicsProcessorSettings, KeySource))
	{
		switch (Settings.KeySource)
		{
		case ESubmixEffectDynamicsKeySource::AudioBus:
		{
			Settings.ExternalSubmix = nullptr;
		}
		break;

		case ESubmixEffectDynamicsKeySource::Submix:
		{
			Settings.ExternalAudioBus = nullptr;
		}
		break;

		case ESubmixEffectDynamicsKeySource::Default:
		default:
		{
			Settings.ExternalSubmix = nullptr;
			Settings.ExternalAudioBus = nullptr;
			static_assert(static_cast<int32>(ESubmixEffectDynamicsKeySource::Count) == 3, "Possible missing KeySource switch case coverage");
		}
		break;
		}
	}

	Super::PostEditChangeChainProperty(InChainEvent);
}
#endif // WITH_EDITOR

void USubmixEffectDynamicsProcessorPreset::Serialize(FStructuredArchive::FRecord Record)
{
	FArchive& UnderlyingArchive = Record.GetUnderlyingArchive();
	if (UnderlyingArchive.IsLoading())
	{
		if (Settings.bChannelLinked_DEPRECATED)
		{
			Settings.LinkMode = ESubmixEffectDynamicsChannelLinkMode::Average;
			Settings.bChannelLinked_DEPRECATED = 0;
		}
	}

	Super::Serialize(Record);
}

void USubmixEffectDynamicsProcessorPreset::ResetKey()
{
	EffectCommand<FSubmixEffectDynamicsProcessor>([](FSubmixEffectDynamicsProcessor& Instance)
	{
		Instance.ResetKey();
	});
}

void USubmixEffectDynamicsProcessorPreset::SetAudioBus(UAudioBus* InAudioBus)
{
	int32 BusChannels = 0;
	if (InAudioBus)
	{
		BusChannels = static_cast<int32>(InAudioBus->AudioBusChannels) + 1;
		SetKey(ESubmixEffectDynamicsKeySource::AudioBus, InAudioBus, BusChannels);
	}
	else
	{
		ResetKey();
	}
}

void USubmixEffectDynamicsProcessorPreset::SetExternalSubmix(USoundSubmix* InSubmix)
{
	if (InSubmix)
	{
		SetKey(ESubmixEffectDynamicsKeySource::Submix, InSubmix);
	}
	else
	{
		ResetKey();
	}
}

void USubmixEffectDynamicsProcessorPreset::SetKey(ESubmixEffectDynamicsKeySource InKeySource, UObject* InObject, int32 InNumChannels)
{
	if (InObject)
	{
		EffectCommand<FSubmixEffectDynamicsProcessor>([this, ObjectId = InObject->GetUniqueID(), InKeySource, InNumChannels](FSubmixEffectDynamicsProcessor& Instance)
		{
			Instance.KeySource.Update(InKeySource, ObjectId, InNumChannels);
		});
	}
}

void USubmixEffectDynamicsProcessorPreset::SetSettings(const FSubmixEffectDynamicsProcessorSettings& InSettings)
{
	UpdateSettings(InSettings);

	IterateEffects<FSubmixEffectDynamicsProcessor>([&](FSubmixEffectDynamicsProcessor& Instance)
	{
		Instance.UpdateKeyFromSettings(InSettings);
	});
}

