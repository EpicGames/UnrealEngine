// Copyright Epic Games, Inc. All Rights Reserved.


#include "SubmixEffects/AudioMixerSubmixEffectEQ.h"
#include "Misc/ScopeLock.h"
#include "AudioMixer.h"
#include "ProfilingDebugging/CsvProfiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioMixerSubmixEffectEQ)

// Link to "Audio" profiling category
CSV_DECLARE_CATEGORY_MODULE_EXTERN(AUDIOMIXERCORE_API, Audio);

DEFINE_STAT(STAT_AudioMixerSubmixEQ);

static bool IsEqual(const FSubmixEffectSubmixEQSettings& Left, const FSubmixEffectSubmixEQSettings& Right)
{
	// return false if the number of bands changed
	if (Left.EQBands.Num() != Right.EQBands.Num())
	{
		return false;
	}


	for (int32 i = 0; i < Right.EQBands.Num(); ++i)
	{
		const FSubmixEffectEQBand& OtherBand = Right.EQBands[i];
		const FSubmixEffectEQBand& ThisBand = Left.EQBands[i];

		if (OtherBand.bEnabled != ThisBand.bEnabled)
		{
			return false;
		}

		if (!FMath::IsNearlyEqual(OtherBand.Bandwidth, ThisBand.Bandwidth))
		{
			return false;
		}

		if (!FMath::IsNearlyEqual(OtherBand.Frequency, ThisBand.Frequency))
		{
			return false;
		}

		if (!FMath::IsNearlyEqual(OtherBand.GainDb, ThisBand.GainDb))
		{
			return false;
		}
	}

	// If we made it this far these are equal
	return true;
}


FSubmixEffectSubmixEQ::FSubmixEffectSubmixEQ()
	: SampleRate(0)
	, NumOutputChannels(2)
{
	FMemory::Memzero((void*)ScratchInBuffer, sizeof(float) * 2);
	FMemory::Memzero((void*)ScratchOutBuffer, sizeof(float) * 2);
}

void FSubmixEffectSubmixEQ::Init(const FSoundEffectSubmixInitData& InitData)
{
	SampleRate = InitData.SampleRate;

	// Assume 8 channels (max supported channels)
	NumOutputChannels = 8;

	const int32 NumFilters = NumOutputChannels / 2;
	for (int32 i = 0; i < NumFilters; ++i)
	{
		int32 Index = FiltersPerChannel.Add(FEQ());
	}

	bEQSettingsSet = false;
}

// Called when an audio effect preset is changed
void FSubmixEffectSubmixEQ::OnPresetChanged()
{
	LLM_SCOPE(ELLMTag::AudioMixer);

	GET_EFFECT_SETTINGS(SubmixEffectSubmixEQ);

	// Don't make any changes if this is the exact same parameters
	if (!IsEqual(GameThreadEQSettings, Settings))
	{
		GameThreadEQSettings = Settings;
		PendingSettings.SetParams(GameThreadEQSettings);
	}
}

void FSubmixEffectSubmixEQ::OnProcessAudio(const FSoundEffectSubmixInputData& InData, FSoundEffectSubmixOutputData& OutData)
{
	LLM_SCOPE(ELLMTag::AudioMixer);

	CSV_SCOPED_TIMING_STAT(Audio, SubmixEQ);
	SCOPE_CYCLE_COUNTER(STAT_AudioMixerSubmixEQ);

	// Update parameters that may have been set from game thread
	UpdateParameters(InData.NumChannels);

	Audio::FAlignedFloatBuffer& InAudioBuffer = *InData.AudioBuffer;
	Audio::FAlignedFloatBuffer& OutAudioBuffer = *OutData.AudioBuffer;

	if (bEQSettingsSet && RenderThreadEQSettings.EQBands.Num() > 0)
	{
		// Feed every other channel through the EQ filters
		int32 NumFilters = InData.NumChannels / 2;
		for (int32 FilterIndex = 0; FilterIndex < NumFilters; ++FilterIndex)
		{
			FEQ& EQFilter = FiltersPerChannel[FilterIndex];
			const int32 ChannelOffset = FilterIndex * 2;
			for (int32 FrameIndex = 0; FrameIndex < InData.NumFrames; ++FrameIndex)
			{
				// Get the sample index of this frame for this filter
				const int32 SampleIndex = FrameIndex * InData.NumChannels + ChannelOffset;

				// Copy the audio from the input buffer for this frame
				ScratchInBuffer[0] = InAudioBuffer[SampleIndex];
				ScratchInBuffer[1] = InAudioBuffer[SampleIndex + 1];

				const int32 NumBands = EQFilter.Bands.Num();
				for (int32 BandIndex = 0; BandIndex < NumBands; ++BandIndex)
				{
					EQFilter.Bands[BandIndex].ProcessAudioFrame(ScratchInBuffer, ScratchOutBuffer);

					// Copy the output of this band into the input for sequential processing
					for (int32 Channel = 0; Channel < 2; ++Channel)
					{
						ScratchInBuffer[Channel] = ScratchOutBuffer[Channel];
					}
				}

				// Copy the results of this frame to the output buffer
				OutAudioBuffer[SampleIndex] = ScratchOutBuffer[0];
				OutAudioBuffer[SampleIndex + 1] = ScratchOutBuffer[1];
			}
		}
	}
	else
	{
		// pass through 
		for (int32 i = 0; i < InAudioBuffer.Num(); ++i)
		{
			OutAudioBuffer[i] = InAudioBuffer[i];
		}
		
	}

}

static float GetClampedGain(const float InGain)
{
	// These are clamped to match XAudio2 FXEQ_MIN_GAIN and FXEQ_MAX_GAIN
	return FMath::Clamp(InGain, 0.001f, 7.94f);
}

static float GetClampedBandwidth(const float InBandwidth)
{
	// These are clamped to match XAudio2 FXEQ_MIN_BANDWIDTH and FXEQ_MAX_BANDWIDTH
	return FMath::Clamp(InBandwidth, 0.1f, 2.0f);
}

static float GetClampedFrequency(const float InFrequency)
{
	// These are clamped to match XAudio2 FXEQ_MIN_FREQUENCY_CENTER and FXEQ_MAX_FREQUENCY_CENTER
	return FMath::Clamp(InFrequency, 20.0f, 20000.0f);
}

bool FSubmixEffectSubmixEQ::SetParameters(const FAudioEffectParameters& InParameters)
{
	const FAudioEQEffect& EQEffectParameters = static_cast<const FAudioEQEffect&>(InParameters);

	// This function maps the old audio engine eq effect params to the new eq effect.
	// Note that this is always a 4-band EQ and not flexible w/ respect.
	FSubmixEffectSubmixEQSettings NewSettings;
	FSubmixEffectEQBand Band;

	Band.bEnabled = true;
	Band.Frequency = GetClampedFrequency(EQEffectParameters.FrequencyCenter0);
	Band.Bandwidth = GetClampedBandwidth(EQEffectParameters.Bandwidth0);
	Band.GainDb = Audio::ConvertToDecibels(GetClampedGain(EQEffectParameters.Gain0));
	NewSettings.EQBands.Add(Band);

	Band.bEnabled = true;
	Band.Frequency = GetClampedFrequency(EQEffectParameters.FrequencyCenter1);
	Band.Bandwidth = GetClampedBandwidth(EQEffectParameters.Bandwidth1);
	Band.GainDb = Audio::ConvertToDecibels(GetClampedGain(EQEffectParameters.Gain1));
	NewSettings.EQBands.Add(Band);

	Band.bEnabled = true;
	Band.Frequency = GetClampedFrequency(EQEffectParameters.FrequencyCenter2);
	Band.Bandwidth = GetClampedBandwidth(EQEffectParameters.Bandwidth2);
	Band.GainDb = Audio::ConvertToDecibels(GetClampedGain(EQEffectParameters.Gain2));
	NewSettings.EQBands.Add(Band);

	Band.bEnabled = true;
	Band.Frequency = GetClampedFrequency(EQEffectParameters.FrequencyCenter3);
	Band.Bandwidth = GetClampedBandwidth(EQEffectParameters.Bandwidth3);
	Band.GainDb = Audio::ConvertToDecibels(GetClampedGain(EQEffectParameters.Gain3));
	NewSettings.EQBands.Add(Band);

	// Don't make any changes if this is the exact same parameters
	if (!IsEqual(GameThreadEQSettings, NewSettings))
	{
		GameThreadEQSettings = NewSettings;
		PendingSettings.SetParams(GameThreadEQSettings);
	}

	return true;
}

void FSubmixEffectSubmixEQ::UpdateParameters(const int32 InNumOutputChannels)
{
	// We need to update parameters if the output channel count changed
	bool bParamsChanged = false;

	// Also need to update if new settings have been applied
	FSubmixEffectSubmixEQSettings NewSettings;
	if (PendingSettings.GetParams(&NewSettings))
	{
		bParamsChanged = true;

		// Make sure we clamp our freq and bandwidth to reasonable values here
		for (FSubmixEffectEQBand& Band : NewSettings.EQBands)
		{
			Band.Frequency = GetClampedFrequency(Band.Frequency);
			Band.Bandwidth = GetClampedBandwidth(Band.Bandwidth);
		}

		RenderThreadEQSettings = NewSettings;
	}

	if (bParamsChanged || !bEQSettingsSet)
	{
		bEQSettingsSet = true;

		const int32 NumBandsInSetting = RenderThreadEQSettings.EQBands.Num();
		const int32 CurrentFilterCount = FiltersPerChannel.Num();

		// Now loop through all the bands and set them up on all the filters.
		for (int32 FilterIndex = 0; FilterIndex < CurrentFilterCount; ++FilterIndex)
		{
			FEQ& EqFilter = FiltersPerChannel[FilterIndex];
			EqFilter.bEnabled = true;

			// Create more bands as needed
			const int32 NumCurrentBands = EqFilter.Bands.Num();
			if (NumCurrentBands < NumBandsInSetting)
			{
				// Create and initialize the biquad filters per band
				for (int32 BandIndex = NumCurrentBands; BandIndex < NumBandsInSetting; ++BandIndex)
				{
					// Create new filter instance
					int32 BiquadIndex = EqFilter.Bands.Add(Audio::FBiquadFilter());

					// Initialize it
					EqFilter.Bands[BiquadIndex].Init(SampleRate, 2, Audio::EBiquadFilter::ParametricEQ);
				}
			}
			// Disable bands as needed
			else if (NumCurrentBands > NumBandsInSetting)
			{
				// Disable all filters that are greater than the number of bands in the new setting
				for (int32 BandIndex = NumBandsInSetting; BandIndex < NumCurrentBands; ++BandIndex)
				{
					EqFilter.Bands[BandIndex].SetEnabled(false);
				}
			}

			// Now copy the settings over to the specific EQ filter
			check(NumBandsInSetting <= EqFilter.Bands.Num());
			for (int32 BandIndex = 0; BandIndex < NumBandsInSetting; ++BandIndex)
			{
				const FSubmixEffectEQBand& EQBandSetting = RenderThreadEQSettings.EQBands[BandIndex];
				EqFilter.Bands[BandIndex].SetEnabled(EQBandSetting.bEnabled);
				EqFilter.Bands[BandIndex].SetParams(Audio::EBiquadFilter::ParametricEQ, EQBandSetting.Frequency, EQBandSetting.Bandwidth, EQBandSetting.GainDb);
	   		}
		}
	}
}

void USubmixEffectSubmixEQPreset::SetSettings(const FSubmixEffectSubmixEQSettings& InSettings)
{
	UpdateSettings(InSettings);
}

