// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubmixEffects/AudioMixerSubmixEffectReverb.h"
#include "AudioMixerEffectsManager.h"
#include "HAL/IConsoleManager.h"
#include "Sound/ReverbEffect.h"
#include "Audio.h"
#include "AudioMixer.h"
#include "DSP/BufferVectorOperations.h"
#include "DSP/FloatArrayMath.h"
#include "DSP/ReverbFast.h"
#include "DSP/Amp.h"
#include "ProfilingDebugging/CsvProfiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioMixerSubmixEffectReverb)

// Link to "Audio" profiling category
CSV_DECLARE_CATEGORY_MODULE_EXTERN(AUDIOMIXERCORE_API, Audio);

DEFINE_STAT(STAT_AudioMixerSubmixReverb);

static int32 DisableSubmixReverbCVarFast = 0;
static FAutoConsoleVariableRef CVarDisableSubmixReverb(
	TEXT("au.DisableReverbSubmix"),
	DisableSubmixReverbCVarFast,
	TEXT("Disables the reverb submix.\n")
	TEXT("0: Not Disabled, 1: Disabled"),
	ECVF_Default);


static int32 EnableReverbStereoFlipForQuadCVarFast = 0;
static FAutoConsoleVariableRef CVarReverbStereoFlipForQuadFast(
	TEXT("au.EnableReverbStereoFlipForQuad"),
	EnableReverbStereoFlipForQuadCVarFast,
	TEXT("Enables doing a stereo flip for quad reverb when in surround.\n")
	TEXT("0: Not Enabled, 1: Enabled"),
	ECVF_Default);

static int32 DisableQuadReverbCVarFast = 0;
static FAutoConsoleVariableRef CVarDisableQuadReverbCVarFast(
	TEXT("au.DisableQuadReverb"),
	DisableQuadReverbCVarFast,
	TEXT("Disables quad reverb in surround.\n")
	TEXT("0: Not Disabled, 1: Disabled"),
	ECVF_Default);


const float FSubmixEffectReverb::MinWetness = 0.0f;
const float FSubmixEffectReverb::MaxWetness = 10.f;

FSubmixEffectReverb::FSubmixEffectReverb()
	: CurrentWetDry(-1.0f, -1.0f)
{
}

void FSubmixEffectReverb::Init(const FSoundEffectSubmixInitData& InitData)
{
	LLM_SCOPE(ELLMTag::AudioMixer);

	/* `FPlateReverb` produces slightly different quality effect than `FPlateReverb`. Comparing the Init
	 * settings between FSubmixEffectReverb and FSubmixEffectReverb slight differences will arise.
	 *
	 * The delay line implementations significantly differ between the `FPlateReverb` and `FPlateReverb` classes.
	 * Specifically, the `FPlateReverb` class utilizes linearly interpolated fractional delay line and fractional
	 * delays while the `FPlateReverb` class uses integer delay lines and integer delays whenever possible.
	 * Linearly interpolated fractional delay lines introduce a low pass filter dependent upon the fractional portion
	 * of the delay value. As a result, the `FPlateReverb` class produces a darker reverb.
	 */
	Audio::FPlateReverbFastSettings NewSettings;

	SampleRate = InitData.SampleRate;

	NewSettings.EarlyReflections.Decay = 0.9f;
	NewSettings.EarlyReflections.Absorption = 0.7f;
	NewSettings.EarlyReflections.Gain = 1.0f;
	NewSettings.EarlyReflections.PreDelayMsec = 0.0f;
	NewSettings.EarlyReflections.Bandwidth = 0.8f;

	NewSettings.LateReflections.LateDelayMsec = 0.0f;
	NewSettings.LateReflections.LateGainDB = 0.0f;
	NewSettings.LateReflections.Bandwidth = 0.54f;
	NewSettings.LateReflections.Diffusion = 0.60f;
	NewSettings.LateReflections.Dampening = 0.35f;
	NewSettings.LateReflections.Decay = 0.15f;
	NewSettings.LateReflections.Density = 0.85f;

	ReverbParams.SetParams(NewSettings);

	DecayCurve.AddKey(0.0f, 0.99f);
	DecayCurve.AddKey(2.0f, 0.45f);
	DecayCurve.AddKey(5.0f, 0.15f);
	DecayCurve.AddKey(10.0f, 0.1f);
	DecayCurve.AddKey(18.0f, 0.01f);
	DecayCurve.AddKey(19.0f, 0.002f);
	DecayCurve.AddKey(20.0f, 0.0001f);

	if (DisableSubmixReverbCVarFast == 0)
	{
		PlateReverb = MakeUnique<Audio::FPlateReverbFast>(SampleRate, 512, NewSettings);
	}
}

void FSubmixEffectReverb::OnPresetChanged()
{
	LLM_SCOPE(ELLMTag::AudioMixer);

	GET_EFFECT_SETTINGS(SubmixEffectReverb);

	FAudioReverbEffect ReverbEffect;
	ReverbEffect.bBypassEarlyReflections = Settings.bBypassEarlyReflections;
	ReverbEffect.bBypassLateReflections = Settings.bBypassLateReflections;
	ReverbEffect.Density = Settings.Density;
	ReverbEffect.Diffusion = Settings.Diffusion;
	ReverbEffect.Gain = Settings.Gain;
	ReverbEffect.GainHF = Settings.GainHF;
	ReverbEffect.DecayTime = Settings.DecayTime;
	ReverbEffect.DecayHFRatio = Settings.DecayHFRatio;
	ReverbEffect.ReflectionsGain = Settings.ReflectionsGain;
	ReverbEffect.ReflectionsDelay = Settings.ReflectionsDelay;
	ReverbEffect.LateGain = Settings.LateGain;
	ReverbEffect.LateDelay = Settings.LateDelay;
	ReverbEffect.AirAbsorptionGainHF = Settings.AirAbsorptionGainHF;

	ReverbEffect.Volume = Settings.bBypass ? 0.0f : FMath::Clamp(Settings.WetLevel, MinWetness, MaxWetness);

	SetParameters(ReverbEffect);

	// These wet dry parameters need to be set after the call to SetParameters(ReverbEffect) parameter.
	// SetParameters sets the WetDryParams, but they need to be overriden here. 
	Audio::FWetDry NewWetDryParams;
	NewWetDryParams.DryLevel = Settings.bBypass ? 1.0f : Settings.DryLevel;
	NewWetDryParams.WetLevel = Settings.bBypass ? 0.0f : FMath::Clamp(Settings.WetLevel, MinWetness, MaxWetness);

	WetDryParams.SetParams(NewWetDryParams);
}

void FSubmixEffectReverb::OnProcessAudio(const FSoundEffectSubmixInputData& InData, FSoundEffectSubmixOutputData& OutData)
{
	LLM_SCOPE(ELLMTag::AudioMixer);

	check(InData.NumChannels == 2);
 	if (OutData.NumChannels < 2 || DisableSubmixReverbCVarFast == 1)
	{
		// Not supported
		return;
	}

	if (!PlateReverb.IsValid())
	{
		Audio::FPlateReverbFastSettings NewSettings;
		ReverbParams.CopyParams(NewSettings);
		PlateReverb = MakeUnique<Audio::FPlateReverbFast>(SampleRate, 512, NewSettings);
	}

	CSV_SCOPED_TIMING_STAT(Audio, SubmixReverb);
	SCOPE_CYCLE_COUNTER(STAT_AudioMixerSubmixReverb);

	UpdateParameters();

	float LastWet = CurrentWetDry.WetLevel;
	float LastDry = CurrentWetDry.DryLevel;
	WetDryParams.GetParams(&CurrentWetDry);

	// Set to most recent if uninitialized (less than 0.0f)
	if (LastWet < 0.0f)
	{
		LastWet = CurrentWetDry.WetLevel;
	}


	WetInputBuffer.Reset();
	if (InData.AudioBuffer->Num() > 0)
	{
		// Wet level is applied to input audio to preserve reverb tail when changing wet level
		WetInputBuffer.AddZeroed(InData.AudioBuffer->Num());
		Audio::ArrayMixIn(*InData.AudioBuffer, WetInputBuffer, LastWet, CurrentWetDry.WetLevel);
	}

	PlateReverb->ProcessAudio(WetInputBuffer, InData.NumChannels, *OutData.AudioBuffer, OutData.NumChannels);
}

bool FSubmixEffectReverb::SetParameters(const FAudioEffectParameters& InParams)
{
	LLM_SCOPE(ELLMTag::AudioMixer);

	const FAudioReverbEffect& ReverbEffectParams = static_cast<const FAudioReverbEffect&>(InParams);

	/* `FPlateReverb` produces slightly different quality effect than `FPlateReverb`. Comparing the 
	 * settings between FSubmixEffectReverb and FSubmixEffectReverb slight differences will arise.
	 *
	 * The delay line implementations significantly differ between the `FPlateReverb` and `FPlateReverb` classes.
	 * Specifically, the `FPlateReverb` class utilizes linearly interpolated fractional delay line and fractional
	 * delays while the `FPlateReverb` class uses integer delay lines and integer delays whenever possible.
	 * Linearly interpolated fractional delay lines introduce a low pass filter dependent upon the fractional portion
	 * of the delay value. As a result, the `FPlateReverb` class produces a darker reverb.
	 */
	Audio::FPlateReverbFastSettings NewSettings;

	NewSettings.bEnableEarlyReflections = !ReverbEffectParams.bBypassEarlyReflections;
	NewSettings.bEnableLateReflections = !ReverbEffectParams.bBypassLateReflections;

	// Early Reflections
	NewSettings.EarlyReflections.Gain = FMath::GetMappedRangeValueClamped(FVector2f{ 0.0f, 3.16f }, FVector2f{ 0.0f, 1.0f }, ReverbEffectParams.ReflectionsGain);
	NewSettings.EarlyReflections.PreDelayMsec = FMath::GetMappedRangeValueClamped(FVector2f{ 0.0f, 0.3f }, FVector2f{ 0.0f, 300.0f }, ReverbEffectParams.ReflectionsDelay);
	NewSettings.EarlyReflections.Bandwidth = FMath::GetMappedRangeValueClamped(FVector2f{ 0.0f, 1.0f }, FVector2f{ 0.0f, 1.0f }, 1.0f - ReverbEffectParams.GainHF);

	// LateReflections
	NewSettings.LateReflections.LateDelayMsec = FMath::GetMappedRangeValueClamped(FVector2f{ 0.0f, 0.1f }, FVector2f{ 0.0f, 100.0f }, ReverbEffectParams.LateDelay);
	NewSettings.LateReflections.LateGainDB = FMath::GetMappedRangeValueClamped(FVector2f{ 0.0f, 1.0f }, FVector2f{ 0.0f, 1.0f }, ReverbEffectParams.Gain);
	NewSettings.LateReflections.Bandwidth = FMath::GetMappedRangeValueClamped(FVector2f{ 0.0f, 1.0f }, FVector2f{ 0.1f, 0.6f }, ReverbEffectParams.AirAbsorptionGainHF);
	NewSettings.LateReflections.Diffusion = FMath::GetMappedRangeValueClamped(FVector2f{ 0.05f, 1.0f }, FVector2f{ 0.0f, 0.95f }, ReverbEffectParams.Diffusion);
	NewSettings.LateReflections.Dampening = FMath::GetMappedRangeValueClamped(FVector2f{ 0.05f, 1.95f }, FVector2f{ 0.0f, 0.999f }, ReverbEffectParams.DecayHFRatio);
	NewSettings.LateReflections.Density = FMath::GetMappedRangeValueClamped(FVector2f{ 0.0f, 0.95f }, FVector2f{ 0.06f, 1.0f }, ReverbEffectParams.Density);

	// Use mapping function to get decay time in seconds to internal linear decay scale value
	const float DecayValue = DecayCurve.Eval(ReverbEffectParams.DecayTime);
	NewSettings.LateReflections.Decay = DecayValue;

	// Convert to db
	NewSettings.LateReflections.LateGainDB = Audio::ConvertToDecibels(NewSettings.LateReflections.LateGainDB);

	// Apply the settings the thread safe settings object
	ReverbParams.SetParams(NewSettings);

	// Apply wet/dry level
	// When using a FAudioReverbEffect, the volume parameter controls wetness and the dry level remains 0.
	Audio::FWetDry NewWetDry(ReverbEffectParams.Volume, 0.f);
	WetDryParams.SetParams(NewWetDry);

	return true;
}

void FSubmixEffectReverb::UpdateParameters()
{
	Audio::FPlateReverbFastSettings NewSettings;
	if (PlateReverb.IsValid() && ReverbParams.GetParams(&NewSettings))
	{
		PlateReverb->SetSettings(NewSettings);
	}

	// Check cVars for quad mapping
	Audio::FPlateReverbFastSettings::EQuadBehavior TargetQuadBehavior;
	if (DisableQuadReverbCVarFast)
	{
		// Disable quad mapping.
 		TargetQuadBehavior = Audio::FPlateReverbFastSettings::EQuadBehavior::StereoOnly;
	}
	else if (!DisableQuadReverbCVarFast && EnableReverbStereoFlipForQuadCVarFast)
	{
		// Enable quad flipped mapping
		TargetQuadBehavior = Audio::FPlateReverbFastSettings::EQuadBehavior::QuadFlipped;
	}
	else
	{
		// Enable quad mapping
		TargetQuadBehavior = Audio::FPlateReverbFastSettings::EQuadBehavior::QuadMatched;
	}

	if (!PlateReverb.IsValid())
	{
		return;
	}

	// Check if settings need to be updated
	const Audio::FPlateReverbFastSettings& ReverbSettings = PlateReverb->GetSettings();
	if (ReverbSettings.QuadBehavior != TargetQuadBehavior)
	{
		// Update quad settings 
		NewSettings = ReverbSettings;
		NewSettings.QuadBehavior = TargetQuadBehavior;
		PlateReverb->SetSettings(NewSettings);
	}
}

void USubmixEffectReverbPreset::PostInitProperties()
{
	Super::PostInitProperties();

	// The original default for FSubmixEffectReverbSettings::Gain was 0.0, which silenced the effect entirely —
	// users would add the reverb preset and hear nothing until manually setting Gain. We want new assets to
	// default to 1.0 (audible out of the box), but changing the CDO would silently affect existing assets
	// that relied on the old default without ever serializing the value. RF_NeedLoad is set on any object
	// being deserialized from disk, so gating here ensures only freshly created assets get the new default;
	// existing assets are always left exactly as serialized.
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		Settings.Gain = 1.0f;
	}
}

void USubmixEffectReverbPreset::SetSettingsWithReverbEffect(const UReverbEffect* InReverbEffect, const float InWetLevel, const float InDryLevel)
{
	if (InReverbEffect)
	{
		Settings.bBypassEarlyReflections = InReverbEffect->bBypassEarlyReflections;
		Settings.bBypassLateReflections = InReverbEffect->bBypassLateReflections;
		Settings.Density = InReverbEffect->Density;
		Settings.Diffusion = InReverbEffect->Diffusion;
		Settings.Gain = InReverbEffect->Gain;
		Settings.GainHF = InReverbEffect->GainHF;
		Settings.DecayTime = InReverbEffect->DecayTime;
		Settings.DecayHFRatio = InReverbEffect->DecayHFRatio;
		Settings.ReflectionsGain = InReverbEffect->ReflectionsGain;
		Settings.ReflectionsDelay = InReverbEffect->ReflectionsDelay;
		Settings.LateGain = InReverbEffect->LateGain;
		Settings.LateDelay = InReverbEffect->LateDelay;
		Settings.AirAbsorptionGainHF = InReverbEffect->AirAbsorptionGainHF;
		Settings.WetLevel = InWetLevel;
		Settings.DryLevel = InDryLevel;

		Update();
	}
}

void USubmixEffectReverbPreset::SetSettings(const FSubmixEffectReverbSettings& InSettings)
{
	UpdateSettings(InSettings);
}

