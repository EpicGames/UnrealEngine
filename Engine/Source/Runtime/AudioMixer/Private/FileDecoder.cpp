// Copyright Epic Games, Inc. All Rights Reserved.

#include "FileDecoder.h"
#include "HAL/PlatformFileManager.h"
#include "DSP/FloatArrayMath.h"

FAudioFileReader::FAudioFileReader(const FString& InPath)
{
	QualityInfo = { 0 };

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	FileHandle.Reset(PlatformFile.OpenRead(*InPath));
	if (FileHandle.IsValid())
	{
		int64 FileSize = FileHandle->Size();
		CompressedFile.Reset();
		CompressedFile.AddUninitialized(FileSize);
		FileHandle->Read(CompressedFile.GetData(), FileSize);
		
		Decompressor.Reset(GetNewDecompressorForFile(InPath));

		if (Decompressor.IsValid())
		{
			Decompressor->ReadCompressedInfo(CompressedFile.GetData(), FileSize, &QualityInfo);
		}
		else
		{
			QualityInfo.NumChannels = 0;
			UE_LOGF(LogTemp, Error, "Invalid file extension!");
		}
	}
	else
	{
		QualityInfo.NumChannels = 0;
		UE_LOGF(LogTemp, Error, "Invalid file %ls!", *InPath);
	}
}

void FAudioFileReader::GetFileInfo(FSoundQualityInfo& OutInfo)
{
	OutInfo = QualityInfo;
}

bool FAudioFileReader::PopAudio(float* OutAudio, int32 NumSamples)
{
	check(FileHandle.IsValid());
	check(Decompressor.IsValid());

	DecompressionBuffer.Reset();
	DecompressionBuffer.AddUninitialized(NumSamples);

	bool bIsFinished = Decompressor->ReadCompressedData((uint8*) DecompressionBuffer.GetData(), false, NumSamples * sizeof(Audio::DefaultUSoundWaveSampleType));

	// Convert to float:
	Audio::ArrayPcm16ToFloat(MakeArrayView((int16*)DecompressionBuffer.GetData(), NumSamples), MakeArrayView(OutAudio, NumSamples));

	return bIsFinished;
}

ICompressedAudioInfo* FAudioFileReader::GetNewDecompressorForFile(const FString& InPath)
{	
	using namespace Audio;
	using FMapping = TTuple<FString, FName>; 
	const FMapping Extensions[] =
	{	
		{ TEXT(".opus"), NAME_OPUS },
		{ TEXT(".vorbis"), NAME_OGG },
		{ TEXT(".binka"), NAME_BINKA }
	};

	const FString LowerPath = InPath.ToLower();
	if (const FMapping* Found = Algo::FindByPredicate(Extensions, [LowerPath](const auto &i) -> bool { return LowerPath.EndsWith(i.Key); }) )
	{
		return IAudioInfoFactoryRegistry::Get().Create(Found->Value);		
	}
	UE_LOGF(LogTemp, Error, "Unable to determin/create decompressor for '%ls'", *InPath);
	return nullptr;
}
