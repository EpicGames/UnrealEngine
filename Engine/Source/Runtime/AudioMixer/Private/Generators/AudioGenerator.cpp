// Copyright Epic Games, Inc. All Rights Reserved.

#include "Generators/AudioGenerator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioGenerator)

UAudioGenerator::UAudioGenerator() 
{
}

UAudioGenerator::~UAudioGenerator()
{
}

FAudioGeneratorHandle UAudioGenerator::AddGeneratorDelegate(FOnAudioGenerate InFunction)
{
	static int32 Id = 0;

	FAudioGeneratorHandle NewHandle;
	NewHandle.Id = Id++;

	FScopeLock Lock(&CritSect);
	OnGeneratedMap.Add(NewHandle.Id, MoveTemp(InFunction));
	return NewHandle;
}

void UAudioGenerator::RemoveGeneratorDelegate(FAudioGeneratorHandle InHandle)
{
	FScopeLock Lock(&CritSect);
	OnGeneratedMap.Remove(InHandle.Id);
}

void UAudioGenerator::Init(int32 InSampleRate, int32 InNumChannels)
{
	SampleRate = InSampleRate;
	NumChannels = InNumChannels;
}

void UAudioGenerator::OnGeneratedAudio(const float* InAudio, int32 NumSamples)
{
	FScopeLock Lock(&CritSect);
	for (auto& It : OnGeneratedMap)
	{
		It.Value(InAudio, NumSamples);
	}
}


