// Copyright Epic Games, Inc. All Rights Reserved.

#include "Generators/SoundWaveScrubber.h"
#include "Sound/SoundWaveProxyPlayer.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundWaveScrubber)

namespace Audio
{ 

FSoundWaveScrubber::FSoundWaveScrubber()
	: CurrentPlayheadTimeSeconds(0.0f)
	, SRC(Audio::ISampleRateConverter::CreateSampleRateConverter())
	, GrainDurationSeconds(0.0f)
{
	// Generate the grain envelope data
	Audio::Grain::GenerateEnvelopeData(GrainEnvelope, 512, Audio::Grain::EEnvelope::Hann);
}

FSoundWaveScrubber::~FSoundWaveScrubber()
{
}

void FSoundWaveScrubber::Init(TSharedRef<const FSoundWaveProxy> InSoundWaveProxyRef, float InSampleRate, int32 InNumChannels, float InPlayheadTimeSeconds)
{
	SoundWaveDataPtr = InSoundWaveProxyRef->GetSoundWaveDataRef();
	check(SoundWaveDataPtr.IsValid());
	check(InSampleRate > 0);

	AudioMixerSampleRate = InSampleRate;
	SourceFileSampleRate = SoundWaveDataPtr->GetSampleRate();
	SourceFileDurationSeconds = SoundWaveDataPtr->GetDuration();
	NumChannels = InNumChannels;

	TargetGrainDurationRange = { 0.4f, 0.05f };
	GrainDurationRange = TargetGrainDurationRange;
	GrainDurationSeconds = TargetGrainDurationRange.X;

	CurrentPlayheadTimeSeconds.Set(InPlayheadTimeSeconds, 0.0f);
	TargetPlayheadTimeSeconds = CurrentPlayheadTimeSeconds.GetValue();

	// We need 3 slots for decoded data. 2 for the grain playback, 1 to decode new chunks while grains are playing.
	DecodedChunks.Reset();
	DecodedChunks.AddDefaulted(3);

	float DecoderSeekTimeSeconds = FMath::Max(CurrentPlayheadTimeSeconds.GetValue() - 0.5f * DecodedAudioSizeInSeconds, 0.0f);

	bHasErrorWithDecoder = false;

	// This could result in a decoder error if there is an issue with the sound wave proxy
	DecodeToDataChunk(DecodedChunks[0], DecoderSeekTimeSeconds);
}

void FSoundWaveScrubber::Init(FSoundWaveProxyPtr InSoundWaveProxyPtr, float InSampleRate, int32 InNumChannels, float InPlayheadTimeSeconds)
{
	if (!ensure(InSoundWaveProxyPtr))
	{
		return;
	}

	TSharedRef<const FSoundWaveProxy> ProxyRef = InSoundWaveProxyPtr->AsShared();
	this->Init(ProxyRef, InSampleRate, InNumChannels, InPlayheadTimeSeconds);
}

int32 FSoundWaveScrubber::GetDecodedDataChunkIndexForCurrentReadIndex(int32 InReadFrameIndex)
{
	for (int32 i = 0; i < DecodedChunks.Num(); ++i)
	{
		const FDecodedDataChunk& DecodedChunk = DecodedChunks[i];
		if (DecodedChunk.PCMAudio.Num() > 0)
		{
			int32 DecodedFrameCount = DecodedChunk.PCMAudio.Num() / NumChannels;
			if (InReadFrameIndex >= DecodedChunk.FrameStart && InReadFrameIndex < DecodedChunk.FrameStart + DecodedFrameCount)
			{
				// We found a decoded audio chunk that contains the desired read frame index
				return i;
			}
		}
	}
	return INDEX_NONE;
}

int32 FSoundWaveScrubber::DecodeDataChunkIndexForCurrentReadIndex(int32 InReadFrameIndex)
{
	check(!bHasErrorWithDecoder);
	
	// No decoded chunk was found for desired read frame index. 
	// This indicates that we need to decode more audio.
	for (int32 i = 0; i < DecodedChunks.Num(); ++i)
	{
		FDecodedDataChunk& DecodedChunk = DecodedChunks[i];
		if (!DecodedChunk.NumGrainsUsingChunk)
		{
			float DecoderSeekTimeSeconds = InReadFrameIndex / AudioMixerSampleRate;
			DecodeToDataChunk(DecodedChunk, DecoderSeekTimeSeconds);
			check(DecodedChunk.PCMAudio.Num() > 0);
			return i;
		}
	}

	FDecodedDataChunk NewChunk;
	float DecoderSeekTimeSeconds = InReadFrameIndex / AudioMixerSampleRate;
	DecodeToDataChunk(NewChunk, DecoderSeekTimeSeconds);
	check(NewChunk.PCMAudio.Num() > 0);
	DecodedChunks.Add(MoveTemp(NewChunk));
	
	// This should not be able to error at this point
	check(!bHasErrorWithDecoder);

	return DecodedChunks.Num() - 1;
}


void FSoundWaveScrubber::DecodeToDataChunk(FDecodedDataChunk& InOutDataChunk, float InDecoderSeekTimeSeconds)
{
	check(InOutDataChunk.NumGrainsUsingChunk == 0);
	check(DecodedAudioSizeInSeconds > 0.0f);
	check(SourceFileSampleRate > 0.0f);
	check(InDecoderSeekTimeSeconds >= 0.0f);
	check(NumChannels > 0);
	check(SoundWaveDataPtr.IsValid());

	if (!SoundWaveProxyPlayerPtr.IsValid())
	{
		// Make sure we haven't already tried to create a proxy reader (decoder)
		check(!bHasErrorWithDecoder);

		// Create the proxy player, which is our decoder. Initialize it at the initial playhead time
		int32 SourceFileNumChanels = SoundWaveDataPtr->GetNumChannels();
		FSoundWaveProxyPlayer::FSettings ProxyPlayerSettings(SourceFileSampleRate, SourceFileNumChanels);
		ProxyPlayerSettings.MaxDecodeSizeFrames = DecodedAudioSizeInSeconds * SourceFileSampleRate;
		SoundWaveProxyPlayerPtr = FSoundWaveProxyPlayer::Create(ProxyPlayerSettings);
		SoundWaveProxyPlayerPtr->SetSoundWave(SoundWaveDataPtr.ToSharedRef());

		if (!SoundWaveProxyPlayerPtr->IsPlayerValid())
		{
			bHasErrorWithDecoder = true;
			UE_LOGF(LogAudioMixer, Warning, "Unable to make a Sound Wave Proxy Player for sound wave '%ls' in the sound wave scrubber.", *SoundWaveDataPtr->GetFName().ToString());
			return;
		}

		SoundWaveProxyPlayerPtr->SeekToTime(InDecoderSeekTimeSeconds);
	}
	else
	{
		check(SoundWaveProxyPlayerPtr.IsValid());
		float DecoderSeekTimeSeconds = InDecoderSeekTimeSeconds;
		// If proxy player already exists, then simply seek the decoder to the desired location
		SoundWaveProxyPlayerPtr->SeekToTime(DecoderSeekTimeSeconds);
	}

	InOutDataChunk.FrameStart = InDecoderSeekTimeSeconds * AudioMixerSampleRate;

	// We allocate the size of buffer pre-SRC based on source file sample rate
	int32 DecodedAudioSize = DecodedAudioSizeInSeconds * SourceFileSampleRate * NumChannels;
	check(DecodedAudioSize > 0);
	InOutDataChunk.PCMAudio.Reset();
	InOutDataChunk.PCMAudio.AddUninitialized(DecodedAudioSize);

	// This does the actual decoding of the audio to match the size of the input buffer
	SoundWaveProxyPlayerPtr->GenerateSourceAudio(InOutDataChunk.PCMAudio);

	// Check if we need to do SRC. If we do, this will change the allocated 
	// size (potentially expanding or shrinking) to match 
	// the audio mixer sample rate.
	if (!FMath::IsNearlyEqual(SourceFileSampleRate, AudioMixerSampleRate))
	{
		SRC->Init((float)SourceFileSampleRate / AudioMixerSampleRate, NumChannels);
		TArray<float> SampleRateConvertedPCM;
		SRC->ProcessFullbuffer(InOutDataChunk.PCMAudio.GetData(), InOutDataChunk.PCMAudio.Num(), SampleRateConvertedPCM);
		InOutDataChunk.PCMAudio = MoveTemp(SampleRateConvertedPCM);
	}
}

FSoundWaveScrubber::FGrain FSoundWaveScrubber::SpawnGrain()
{
	// If we have a decoder error, spawn null grains vs asserting
	if (bHasErrorWithDecoder)
	{
		return { 0 };
	}
	
	// Try to retrieve a decoded data chunk for the current read frame based on the curret, interpolated playhead time seconds value
	int32 CurrentReadFrame = CurrentPlayheadTimeSeconds.GetValue() * AudioMixerSampleRate;
	int32 DecodedDataChunkIndex = GetDecodedDataChunkIndexForCurrentReadIndex(CurrentReadFrame);
	if (DecodedDataChunkIndex == INDEX_NONE)
	{
		DecodedDataChunkIndex = DecodeDataChunkIndexForCurrentReadIndex(CurrentReadFrame);
	}

	if (!ensureMsgf(DecodedDataChunkIndex != INDEX_NONE, TEXT("Failed to retreived decoded chunk for read frame index: %d"), CurrentReadFrame))
	{
		return { 0 };
	}

	// Get the grain runtime data for the new grain spawn
	FGrain NewGrain;
	NewGrain.CurrentRenderedFramesCount = 0;
	NewGrain.DecodedDataChunkIndex = DecodedDataChunkIndex;
	NewGrain.CurrentReadFrame = CurrentReadFrame;
	NewGrain.GrainDurationFrames = CurrentGrainDurationFrames;
	DecodedChunks[DecodedDataChunkIndex].NumGrainsUsingChunk++;

	GrainCount++;
	NumActiveGrains++;

	return NewGrain;
}

void FSoundWaveScrubber::SetIsScrubbing(bool bInIsScrubbing)
{
	bIsScrubbing = bInIsScrubbing;
}

void FSoundWaveScrubber::SetIsScrubbingWhileStationary(bool bInIsScrubWhileStationary)
{
	bIsScrubbingWhileStationary = bInIsScrubWhileStationary;
}

void FSoundWaveScrubber::SetPlayheadTime(float InPlayheadTimeSeconds)
{
	FScopeLock Lock(&CritSect);

	TargetPlayheadTimeSeconds = FMath::Fmod(FMath::Max(InPlayheadTimeSeconds, 0.0f), SourceFileDurationSeconds);
}

void FSoundWaveScrubber::SetGrainDurationRange(const FVector2D& InGrainDurationRange)
{
	FVector2D GrainDurationRangeClamped =
	{
		FMath::Clamp(InGrainDurationRange.X, 0.05f, 0.5f),
		FMath::Clamp(InGrainDurationRange.Y, 0.05f, 0.5f),
	};

	FScopeLock Lock(&CritSect);
	TargetGrainDurationRange = GrainDurationRangeClamped;
}

int32 FSoundWaveScrubber::RenderAudio(TArrayView<float>& OutAudio)
{
	SCOPED_NAMED_EVENT_TEXT("FSoundWaveScrubber::RenderAudio", FColor::Emerald);

	// If we have an error with our decoder, we don't need to render audio
	// To avoid spamming, we'll allow it to render silence
	if (bHasErrorWithDecoder || !SoundWaveDataPtr.IsValid())
	{
		return OutAudio.Num();
	}

	// Number of frames of this generate audio
	int32 NumFrames = OutAudio.Num() / NumChannels;
	float DeltaTimeSecond = (float)NumFrames / AudioMixerSampleRate;

	float PlayheadTimeDistanceSeconds = 0.0f;

	constexpr float PlayheadLerpTime = 0.2f;

	// Update parameters
	{
		FScopeLock Lock(&CritSect);

		// Update the current playhead time seconds
		if (!FMath::IsNearlyEqual(TargetPlayheadTimeSeconds, CurrentPlayheadTimeSeconds.GetTargetValue()))
		{
			float PlayheadTimeDelta = FMath::Abs(CurrentPlayheadTimeSeconds.GetValue() - TargetPlayheadTimeSeconds);
			// If the playhead time jumps suddenly, we'll instantly set the current playhead to the target
			if (PlayheadTimeDelta > 0.5f)
			{
				CurrentPlayheadTimeSeconds.Set(TargetPlayheadTimeSeconds, 0.0f);
			}
			else
			{
				CurrentPlayheadTimeSeconds.Set(TargetPlayheadTimeSeconds, PlayheadLerpTime);
			}
		}
		float PrevPlayHeadTime = CurrentPlayheadTimeSeconds.GetValue();
		CurrentPlayheadTimeSeconds.Update(DeltaTimeSecond);

		// Check if we're stationary
		if (!bIsScrubbingWhileStationary)
		{
			if (FMath::IsNearlyEqual(PrevPlayHeadTime, CurrentPlayheadTimeSeconds.GetValue(), 0.001f))
			{
				TimeSincePlayheadHasNotChanged += DeltaTimeSecond;
			}
			else
			{
				TimeSincePlayheadHasNotChanged = 0.0f;
			}

			bIsScrubbingDueToBeingStationary = TimeSincePlayheadHasNotChanged < 0.1f;
		}
		else
		{
			bIsScrubbingDueToBeingStationary = true;
		}

		PlayheadTimeDistanceSeconds = FMath::Abs(CurrentPlayheadTimeSeconds.GetTargetValue() - CurrentPlayheadTimeSeconds.GetValue());

		GrainDurationRange = TargetGrainDurationRange;
	}

	// This maps the playhead delta from the target playhead time (which is an indirect measure of playhead velocity) to the duration range
	float MappedGrainDurationRange = FMath::GetMappedRangeValueClamped({ 0.0f, PlayheadLerpTime }, GrainDurationRange, PlayheadTimeDistanceSeconds);
	GrainDurationSeconds = MappedGrainDurationRange;

	// Update the grain duration based on a mapping between the duration range and scrub velocity
	CurrentGrainDurationFrames = GrainDurationSeconds * AudioMixerSampleRate;
	CurrentHalfGrainDurationFrames = 0.5f * CurrentGrainDurationFrames;

	// If we're actively scrubbing we need to spawn grains and render the granular audio
	if (bIsScrubbing && bIsScrubbingDueToBeingStationary)
	{
		if (!ActiveGrains.Num())
		{
			FSoundWaveScrubber::FGrain NewGrain = SpawnGrain();
			if (NewGrain.GrainDurationFrames > 0)
			{
				ActiveGrains.Add(NewGrain);
			}

			NumFramesTillNextGrainSpawn = CurrentHalfGrainDurationFrames;
		}

		int32 StartRenderFrame = 0;
		int32 NumFramesToRender = FMath::Min(NumFramesTillNextGrainSpawn, NumFrames);
		int32 NumFramesRendered = 0;
		int32 RemainingFrames = NumFrames;

		while (RemainingFrames > 0)
		{
			if (ActiveGrains.Num() > 0)
			{
				// This renders the currently active grains (which should only ever max 2 grains)
				// starting from the given start render frame and for the  numbe of indicated frames
				RenderActiveGrains(OutAudio, StartRenderFrame, NumFramesToRender);
			}

			// Update the number of frames rendered this render block
			NumFramesRendered += NumFramesToRender;

			NumFramesTillNextGrainSpawn -= NumFramesToRender;
			check(NumFramesTillNextGrainSpawn >= 0);

			// Determine how many more frames, this block, that we need to render
			RemainingFrames = FMath::Max(NumFrames - NumFramesRendered, 0);

			// Check if we need to spawn a new grain
			if (NumFramesTillNextGrainSpawn == 0)
			{
				// Spawn a new grain with a frame start 
				StartRenderFrame = NumFrames - RemainingFrames;
				FSoundWaveScrubber::FGrain NewGrain = SpawnGrain(); 
				if (NewGrain.GrainDurationFrames > 0)
				{
					ActiveGrains.Add(NewGrain);
				}

				// If we still have frames remaining in the buffer, then 
				NumFramesTillNextGrainSpawn = CurrentHalfGrainDurationFrames;

				NumFramesToRender = FMath::Min(NumFramesTillNextGrainSpawn, RemainingFrames);
			}
		}
	}

	return OutAudio.Num();
}

void FSoundWaveScrubber::UpdateGrainDecodeData(FGrain& InGrain)
{
	// Check for invalid chunk index 
	if (!ensureMsgf(DecodedChunks.IsValidIndex(InGrain.DecodedDataChunkIndex), TEXT("Decoded data chunk index was invalid: %d"), InGrain.DecodedDataChunkIndex))
	{
		// If the index is invalid, finish the grain by setting it's rendered frame count to the grain duration. This kills the grain.
		InGrain.DecodedDataChunkIndex = INDEX_NONE;
		InGrain.CurrentRenderedFramesCount = InGrain.GrainDurationFrames;
		return;
	}

	FDecodedDataChunk& DecodedData = DecodedChunks[InGrain.DecodedDataChunkIndex];

	// Total number of frames of decoded data in the chunk
	int32 NumReadFramesInDecodedData = DecodedData.PCMAudio.Num() / NumChannels;

	// The number of frames that this grain is offset from the decoded data
	int32 NumFramesOffsetInDecodedData = InGrain.CurrentReadFrame - DecodedData.FrameStart;
	int32 NumFramesPossibleToRenderInChunk = NumReadFramesInDecodedData - NumFramesOffsetInDecodedData;

	// If our current grain chunk is out of range of the current chunk, lets re-acquire the chunk
	// note this can happen to slight SRC rounding errors, etc.
	if (NumFramesOffsetInDecodedData < 0 || NumFramesPossibleToRenderInChunk < 0)
	{
		// first decrement the current decoded data chunk
		DecodedData.NumGrainsUsingChunk = FMath::Max(DecodedData.NumGrainsUsingChunk - 1, 0);
		int32 NewIndex = GetDecodedDataChunkIndexForCurrentReadIndex(InGrain.CurrentReadFrame);
		if (NewIndex == INDEX_NONE)
		{
			NewIndex = DecodeDataChunkIndexForCurrentReadIndex(InGrain.CurrentReadFrame);
		}

		// Failed to reacquire the decode chunk
		if (!ensureMsgf(NewIndex != INDEX_NONE, TEXT("Failed to reacquire decode chunk in soundwave scrubber at current read frame: %d"), InGrain.CurrentReadFrame))
		{
			InGrain.DecodedDataChunkIndex = INDEX_NONE;
			InGrain.CurrentRenderedFramesCount = InGrain.GrainDurationFrames;
			return;
		}

		// Update the grain to the refreshed decode chunk
		InGrain.DecodedDataChunkIndex = NewIndex;
		DecodedChunks[InGrain.DecodedDataChunkIndex].NumGrainsUsingChunk++;
	}
	// If we've totally consumed this decoded audio chunk, we need to get a new decoded audio chunk
	else if (NumFramesPossibleToRenderInChunk == 0)
	{
		// We're no longer using this decoded audio chunk
		DecodedData.NumGrainsUsingChunk  = FMath::Max(DecodedData.NumGrainsUsingChunk - 1, 0);

		int32 NewIndex = GetDecodedDataChunkIndexForCurrentReadIndex(InGrain.CurrentReadFrame);
		if (NewIndex == INDEX_NONE)
		{
			NewIndex = DecodeDataChunkIndexForCurrentReadIndex(InGrain.CurrentReadFrame);
		}
		if (!ensureMsgf(NewIndex != INDEX_NONE, TEXT("Couldn't find a decoded data chunk for current read frame: %d"), InGrain.CurrentReadFrame))
		{
			// Kill the grain
			InGrain.DecodedDataChunkIndex = INDEX_NONE;
			InGrain.CurrentRenderedFramesCount = InGrain.GrainDurationFrames;
			return;
		}

		// Defensive, non-fatal range validation
		// We very rarely get out-of-range due to slight rounding errors.
		const int32 NewFrameStart = DecodedChunks[NewIndex].FrameStart;
		const int32 NewFrameEnd = NewFrameStart + DecodedChunks[NewIndex].PCMAudio.Num() / NumChannels;
		if (!ensureMsgf(InGrain.CurrentReadFrame >= NewFrameStart && InGrain.CurrentReadFrame < NewFrameEnd, TEXT("Current read frame (%d) is out of range [%d, %d)."), InGrain.CurrentReadFrame, NewFrameStart, NewFrameEnd))
		{
			// We're out of range, kill the grain
			InGrain.CurrentRenderedFramesCount = InGrain.GrainDurationFrames;
			return;
		}

		// this chunk is good and we're in the right range
		InGrain.DecodedDataChunkIndex = NewIndex;
		DecodedChunks[InGrain.DecodedDataChunkIndex].NumGrainsUsingChunk++;
	}
}

void FSoundWaveScrubber::RenderActiveGrains(TArrayView<float>& OutAudio, int32 InStartFrame, int32 NumFramesToRender)
{
	check(NumChannels > 0);

	// Reverse iterate so we can quickly remove grains when they're done
	for (int32 GrainIndex = ActiveGrains.Num() - 1; GrainIndex >= 0; --GrainIndex)
	{
		FGrain& Grain = ActiveGrains[GrainIndex];

		// This is the number of frames we have left to render for this grain
		int32 NumFramesLeftInGrain = Grain.GrainDurationFrames - Grain.CurrentRenderedFramesCount;
		int32 NumFramesLeftToRender = FMath::Min(NumFramesToRender, NumFramesLeftInGrain);
		NumFramesLeftToRender = FMath::Max(NumFramesLeftToRender, 0);

		int32 GrainWriteIndex = InStartFrame;

		bool bGrainFinished = (NumFramesLeftInGrain == 0);
		while (NumFramesLeftToRender > 0 && !bGrainFinished)
		{
			// Make sure we have a valid decode data chunk ready for rendering
			UpdateGrainDecodeData(Grain);

			// It's possible we still don't have a valid decoded data chunk index
			if (!DecodedChunks.IsValidIndex(Grain.DecodedDataChunkIndex))
			{
				bGrainFinished = true;
				break;
			}

			// Retrieve the decoded data.
			const FDecodedDataChunk& DecodedData = DecodedChunks[Grain.DecodedDataChunkIndex];

			// Total number of frames of decoded data in the chunk
			int32 NumReadFramesInDecodedData = DecodedData.PCMAudio.Num() / NumChannels;

			// The number of frames that this grain is offset from the decoded data
			int32 NumFramesOffsetInDecodedData = FMath::Max(Grain.CurrentReadFrame - DecodedData.FrameStart, 0);
			int32 NumFramesPossibleToRenderInChunk = FMath::Max(NumReadFramesInDecodedData - NumFramesOffsetInDecodedData, 0);

			// This may completely consume the decoded chunk here, so only render the maximum number of frames in the 
			// chunk. if NumFramesToRender is smaller than that, then we won't completely consume the decoded data
			int32 NumFramesToRenderInThisChunk = FMath::Min(NumFramesPossibleToRenderInChunk, NumFramesLeftToRender);
			NumFramesToRenderInThisChunk = FMath::Max(NumFramesToRenderInThisChunk, 0);

			int32 SampleWriteIndex = GrainWriteIndex * NumChannels;
			int32 SampleReadIndex = NumFramesOffsetInDecodedData * NumChannels;

			// For inner-loop audio rendering, we avoid the constant array-access checks that happen on our containers
			const float* DecodedPCMDataPtr = DecodedData.PCMAudio.GetData();
			float* OutDataPtr = OutAudio.GetData();

			for (int32 FrameIndex = 0; FrameIndex < NumFramesToRenderInThisChunk; ++FrameIndex)
			{
				// Retrieve the grain amplitude from the envelope
				// Note the envelope is intentionally sized to match the size of the grain so we don't have
				// to do any interpolation math on look up.
				float EnvelopeFraction = FMath::Clamp((float)Grain.CurrentRenderedFramesCount++ / Grain.GrainDurationFrames, 0.0f, 1.0f);
				float GrainAmplitude = Audio::Grain::GetValue(GrainEnvelope, EnvelopeFraction);

				for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
				{
					// Read the decoded sample of the audio at this channel index
					float DecodedSampleValue = DecodedPCMDataPtr[SampleReadIndex++];

					// Scale the sample value by the grain amplitude
					DecodedSampleValue *= GrainAmplitude;

					// Mix the grain audio into the output buffer
					OutDataPtr[SampleWriteIndex++] += DecodedSampleValue;
				}
			}

			Grain.CurrentReadFrame += NumFramesToRenderInThisChunk;

			GrainWriteIndex += NumFramesToRenderInThisChunk;
			NumFramesLeftToRender -= NumFramesToRenderInThisChunk;

			bGrainFinished = (Grain.CurrentRenderedFramesCount >= Grain.GrainDurationFrames);
		}

		// If the grain has finished, then remove it from the active grain list
		if (bGrainFinished)
		{
			if (DecodedChunks.IsValidIndex(Grain.DecodedDataChunkIndex))
			{
				FDecodedDataChunk& DataChunk = DecodedChunks[Grain.DecodedDataChunkIndex];
				DataChunk.NumGrainsUsingChunk = FMath::Max(DataChunk.NumGrainsUsingChunk - 1, 0);
			}
			ActiveGrains.RemoveAtSwap(GrainIndex, EAllowShrinking::No);
			NumActiveGrains = FMath::Max(NumActiveGrains - 1, 0);
		}
	}
}

void FSoundWaveScrubberGenerator::Init(TSharedRef<const FSoundWaveProxy> InSoundWaveProxyRef, float InSampleRate, int32 InNumChannels, float InPlayheadTimeSeconds)
{
	NumChannels = InNumChannels;
	SoundWaveScrubber.Init(InSoundWaveProxyRef, InSampleRate, InNumChannels, InPlayheadTimeSeconds);
}

void FSoundWaveScrubberGenerator::Init(FSoundWaveProxyPtr InSoundWaveProxyPtr, float InSampleRate, int32 InNumChannels, float InPlayheadTimeSeconds)
{
	if (!ensure(InSoundWaveProxyPtr))
	{
		return;
	}

	TSharedRef<const FSoundWaveProxy> ProxyRef = InSoundWaveProxyPtr->AsShared();
	this->Init(ProxyRef, InSampleRate, InNumChannels, InPlayheadTimeSeconds);
}

void FSoundWaveScrubberGenerator::SetIsScrubbing(bool bInIsScrubbing)
{
	SoundWaveScrubber.SetIsScrubbing(bInIsScrubbing);
}

void FSoundWaveScrubberGenerator::SetIsScrubbingWhileStationary(bool bInScrubWhileStationary)
{
	SoundWaveScrubber.SetIsScrubbingWhileStationary(bInScrubWhileStationary);
}

void FSoundWaveScrubberGenerator::SetPlayheadTime(float InPlayheadTimeSeconds)
{
	SoundWaveScrubber.SetPlayheadTime(InPlayheadTimeSeconds);
}

void FSoundWaveScrubberGenerator::SetGrainDurationRange(const FVector2D& InGrainDurationRange)
{
	SoundWaveScrubber.SetGrainDurationRange(InGrainDurationRange);
}

int32 FSoundWaveScrubberGenerator::OnGenerateAudio(float* OutAudio, int32 NumSamples)
{
	FMemory::Memzero(OutAudio, NumSamples*sizeof(float));
	TArrayView<float> OutView = TArrayView<float>(OutAudio, NumSamples);
	return SoundWaveScrubber.RenderAudio(OutView);
}

int32 FSoundWaveScrubberGenerator::GetDesiredNumSamplesToRenderPerCallback() const
{
	return 256 * NumChannels;
}

bool FSoundWaveScrubberGenerator::IsFinished() const
{
	// This is intended to be "always on" unless stopped by owning audio component, etc.
	return false;
}

} // namespace Audio

UScrubbedSound::UScrubbedSound(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bProcedural = true;
}

ISoundGeneratorPtr UScrubbedSound::CreateSoundGenerator(const FSoundGeneratorInitParams& InParams)
{
	using namespace Audio;

	if (SoundWaveToScrub)
	{
		TSharedRef<const FSoundWaveProxy> SoundWaveProxyRef = SoundWaveToScrub->GetSoundWaveProxy();

		SoundWaveScrubber = ISoundGeneratorPtr(new FSoundWaveScrubberGenerator());

		FSoundWaveScrubberGenerator* Scrubber = static_cast<FSoundWaveScrubberGenerator*>(SoundWaveScrubber.Get());
		Scrubber->Init(SoundWaveProxyRef, InParams.SampleRate, NumChannels, PlayheadTimeSeconds);
		Scrubber->SetIsScrubbing(bIsScrubbing);
		Scrubber->SetIsScrubbingWhileStationary(bScrubWhileStationary);
		Scrubber->SetGrainDurationRange(GrainDurationRange);

		return SoundWaveScrubber;
	}

	return nullptr;
}

void UScrubbedSound::SetSoundWave(USoundWave* InSoundWave)
{
	 SoundWaveToScrub = InSoundWave;
	 NumChannels = InSoundWave->NumChannels;
}

void UScrubbedSound::SetIsScrubbing(bool bInIsScrubbing)
{
	using namespace Audio;

	bIsScrubbing = bInIsScrubbing;
	if (SoundWaveScrubber.IsValid())
	{
		FSoundWaveScrubberGenerator* Scrubber = static_cast<FSoundWaveScrubberGenerator*>(SoundWaveScrubber.Get());
		Scrubber->SetIsScrubbing(bIsScrubbing);
	}
}

void UScrubbedSound::SetIsScrubbingWhileStationary(bool bInScrubWhileStationary)
{
	using namespace Audio;

	bScrubWhileStationary = bInScrubWhileStationary;
	if (SoundWaveScrubber.IsValid())
	{
		FSoundWaveScrubberGenerator* Scrubber = static_cast<FSoundWaveScrubberGenerator*>(SoundWaveScrubber.Get());
		Scrubber->SetIsScrubbingWhileStationary(bInScrubWhileStationary);
	}
}

void UScrubbedSound::SetPlayheadTime(float InPlayheadTimeSeconds)
{
	using namespace Audio;

	PlayheadTimeSeconds = InPlayheadTimeSeconds;
	if (SoundWaveScrubber.IsValid())
	{
		FSoundWaveScrubberGenerator* Scrubber = static_cast<FSoundWaveScrubberGenerator*>(SoundWaveScrubber.Get());
		Scrubber->SetPlayheadTime(InPlayheadTimeSeconds);
	}
}

void UScrubbedSound::SetGrainDurationRange(const FVector2D& InGrainDurationRangeSeconds)
{
	using namespace Audio;

	GrainDurationRange = InGrainDurationRangeSeconds;
	if (SoundWaveScrubber.IsValid())
	{
		FSoundWaveScrubberGenerator* Scrubber = static_cast<FSoundWaveScrubberGenerator*>(SoundWaveScrubber.Get());
		Scrubber->SetGrainDurationRange(GrainDurationRange);
	}
}
