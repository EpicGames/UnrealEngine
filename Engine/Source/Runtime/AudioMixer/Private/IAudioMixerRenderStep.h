// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace Audio
{
	/** Interface representing an item of audio rendering work that can be scheduled as part
	of an AudioMixerDevice's rendering pass.
	*/
	struct IAudioMixerRenderStep
	{
		virtual ~IAudioMixerRenderStep() = default;

		/* Do the work associated with this render step. */
		virtual void DoRenderStep() = 0;

		/* Get a name for this render step to display for debugging purposes. */
		virtual const TCHAR* GetRenderStepName() = 0;
	};

}
