// Copyright Epic Games, Inc. All Rights Reserved.

#include "Quartz/AudioMixerClock.h"
#include "Quartz/AudioMixerClockManager.h"
#include "AudioMixerSourceManager.h"
#include "Sound/QuartzSubscription.h"
#include "HAL/UnrealMemory.h" // Memcpy


static float HeadlessClockSampleRateCvar = 100000.f;
FAutoConsoleVariableRef CVarHeadlessClockSampleRate(
	TEXT("au.Quartz.HeadlessClockSampleRate"),
	HeadlessClockSampleRateCvar,
	TEXT("Sample rate to use for Quartz Clocks/Metronomes when no Mixer Device is present.\n")
	TEXT("0: Not Enabled, 1: Enabled"),
	ECVF_Default);

namespace Audio
{
	// FQuartzClockProxy Implementation
	// ctor
	FQuartzClockProxy::FQuartzClockProxy(TSharedPtr<FQuartzClock, ESPMode::ThreadSafe> InClock)
		: ClockId(InClock->GetName())
		, SharedQueue(InClock->GetCommandQueue())
		, ClockWeakPtr(InClock)
	{
	}

	bool FQuartzClockProxy::IsValid() const
	{
		return SharedQueue.Pin().IsValid();
	}

	bool FQuartzClockProxy::DoesClockExist() const
	{
		return IsValid();
	}

	bool FQuartzClockProxy::IsClockRunning() const
	{
		TSharedPtr<FQuartzClock, ESPMode::ThreadSafe> ClockPtr = ClockWeakPtr.Pin();
		if (!ClockPtr)
		{
			return false;
		}

		return ClockPtr->IsRunning();
	}

	float FQuartzClockProxy::GetDurationOfQuantizationTypeInSeconds(const EQuartzCommandQuantization& QuantizationType, float Multiplier) const
	{
		TSharedPtr<FQuartzClock, ESPMode::ThreadSafe> ClockPtr = ClockWeakPtr.Pin();
		if (!ClockPtr)
		{
			return 0.f;
		}

		return ClockPtr->GetDurationOfQuantizationTypeInSeconds(QuantizationType, Multiplier);
	}

	float FQuartzClockProxy::GetBeatProgressPercent(
		const EQuartzCommandQuantization& QuantizationType) const
	{
		TSharedPtr<FQuartzClock, ESPMode::ThreadSafe> ClockPtr = ClockWeakPtr.Pin();
		if (!ClockPtr)
		{
			return 0.f;
		}

		return ClockPtr->GetBeatProgressPercent(QuantizationType);
	}

	Audio::FQuartzClockTickRate FQuartzClockProxy::GetTickRate() const
	{
		TSharedPtr<FQuartzClock, ESPMode::ThreadSafe> ClockPtr = ClockWeakPtr.Pin();
		if (!ClockPtr)
		{
			return {};
		}

		return ClockPtr->GetTickRate();
	}

	FQuartzTransportTimeStamp FQuartzClockProxy::GetCurrentClockTimestamp() const
	{
		TSharedPtr<FQuartzClock, ESPMode::ThreadSafe> ClockPtr = ClockWeakPtr.Pin();
		if (!ClockPtr)
		{
			return {};
		}

		return ClockPtr->GetCurrentTimestamp();
	}

	float FQuartzClockProxy::GetEstimatedClockRunTimeSeconds() const
	{
		TSharedPtr<FQuartzClock, ESPMode::ThreadSafe> ClockPtr = ClockWeakPtr.Pin();
		if (!ClockPtr)
		{
			return 0.f;
		}

		return ClockPtr->GetEstimatedRunTime();
	}


	bool FQuartzClockProxy::SendCommandToClock(TFunction<void(FQuartzClock*)> InCommand)
	{
		if (FQuartzClockCommandQueuePtr QueuePtr = SharedQueue.Pin())
		{
			QueuePtr->PushLambda<Quartz::IQuartzClock>(
				[Command = MoveTemp(InCommand)](Quartz::IQuartzClock& InListener)
				{
					Command(static_cast<FQuartzClock*>(&InListener));
				});

			return true;
		}

		return false;
	}

	// FQuartzClock Implementation
	FQuartzClock::FQuartzClock(const FName& InName, const FQuartzClockSettings& InClockSettings, FQuartzClockManager* InOwningClockManagerPtr)
		: Metronome(InClockSettings.TimeSignature, InName)
		, OwningClockManagerPtr(InOwningClockManagerPtr)
		, Name(InName)
		, bIsRunning(false)
		, bIgnoresFlush(InClockSettings.bIgnoreLevelChange)
	{
		FMixerDevice* MixerDevice = GetMixerDevice();

		if (MixerDevice)
		{
			Metronome.SetSampleRate(MixerDevice->GetSampleRate());
		}
		else
		{
			Metronome.SetSampleRate(HeadlessClockSampleRateCvar);
		}

		UpdateCachedState();
	}

	FQuartzClock::~FQuartzClock()
	{
		Shutdown();
	}

	void FQuartzClock::ChangeTickRate(FQuartzClockTickRate InNewTickRate, int32 NumFramesLeft)
	{
		FMixerDevice* MixerDevice = GetMixerDevice();

		if (MixerDevice)
		{
			InNewTickRate.SetSampleRate(MixerDevice->GetSampleRate());
		}
		else
		{
			InNewTickRate.SetSampleRate(HeadlessClockSampleRateCvar);
		}

		Metronome.SetTickRate(InNewTickRate, NumFramesLeft);
		FQuartzClockTickRate CurrentTickRate = Metronome.GetTickRate();

		// ratio between new and old rates
		const double Ratio = InNewTickRate.GetFramesPerTick() / CurrentTickRate.GetFramesPerTick();

		// adjust time-till-fire for existing commands
		for (auto& Command : PendingCommands)
		{
			if(Command.Command && !Command.Command->ShouldDeadlineIgnoresBpmChanges())
			{
				Command.NumFramesUntilExec = NumFramesLeft + Ratio * (Command.NumFramesUntilExec - NumFramesLeft);
			}
		}

		for (auto& Command : ClockAlteringPendingCommands)
		{
			if(Command.Command && !Command.Command->ShouldDeadlineIgnoresBpmChanges())
			{
				Command.NumFramesUntilExec = NumFramesLeft + Ratio * (Command.NumFramesUntilExec - NumFramesLeft);
			}
		}

		UpdateCachedState();
	}

	void FQuartzClock::ChangeTimeSignature(const FQuartzTimeSignature& InNewTimeSignature)
	{
		Metronome.SetTimeSignature(InNewTimeSignature);
		UpdateCachedState();
	}

	void FQuartzClock::Resume()
	{
		if (bIsRunning == false)
		{
			for (auto& Command : PendingCommands)
			{
				// Update countdown time to each quantized command
				Command.Command->OnClockStarted();
			}

			for (auto& Command : ClockAlteringPendingCommands)
			{
				// Update countdown time to each quantized command
				Command.Command->OnClockStarted();
			}
		}

		bIsRunning = true;
	}

	void FQuartzClock::Stop(bool CancelPendingEvents)
	{
		bIsRunning = false;
		Metronome.ResetTransport();
		TickDelayLengthInFrames = 0;

		if (CancelPendingEvents)
		{
			for (auto& Command : PendingCommands)
			{
				Command.Command->Cancel();
			}

			for (auto& Command : ClockAlteringPendingCommands)
			{
				Command.Command->Cancel();
			}

			PendingCommands.Reset();
			ClockAlteringPendingCommands.Reset();
		}
	}

	void FQuartzClock::Pause()
	{
		if (bIsRunning)
		{
			for (auto& Command : PendingCommands)
			{
				// Update countdown time to each quantized command
				Command.Command->OnClockPaused();
			}

			for (auto& Command : ClockAlteringPendingCommands)
			{
				// Update countdown time to each quantized command
				Command.Command->OnClockPaused();
			}
		}

		bIsRunning = false;
	}

	void FQuartzClock::Restart(bool bPause)
	{
		bIsRunning = !bPause;
		TickDelayLengthInFrames = 0;
	}

	void FQuartzClock::Shutdown()
	{
		for (PendingCommand& PendingCommand : PendingCommands)
		{
			PendingCommand.Command->Cancel();
		}

		for (PendingCommand& PendingCommand : ClockAlteringPendingCommands)
		{
			PendingCommand.Command->Cancel();
		}

		PendingCommands.Reset();
		ClockAlteringPendingCommands.Reset();
	}

	void FQuartzClock::LowResolutionTick(float InDeltaTimeSeconds)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(QuartzClock::Tick_LowRes);
		UE_LOGF(LogAudioQuartz, Verbose, "Quartz Clock Tick (low-res): %ls", *Name.ToString());
		PreTickCommands->PumpCommandQueue(*this);
		Tick(static_cast<int32>(InDeltaTimeSeconds * Metronome.GetTickRate().GetSampleRate()));
	}

	void FQuartzClock::Tick(int32 InNumFramesUntilNextTick)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(QuartzClock::Tick);
		TRACE_CPUPROFILER_EVENT_SCOPE(QuartzClock::GameThreadCommands);

		UE_LOGF(LogAudioQuartz, Verbose, "Quartz Clock Tick: %ls", *Name.ToString());

		PreTickCommands->PumpCommandQueue(*this);

		if (!bIsRunning)
		{
			return;
		}

		if (TickDelayLengthInFrames >= InNumFramesUntilNextTick)
		{
			TickDelayLengthInFrames -= InNumFramesUntilNextTick;
			return;
		}

		const int32 FramesOfLatency = (ThreadLatencyInMilliseconds / 1000) * Metronome.GetTickRate().GetSampleRate();
		int32 FramesToTick = InNumFramesUntilNextTick - TickDelayLengthInFrames;

        // commands executed in TickInternal may alter "TickDelayLengthInFrames" for the metronome's benefit
        // for the 2nd TickInternal() call we want to use the unmodified value (OriginalTickDelayLengthInFrames).
        const int32 OriginalTickDelayLengthInFrames = TickDelayLengthInFrames;
        TickInternal(FramesToTick, ClockAlteringPendingCommands, FramesOfLatency, OriginalTickDelayLengthInFrames);
        TickInternal(FramesToTick, PendingCommands, FramesOfLatency, OriginalTickDelayLengthInFrames);

		// FramesToTick may have been updated by TickInternal, recalculate
		FramesToTick = InNumFramesUntilNextTick - TickDelayLengthInFrames;
		Metronome.Tick(FramesToTick, FramesOfLatency);

		TickDelayLengthInFrames = 0;

		UpdateCachedState();
	}

	FQuartzClockCommandQueueWeakPtr FQuartzClock::GetCommandQueue() const
	{
		if (!PreTickCommands.IsValid())
		{
			PreTickCommands = MakeShared<FQuartzClockCommandQueueType>();
		}

		return PreTickCommands;
	}

	void FQuartzClock::TickInternal(int32 InNumFramesUntilNextTick, TArray<PendingCommand>& CommandsToTick, int32 FramesOfLatency, int32 FramesOfDelay)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(QuartzClock::TickInternal);
		bool bHaveCommandsToRemove = false;

		// Update all pending commands
		for (PendingCommand& PendingCommand : CommandsToTick)
		{
			// Time to notify game thread?
			if (PendingCommand.NumFramesUntilExec < FramesOfLatency)
			{
				PendingCommand.Command->AboutToStart();
			}

			// Time To execute?
			if (PendingCommand.NumFramesUntilExec < InNumFramesUntilNextTick)
			{
				PendingCommand.Command->OnFinalCallback(PendingCommand.NumFramesUntilExec + FramesOfDelay);
				PendingCommand.Command.Reset();
				bHaveCommandsToRemove = true;

			}
			else // not yet executing
			{
				PendingCommand.NumFramesUntilExec -= InNumFramesUntilNextTick;
				PendingCommand.Command->Update(PendingCommand.NumFramesUntilExec);
			}
		}

		// clean up executed commands
		if (bHaveCommandsToRemove)
		{
			for (int32 i = CommandsToTick.Num() - 1; i >= 0; --i)
			{
				if (!CommandsToTick[i].Command.IsValid())
				{
					CommandsToTick.RemoveAtSwap(i);
				}
			}
		}
	}

	void FQuartzClock::UpdateCachedState()
	{
		FScopeLock ScopeLock(&CachedClockStateCritSec);

		CachedClockState.TickRate = Metronome.GetTickRate();
		CachedClockState.TimeStamp = Metronome.GetTimeStamp();
		CachedClockState.RunTimeInSeconds = (float)Metronome.GetTimeSinceStart();

		const uint64 TempLastCacheTimestamp = CachedClockState.LastCacheTickCpuCycles64;
		CachedClockState.LastCacheTickCpuCycles64 = Metronome.GetLastTickCpuCycles64();
		CachedClockState.LastCacheTickDeltaCpuCycles64 = CachedClockState.LastCacheTickCpuCycles64 - TempLastCacheTimestamp;

		// copy previous phases (as temp values)
		FMemory::Memcpy(CachedClockState.MusicalDurationPhaseDeltas, CachedClockState.MusicalDurationPhases);
		
		// update current phases
		Metronome.CalculateDurationPhases(CachedClockState.MusicalDurationPhases);
		
		// convert temp copy to deltas
		constexpr int32 NumDurations = static_cast<int32>(EQuartzCommandQuantization::Count);
		for(int32 i = 0; i < NumDurations; ++i)
		{
			CachedClockState.MusicalDurationPhaseDeltas[i] = FMath::Wrap(CachedClockState.MusicalDurationPhases[i] - CachedClockState.MusicalDurationPhaseDeltas[i], 0.f, 1.f);
		}
	}

	void FQuartzClock::SetSampleRate(float InNewSampleRate)
	{
		if (FMath::IsNearlyEqual(InNewSampleRate, Metronome.GetTickRate().GetSampleRate()))
		{
			return;
		}

		// update Tick Rate
		Metronome.SetSampleRate(InNewSampleRate);

		UpdateCachedState();
	}

	bool FQuartzClock::IgnoresFlush() const
	{
		return bIgnoresFlush;
	}

	bool FQuartzClock::DoesMatchSettings(const FQuartzClockSettings& InClockSettings) const
	{
		return Metronome.GetTimeSignature() == InClockSettings.TimeSignature;
	}

	void FQuartzClock::SubscribeToTimeDivision(FQuartzGameThreadSubscriber InSubscriber, EQuartzCommandQuantization InQuantizationBoundary)
	{
		Metronome.SubscribeToTimeDivision(InSubscriber, InQuantizationBoundary);
	}

	void FQuartzClock::SubscribeToAllTimeDivisions(FQuartzGameThreadSubscriber InSubscriber)
	{
		Metronome.SubscribeToAllTimeDivisions(InSubscriber);
	}

	void FQuartzClock::UnsubscribeFromTimeDivision(FQuartzGameThreadSubscriber InSubscriber, EQuartzCommandQuantization InQuantizationBoundary)
	{
		Metronome.UnsubscribeFromTimeDivision(InSubscriber, InQuantizationBoundary);
	}

	void FQuartzClock::UnsubscribeFromAllTimeDivisions(FQuartzGameThreadSubscriber InSubscriber)
	{
		Metronome.UnsubscribeFromAllTimeDivisions(InSubscriber);
	}


	void FQuartzClock::AddQuantizedCommand(FQuartzQuantizationBoundary InQuantizationBondary, TSharedPtr<IQuartzQuantizedCommand> InNewEvent)
	{
		if (!ensure(InNewEvent.IsValid()))
		{
			return;
		}

		if (!bIsRunning && InQuantizationBondary.bCancelCommandIfClockIsNotRunning)
		{
			InNewEvent->Cancel();
			return;
		}

		if (InQuantizationBondary.bResetClockOnQueued)
		{
			Stop(/* clear pending events = */true);
			Restart(!bIsRunning);
		}

		if (!bIsRunning && InQuantizationBondary.bResumeClockOnQueued)
		{
			Resume();
		}

		int32 FramesUntilExec = 0;

		// if this is un-quantized, execute immediately (even if the clock is paused)
		if (InQuantizationBondary.Quantization == EQuartzCommandQuantization::None)
		{
			UE_LOGF(LogAudioQuartz, Verbose, "Quartz Command:(%ls) | Deadline (frames):[%i] | Boundary: [%ls]"
				, *InNewEvent->GetCommandName().ToString()
				, FramesUntilExec
				, *InQuantizationBondary.ToString()
				);
			
			InNewEvent->AboutToStart();
			InNewEvent->OnFinalCallback(0);
			return;
		}

		// get number of frames until event (assuming we are at frame 0)
		FramesUntilExec = FMath::RoundToInt(Metronome.GetFramesUntilBoundary(InQuantizationBondary)); // query metronome (round result to int)
		const int32 OverriddenFramesUntilExec = FMath::Max(0, InNewEvent->OverrideFramesUntilExec(FramesUntilExec)); // allow command to override the deadline (clamp result)
		const bool bOverridden = (FramesUntilExec != OverriddenFramesUntilExec);

		UE_LOGF(LogAudioQuartz, Verbose, "Quartz Command:(%ls) | Deadline (frames):[%i%ls] | Boundary: [%ls]"
			, *InNewEvent->GetCommandName().ToString()
			, OverriddenFramesUntilExec
			, bOverridden? *FString::Printf(TEXT("(overridden from %i)"), FramesUntilExec) : TEXT("")
			, *InQuantizationBondary.ToString()
			);

		// after the log, use tho Overridden value
		FramesUntilExec = OverriddenFramesUntilExec;

		// finalize the requested subscriber offsets and notify the command of their deadline
		InNewEvent->OnScheduled(Metronome.GetTickRate());
		InNewEvent->Update(FramesUntilExec);

		// if this is going to execute on the next tick, warn Game Thread Subscribers as soon as possible
		if (FramesUntilExec == 0)
		{
			InNewEvent->AboutToStart();
		}

		// add to pending commands list, execute OnQueued()
		if (InNewEvent->IsClockAltering())
		{
			ClockAlteringPendingCommands.Emplace(PendingCommand(MoveTemp(InNewEvent), FramesUntilExec));
		}
		else
		{
			PendingCommands.Emplace(PendingCommand(MoveTemp(InNewEvent), FramesUntilExec));
		}
	}

	bool FQuartzClock::CancelQuantizedCommand(TSharedPtr<IQuartzQuantizedCommand> InCommandPtr)
	{
		if (InCommandPtr->IsClockAltering())
		{
			return CancelQuantizedCommandInternal(InCommandPtr, ClockAlteringPendingCommands);
		}

		return CancelQuantizedCommandInternal(InCommandPtr, PendingCommands);
	}

	bool FQuartzClock::HasPendingEvents() const
	{
		// if container has any events in it.
		return (NumPendingEvents() > 0);
	}

	int32 FQuartzClock::NumPendingEvents() const
	{
		return PendingCommands.Num() + ClockAlteringPendingCommands.Num();
	}

	bool FQuartzClock::IsRunning() const
	{
		return bIsRunning;
	}

	float FQuartzClock::GetDurationOfQuantizationTypeInSeconds(const EQuartzCommandQuantization& QuantizationType, float Multiplier)
	{
		FScopeLock ScopeLock(&CachedClockStateCritSec);

		// if this is unquantized, return 0
		if (QuantizationType == EQuartzCommandQuantization::None)
		{
			return 0;
		}

		// get number of frames until the relevant quantization event
		double FramesUntilExec = CachedClockState.TickRate.GetFramesPerDuration(QuantizationType);

		//Translate frames to seconds
		double SampleRate = CachedClockState.TickRate.GetSampleRate();

		if (!FMath::IsNearlyZero(SampleRate))
		{
			return (FramesUntilExec * Multiplier) / SampleRate;
		}
		else //Handle potential divide by zero
		{
			return INDEX_NONE;
		}
	}

	float FQuartzClock::GetBeatProgressPercent(const EQuartzCommandQuantization& QuantizationType) const
	{
		if(CachedClockState.LastCacheTickDeltaCpuCycles64 == 0)
		{
			return CachedClockState.MusicalDurationPhases[static_cast<int32>(QuantizationType)];
		}

		// anticipate beat progress based on the amount of wall clock time that has passed since the last audio engine update
		const float LastPhase = CachedClockState.MusicalDurationPhases[static_cast<int32>(QuantizationType)];
		const float PhaseDelta = CachedClockState.MusicalDurationPhaseDeltas[static_cast<int32>(QuantizationType)];
		const uint64 CyclesSinceLastTick = FPlatformTime::Cycles64() - CachedClockState.LastCacheTickCpuCycles64;
		const float EstimatedPercentToNextTick = static_cast<float>(CyclesSinceLastTick) / static_cast<float>(CachedClockState.LastCacheTickDeltaCpuCycles64);

		return LastPhase + PhaseDelta * EstimatedPercentToNextTick;
	}

	FQuartzTransportTimeStamp FQuartzClock::GetCurrentTimestamp()
	{
		FScopeLock ScopeLock(&CachedClockStateCritSec);
		return CachedClockState.TimeStamp;
	}

	float FQuartzClock::GetEstimatedRunTime()
	{
		FScopeLock ScopeLock(&CachedClockStateCritSec);
		return CachedClockState.RunTimeInSeconds;
	}

	FMixerDevice* FQuartzClock::GetMixerDevice()
	{
		checkSlow(OwningClockManagerPtr);
		if (OwningClockManagerPtr)
		{
			return OwningClockManagerPtr->GetMixerDevice();
		}

		return nullptr;
	}

	void FQuartzClock::AddQuantizedCommand(FQuartzQuantizedRequestData& InQuantizedRequestData)
	{
		float SampleRate = HeadlessClockSampleRateCvar;
		if (FMixerDevice* MixerDevice = GetMixerDevice())
		{
			SampleRate = MixerDevice->GetSampleRate();
		}

		FQuartzQuantizedCommandInitInfo Info(InQuantizedRequestData, SampleRate);
		AddQuantizedCommand(Info);
	}

	void FQuartzClock::AddQuantizedCommand(FQuartzQuantizedCommandInitInfo& InQuantizationCommandInitInfo)
	{
		if (!ensure(InQuantizationCommandInitInfo.QuantizedCommandPtr))
		{
			return;
		}

		// this method can't be utilized by play commands because the AudioMixerSource needs a handle in order to stop it.
		// PlayCommands must be queued via the clock manager in AudioMixerSourceManager.
		if (!ensure(EQuartzCommandType::PlaySound != InQuantizationCommandInitInfo.QuantizedCommandPtr->GetCommandType()))
		{
			return;
		}

		// Can this command run without an Audio Device?
		FMixerDevice* MixerDevice = GetMixerDevice();
		if (!MixerDevice && InQuantizationCommandInitInfo.QuantizedCommandPtr->RequiresAudioDevice())
		{
			InQuantizationCommandInitInfo.QuantizedCommandPtr->Cancel();
		}

		// this function is a friend of FQuartzClockManager, so we can use FindClock() directly
		// to access the shared ptr to "this"
		InQuantizationCommandInitInfo.SetOwningClockPtr(GetClockManager()->FindClock(GetName()));
		InQuantizationCommandInitInfo.QuantizedCommandPtr->OnQueued(InQuantizationCommandInitInfo);
		AddQuantizedCommand(InQuantizationCommandInitInfo.QuantizationBoundary, InQuantizationCommandInitInfo.QuantizedCommandPtr);
	}

	FMixerSourceManager* FQuartzClock::GetSourceManager()
	{
		FMixerDevice* MixerDevice = GetMixerDevice();

		checkSlow(MixerDevice);
		if (MixerDevice)
		{
			return MixerDevice->GetSourceManager();
		}

		return nullptr;
	}

	FQuartzClockTickRate FQuartzClock::GetTickRate()
	{
		FScopeLock ScopeLock(&CachedClockStateCritSec);
		return CachedClockState.TickRate;
	}

	FName FQuartzClock::GetName() const
	{
		return Name;
	}

	FQuartzClockManager* FQuartzClock::GetClockManager()
	{
		checkSlow(OwningClockManagerPtr);
		if (OwningClockManagerPtr)
		{
			return OwningClockManagerPtr;
		}
		return nullptr;
	}

	void FQuartzClock::ResetTransport(const int32 NumFramesToTickBeforeReset)
	{
		if (NumFramesToTickBeforeReset != 0)
		{
			Metronome.Tick(NumFramesToTickBeforeReset);
		}
		
		Metronome.ResetTransport();
	}

	void FQuartzClock::AddToTickDelay(int32 NumFramesOfDelayToAdd)
	{
		TickDelayLengthInFrames += NumFramesOfDelayToAdd;
	}

	void FQuartzClock::SetTickDelay(int32 NumFramesOfDelay)
	{
		TickDelayLengthInFrames = NumFramesOfDelay;
	}

	bool FQuartzClock::CancelQuantizedCommandInternal(TSharedPtr<IQuartzQuantizedCommand> InCommandPtr, TArray<PendingCommand>& CommandsToTick)
	{
		for (int32 i = CommandsToTick.Num() - 1; i >= 0; --i)
		{
			PendingCommand& PendingCommand = CommandsToTick[i];

			if (PendingCommand.Command == InCommandPtr)
			{
				PendingCommand.Command->Cancel();
				CommandsToTick.RemoveAtSwap(i);
				return true;
			}
		}

		return false;
	}
} // namespace Audio
