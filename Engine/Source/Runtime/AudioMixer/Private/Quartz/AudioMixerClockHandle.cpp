// Copyright Epic Games, Inc. All Rights Reserved.

#include "Quartz/AudioMixerClockHandle.h"
#include "Sound/QuartzQuantizationUtilities.h"
#include "AudioDevice.h"
#include "AudioMixerDevice.h"
#include "Engine/GameInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioMixerClockHandle)



// Clock Handle implementation
UQuartzClockHandle::UQuartzClockHandle()
{
}

UQuartzClockHandle::~UQuartzClockHandle()
{
}

void UQuartzClockHandle::BeginDestroy()
{
	Super::BeginDestroy();

	auto Subscriber = GetQuartzSubscriber();
	RawHandle.SendCommandToClock([Subscriber](Audio::FQuartzClock* InClock) { InClock->UnsubscribeFromAllTimeDivisions(Subscriber); });
}

bool UQuartzClockHandle::ShouldUnsubscribe()
{
	return !RawHandle.IsValid();
}

void UQuartzClockHandle::StartClock(const UObject* WorldContextObject, UQuartzClockHandle*& ClockHandle)
{
	ClockHandle = this;
	ResumeClock(WorldContextObject, ClockHandle);
}

void UQuartzClockHandle::StopClock(const UObject* WorldContextObject, bool bCancelPendingEvents, UQuartzClockHandle*& ClockHandle)
{
	ClockHandle = this;
	RawHandle.SendCommandToClock([bCancelPendingEvents](Audio::FQuartzClock* InClock) { InClock->Stop(bCancelPendingEvents); });
}

void UQuartzClockHandle::PauseClock(const UObject* WorldContextObject, UQuartzClockHandle*& ClockHandle)
{
	ClockHandle = this;
	RawHandle.SendCommandToClock([](Audio::FQuartzClock* InClock) { InClock->Pause(); });
}

// Begin BP interface
void UQuartzClockHandle::ResumeClock(const UObject* WorldContextObject, UQuartzClockHandle*& ClockHandle)
{
	ClockHandle = this;
	RawHandle.SendCommandToClock([](Audio::FQuartzClock* InClock) { InClock->Resume(); });
}

void UQuartzClockHandle::QueueQuantizedSound(const UObject* WorldContextObject, UQuartzClockHandle*& InClockHandle, const FAudioComponentCommandInfo& InAudioComponentData, const FOnQuartzCommandEventBP& InDelegate, const FQuartzQuantizationBoundary& InTargetBoundary)
{
	InClockHandle = this;
	FName ClockName = GetClockName();

	//Create a Queue Command, and give it the additional data that it needs
	TSharedPtr<Audio::FQuantizedQueueCommand> QueueCommandPtr = MakeShared<Audio::FQuantizedQueueCommand>();
	QueueCommandPtr->SetQueueCommand(InAudioComponentData);

	//Set up initial command info
	Audio::FQuartzQuantizedRequestData CommandInitInfo = UQuartzSubsystem::CreateRequestDataForSchedulePlaySound(InClockHandle, InDelegate, InTargetBoundary);

	//(Queue's setup is identical to PlaySound except for the command ptr, so fix that here)
	CommandInitInfo.QuantizedCommandPtr.Reset();
	CommandInitInfo.QuantizedCommandPtr = QueueCommandPtr;

	RawHandle.SendCommandToClock([CommandInitInfo](Audio::FQuartzClock* InClock) mutable { InClock->AddQuantizedCommand(CommandInitInfo); });
}

// deprecated: use ResetTransportQuantized
void UQuartzClockHandle::ResetTransport(const UObject* WorldContextObject, const FOnQuartzCommandEventBP& InDelegate)
{
	Audio::FQuartzQuantizedRequestData Data(UQuartzSubsystem::CreateRequestDataForTransportReset(this, FQuartzQuantizationBoundary(EQuartzCommandQuantization::Bar), InDelegate));
	RawHandle.SendCommandToClock([Data](Audio::FQuartzClock* InClock) mutable { InClock->AddQuantizedCommand(Data); });
}

void UQuartzClockHandle::ResetTransportQuantized(const UObject* WorldContextObject, FQuartzQuantizationBoundary InQuantizationBoundary, const FOnQuartzCommandEventBP& InDelegate, UQuartzClockHandle*& ClockHandle)
{
	ClockHandle = this;
	Audio::FQuartzQuantizedRequestData Data(UQuartzSubsystem::CreateRequestDataForTransportReset(this, InQuantizationBoundary, InDelegate));
	RawHandle.SendCommandToClock([Data](Audio::FQuartzClock* InClock) mutable { InClock->AddQuantizedCommand(Data); });
}



bool UQuartzClockHandle::IsClockRunning(const UObject* WorldContextObject)
{
	return RawHandle.IsClockRunning();
}

void UQuartzClockHandle::NotifyOnQuantizationBoundary(const UObject* WorldContextObject, FQuartzQuantizationBoundary InQuantizationBoundary, const FOnQuartzCommandEventBP& InDelegate, float OffsetInMilliseconds)
{
	Audio::FQuartzQuantizedRequestData Data(UQuartzSubsystem::CreateRequestDataForQuantizedNotify(this, InQuantizationBoundary, InDelegate, OffsetInMilliseconds));
	RawHandle.SendCommandToClock([Data](Audio::FQuartzClock* InClock) mutable { InClock->AddQuantizedCommand(Data); });
}

float UQuartzClockHandle::GetDurationOfQuantizationTypeInSeconds(const UObject* WorldContextObject, const EQuartzCommandQuantization& QuantizationType, float Multiplier)
{
	return RawHandle.GetDurationOfQuantizationTypeInSeconds(QuantizationType, Multiplier);
}

FQuartzTransportTimeStamp UQuartzClockHandle::GetCurrentTimestamp(const UObject* WorldContextObject)
{
	return RawHandle.GetCurrentClockTimestamp();
}

float UQuartzClockHandle::GetEstimatedRunTime(const UObject* WorldContextObject)
{
	return RawHandle.GetEstimatedClockRunTimeSeconds();
}

void UQuartzClockHandle::StartOtherClock(const UObject* WorldContextObject, FName OtherClockName, FQuartzQuantizationBoundary InQuantizationBoundary, const FOnQuartzCommandEventBP& InDelegate)
{
	if (OtherClockName == CurrentClockId)
	{
		UE_LOGF(LogAudioQuartz, Warning, "Clock: (%ls) is attempting to start itself on a quantization boundary.  Ignoring command", *CurrentClockId.ToString());
		return;
	}

	Audio::FQuartzQuantizedRequestData Data(UQuartzSubsystem::CreateRequestDataForStartOtherClock(this, OtherClockName, InQuantizationBoundary, InDelegate));
	RawHandle.SendCommandToClock([Data](Audio::FQuartzClock* InClock) mutable { InClock->AddQuantizedCommand(Data); });
}

// todo: Move the bulk of these functions to FQuartzTickableObject once lightweight clock handles are spun up.
void UQuartzClockHandle::SubscribeToQuantizationEvent(const UObject* WorldContextObject, EQuartzCommandQuantization InQuantizationBoundary, const FOnQuartzMetronomeEventBP& OnQuantizationEvent, UQuartzClockHandle*& ClockHandle)
{
	ClockHandle = this;

	if (InQuantizationBoundary == EQuartzCommandQuantization::None)
	{
		UE_LOGF(LogAudioQuartz, Warning, "Clock: (%ls) is attempting to subscribe to 'NONE' as a Quantization Boundary.  Ignoring request", *CurrentClockId.ToString());
		return;
	}

	AddMetronomeBpDelegate(InQuantizationBoundary, OnQuantizationEvent);

	auto Subscriber = GetQuartzSubscriber();
	RawHandle.SendCommandToClock([Subscriber, InQuantizationBoundary](Audio::FQuartzClock* InClock) { InClock->SubscribeToTimeDivision(Subscriber, InQuantizationBoundary); });
}

void UQuartzClockHandle::SubscribeToAllQuantizationEvents(const UObject* WorldContextObject, const FOnQuartzMetronomeEventBP& OnQuantizationEvent, UQuartzClockHandle*& ClockHandle)
{
	ClockHandle = this;

	for (int32 i = 0; i < static_cast<int32>(EQuartzCommandQuantization::Count) - 1; ++i)
	{
		AddMetronomeBpDelegate(static_cast<EQuartzCommandQuantization>(i), OnQuantizationEvent);
	}

	auto Subscriber = GetQuartzSubscriber();
	RawHandle.SendCommandToClock([Subscriber](Audio::FQuartzClock* InClock) { InClock->SubscribeToAllTimeDivisions(Subscriber); });
}

void UQuartzClockHandle::UnsubscribeFromTimeDivision(const UObject* WorldContextObject, EQuartzCommandQuantization InQuantizationBoundary, UQuartzClockHandle*& ClockHandle)
{
	ClockHandle = this;

	auto Subscriber = GetQuartzSubscriber();
	RawHandle.SendCommandToClock([Subscriber, InQuantizationBoundary](Audio::FQuartzClock* InClock) { InClock->UnsubscribeFromTimeDivision(Subscriber, InQuantizationBoundary); });
}

void UQuartzClockHandle::UnsubscribeFromAllTimeDivisions(const UObject* WorldContextObject, UQuartzClockHandle*& ClockHandle)
{
	ClockHandle = this;

	auto Subscriber = GetQuartzSubscriber();
	RawHandle.SendCommandToClock([Subscriber](Audio::FQuartzClock* InClock) { InClock->UnsubscribeFromAllTimeDivisions(Subscriber); });
}

// Metronome Alteration (setters)
void UQuartzClockHandle::SetMillisecondsPerTick(const UObject* WorldContextObject, const FQuartzQuantizationBoundary& InQuantizationBoundary, const FOnQuartzCommandEventBP& InDelegate, UQuartzClockHandle*& ClockHandle, float MillisecondsPerTick)
{
	ClockHandle = this;
	if (MillisecondsPerTick < 0 || FMath::IsNearlyZero(MillisecondsPerTick))
	{
		UE_LOGF(LogAudioQuartz, Warning, "Ignoring invalid request on Clock: %ls: MillisecondsPerTick was %f", *this->CurrentClockId.ToString(), MillisecondsPerTick);
		return;
	}

	Audio::FQuartzClockTickRate TickRate;
	TickRate.SetMillisecondsPerTick(MillisecondsPerTick);
	SetTickRateInternal(InQuantizationBoundary, InDelegate, TickRate);
}

void UQuartzClockHandle::SetTicksPerSecond(const UObject* WorldContextObject, const FQuartzQuantizationBoundary& InQuantizationBoundary, const FOnQuartzCommandEventBP& InDelegate, UQuartzClockHandle*& ClockHandle, float TicksPerSecond)
{
	ClockHandle = this;
	if (TicksPerSecond < 0 || FMath::IsNearlyZero(TicksPerSecond))
	{
		UE_LOGF(LogAudioQuartz, Warning, "Ignoring invalid request on Clock: %ls: TicksPerSecond was %f", *this->CurrentClockId.ToString(), TicksPerSecond);
		return;
	}

	Audio::FQuartzClockTickRate TickRate;
	TickRate.SetSecondsPerTick(1.f / TicksPerSecond);
	SetTickRateInternal(InQuantizationBoundary, InDelegate, TickRate);
}

void UQuartzClockHandle::SetSecondsPerTick(const UObject* WorldContextObject, const FQuartzQuantizationBoundary& InQuantizationBoundary, const FOnQuartzCommandEventBP& InDelegate, UQuartzClockHandle*& ClockHandle, float SecondsPerTick)
{
	ClockHandle = this;
	if (SecondsPerTick < 0 || FMath::IsNearlyZero(SecondsPerTick))
	{
		UE_LOGF(LogAudioQuartz, Warning, "Ignoring invalid request on Clock: %ls: SecondsPerTick was %f", *this->CurrentClockId.ToString(), SecondsPerTick);
		return;
	}

	Audio::FQuartzClockTickRate TickRate;
	TickRate.SetSecondsPerTick(SecondsPerTick);
	SetTickRateInternal(InQuantizationBoundary, InDelegate, TickRate);
}

void UQuartzClockHandle::SetThirtySecondNotesPerMinute(const UObject* WorldContextObject, const FQuartzQuantizationBoundary& InQuantizationBoundary, const FOnQuartzCommandEventBP& InDelegate, UQuartzClockHandle*& ClockHandle, float ThirtySecondsNotesPerMinute)
{
	ClockHandle = this;
	if (ThirtySecondsNotesPerMinute < 0 || FMath::IsNearlyZero(ThirtySecondsNotesPerMinute))
	{
		UE_LOGF(LogAudioQuartz, Warning, "Ignoring invalid request on Clock: %ls: ThirtySecondsNotesPerMinute was %f", *this->CurrentClockId.ToString(), ThirtySecondsNotesPerMinute);
		return;
	}

	Audio::FQuartzClockTickRate TickRate;
	TickRate.SetThirtySecondNotesPerMinute(ThirtySecondsNotesPerMinute);
	SetTickRateInternal(InQuantizationBoundary, InDelegate, TickRate);
}

void UQuartzClockHandle::SetBeatsPerMinute(const UObject* WorldContextObject, const FQuartzQuantizationBoundary& InQuantizationBoundary, const FOnQuartzCommandEventBP& InDelegate, UQuartzClockHandle*& ClockHandle, float BeatsPerMinute)
{
	ClockHandle = this;
	if (BeatsPerMinute < 0 || FMath::IsNearlyZero(BeatsPerMinute))
	{
		UE_LOGF(LogAudioQuartz, Warning, "Ignoring invalid request on Clock: %ls: BeatsPerMinute was %f", *this->CurrentClockId.ToString(), BeatsPerMinute);
		return;
	}

	Audio::FQuartzClockTickRate TickRate;
	TickRate.SetBeatsPerMinute(BeatsPerMinute);
	SetTickRateInternal(InQuantizationBoundary, InDelegate, TickRate);
}

void UQuartzClockHandle::SetTickRateInternal(const FQuartzQuantizationBoundary& InQuantizationBoundary, const FOnQuartzCommandEventBP& InDelegate, const Audio::FQuartzClockTickRate& NewTickRate)
{
	Audio::FQuartzQuantizedRequestData Data(UQuartzSubsystem::CreateRequestDataForTickRateChange(this, InDelegate, NewTickRate, InQuantizationBoundary));
	RawHandle.SendCommandToClock([Data](Audio::FQuartzClock* InClock) mutable { InClock->AddQuantizedCommand(Data); });
}

// Metronome getters
float UQuartzClockHandle::GetMillisecondsPerTick(const UObject* WorldContextObject) const
{
	Audio::FQuartzClockTickRate OutTickRate;

	if (GetCurrentTickRate(WorldContextObject, OutTickRate))
	{
		return OutTickRate.GetMillisecondsPerTick();
	}

	return 0.f;
}

float UQuartzClockHandle::GetTicksPerSecond(const UObject* WorldContextObject) const
{
	Audio::FQuartzClockTickRate OutTickRate;

	if (GetCurrentTickRate(WorldContextObject, OutTickRate))
	{
		const float SecondsPerTick = OutTickRate.GetSecondsPerTick();

		if (!FMath::IsNearlyZero(SecondsPerTick))
		{
			return 1.f / SecondsPerTick;
		}
	}

	return 0.f;
}

float UQuartzClockHandle::GetSecondsPerTick(const UObject* WorldContextObject) const
{
	Audio::FQuartzClockTickRate OutTickRate;

	if (GetCurrentTickRate(WorldContextObject, OutTickRate))
	{
		return OutTickRate.GetSecondsPerTick();
	}

	return 0.f;
}

float UQuartzClockHandle::GetThirtySecondNotesPerMinute(const UObject* WorldContextObject) const
{
	Audio::FQuartzClockTickRate OutTickRate;

	if (GetCurrentTickRate(WorldContextObject, OutTickRate))
	{
		return OutTickRate.GetThirtySecondNotesPerMinute();
	}

	return 0.f;
}

float UQuartzClockHandle::GetBeatsPerMinute(const UObject* WorldContextObject) const
{
	Audio::FQuartzClockTickRate OutTickRate;

	if (GetCurrentTickRate(WorldContextObject, OutTickRate))
	{
		return OutTickRate.GetBeatsPerMinute();
	}

	return 0.f;
}

float UQuartzClockHandle::GetBeatProgressPercent(EQuartzCommandQuantization QuantizationBoundary, float PhaseOffset, float MsOffset)
{
	if(RawHandle.IsValid() && QuantizationBoundary != EQuartzCommandQuantization::None)
	{
		constexpr float ToMilliseconds = 1000.f;
	    const float MsInQuantizationType = ToMilliseconds * RawHandle.GetDurationOfQuantizationTypeInSeconds(QuantizationBoundary, 1.f);
	    if(!FMath::IsNearlyZero(MsInQuantizationType))
	    {
		    PhaseOffset += MsOffset / MsInQuantizationType;
	    }

		return FMath::Wrap(PhaseOffset + RawHandle.GetBeatProgressPercent(QuantizationBoundary), 0.f, 1.f);
	}

	return 0.f;
}

// todo: un-comment when metronome events support the offset
// void UQuartzClockHandle::SetNotificationAnticipationAmountInMilliseconds(const UObject* WorldContextObject, UQuartzClockHandle*& ClockHandle, const double Milliseconds)
// {
// 	ClockHandle = this;
// 	if(Milliseconds < 0.0)
// 	{
// 		UE_LOGF(LogAudioQuartz, Warning, "Setting a negative notification anticipation amount is not supported. (request ignored)");
// 		return;
// 	}
//
// 	SetNotificationAnticipationAmountMilliseconds(Milliseconds);
// }
//
//
// void UQuartzClockHandle::SetNotificationAnticipationAmountAsMusicalDuration(const UObject* WorldContextObject, UQuartzClockHandle*& ClockHandle, const EQuartzCommandQuantization MusicalDuration, const double Multiplier)
// {
// 	ClockHandle = this;
// 	if(Multiplier < 0.0)
// 	{
// 		UE_LOGF(LogAudioQuartz, Warning, "Setting a negative notification anticipation amount is not supported. (request ignored)");
// 		return;
// 	}
//
// 	SetNotificationAnticipationAmountMusicalDuration(MusicalDuration, Multiplier);
// }

// End BP interface


UQuartzClockHandle* UQuartzClockHandle::SubscribeToClock(const UObject* WorldContextObject, FName ClockName, Audio::FQuartzClockProxy const* InHandlePtr)
{
	CurrentClockId = ClockName;

	if (InHandlePtr)
	{
		RawHandle = *InHandlePtr;
	}

	return this;
}


// returns true if OutTickRate is valid and was updated
bool UQuartzClockHandle::GetCurrentTickRate(const UObject* WorldContextObject, Audio::FQuartzClockTickRate& OutTickRate) const
{
	if (RawHandle.IsValid())
	{
		OutTickRate = RawHandle.GetTickRate();
		return true;
	}

	OutTickRate = {};
	return false;
}

