// Copyright Epic Games, Inc. All Rights Reserved.

#include "Quartz/AudioMixerClockManager.h"
#include "AudioMixerDevice.h"
#include "Misc/ScopeLock.h"
#include "ProfilingDebugging/CountersTrace.h"

namespace Audio
{
	FQuartzClockManager::FQuartzClockManager(Audio::FMixerDevice* InOwner)
	: MixerDevice(InOwner)
	{
	}

	FQuartzClockManager::~FQuartzClockManager()
	{
		check(ActiveClocks.Num() == 0);
	}

	void FQuartzClockManager::Update(int32 NumFramesUntilNextUpdate)
	{
		// if this is owned by a MixerDevice, this function should only be called on the Audio Render Thread
		TRACE_CPUPROFILER_EVENT_SCOPE(QuartzClockManager::Update)
		if (MixerDevice)
		{
			check(MixerDevice->IsAudioRenderingThread());
		}

		LastUpdateSizeInFrames = NumFramesUntilNextUpdate;
		TickClocks(NumFramesUntilNextUpdate);
	}

	void FQuartzClockManager::LowResoultionUpdate(float DeltaTimeSeconds)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(QuartzClockManager::Update_LowRes)
		FScopeLock Lock(&ActiveClockCritSec);

		for (auto& Clock : ActiveClocks)
		{
			Clock->LowResolutionTick(DeltaTimeSeconds);
		}
	}

	void FQuartzClockManager::UpdateClock(FName InClockToAdvance, int32 NumFramesToAdvance)
	{
		FScopeLock Lock(&ActiveClockCritSec);
		TSharedPtr<FQuartzClock> ClockPtr = FindClock(InClockToAdvance);

		if (ClockPtr.IsValid())
		{
			ClockPtr->Tick(NumFramesToAdvance);
		}
	}

	FQuartzClockProxy FQuartzClockManager::GetOrCreateClock(const FName& InClockName, const FQuartzClockSettings& InClockSettings, bool bOverrideTickRateIfClockExists)
	{
		FScopeLock Lock(&ActiveClockCritSec);

		// make a copy of the Settings
		FQuartzClockSettings NewSettings = InClockSettings;

		// See if this clock already exists
		TSharedPtr<FQuartzClock> Clock = FindClock(InClockName);

		if (Clock)
		{
			if (bOverrideTickRateIfClockExists && !Clock->DoesMatchSettings(NewSettings))
			{
				UE_LOGF(LogAudioQuartz, Display, "Overriding Tick Rate on Clock: %ls", *Clock->GetName().ToString());
				Clock->ChangeTimeSignature(NewSettings.TimeSignature);
			}

			return FQuartzClockProxy(Clock);
		}

		// doesn't exist, create new clock
		return FQuartzClockProxy(ActiveClocks.Emplace_GetRef(MakeShared<FQuartzClock>(InClockName, NewSettings, this)));
	}

	FQuartzClockProxy FQuartzClockManager::GetClock(const FName& InClockName)
	{
		FScopeLock Lock(&ActiveClockCritSec);
		TSharedPtr<FQuartzClock> ClockPtr = FindClock(InClockName);

		if (ClockPtr)
		{
			return FQuartzClockProxy(ClockPtr);
		}

		UE_LOGF(LogAudioQuartz, Warning, "Could not find Clock: %ls (returning empty handle)", *ClockPtr->GetName().ToString());
		return {};
	}

	bool FQuartzClockManager::DoesClockExist(const FName& InClockName)
	{
		FScopeLock Lock(&ActiveClockCritSec);
		return !!FindClock(InClockName);
	}

	bool FQuartzClockManager::IsClockRunning(const FName& InClockName)
	{
		FScopeLock Lock(&ActiveClockCritSec);

		// See if this clock already exists
		TSharedPtr<FQuartzClock> Clock = FindClock(InClockName);

		if (Clock)
		{
			return Clock->IsRunning();
		}

		// clock doesn't exist
		return false;
	}

	float FQuartzClockManager::GetDurationOfQuantizationTypeInSeconds(const FName& InClockName, const EQuartzCommandQuantization& QuantizationType, float Multiplier)
	{
		FScopeLock Lock(&ActiveClockCritSec);

		// See if this clock already exists
		TSharedPtr<FQuartzClock> Clock = FindClock(InClockName);

		if (Clock)
		{
			return Clock->GetDurationOfQuantizationTypeInSeconds(QuantizationType, Multiplier);
		}

		//Clock doesn't exist
		return INDEX_NONE;
	}

	FQuartzTransportTimeStamp FQuartzClockManager::GetCurrentTimestamp(const FName& InClockName)
	{
		FScopeLock Lock(&ActiveClockCritSec);

		// See if this clock already exists
		TSharedPtr<FQuartzClock> Clock = FindClock(InClockName);

		if (Clock)
		{
			return Clock->GetCurrentTimestamp();
		}

		//Clock doesn't exist
		return FQuartzTransportTimeStamp();
	}

	float FQuartzClockManager::GetEstimatedRunTime(const FName& InClockName)
	{
		FScopeLock Lock(&ActiveClockCritSec);

		// See if this clock already exists
		TSharedPtr<FQuartzClock> Clock = FindClock(InClockName);

		if (Clock)
		{
			return Clock->GetEstimatedRunTime();
		}

		//Clock doesn't exist
		return INDEX_NONE;
	}

	void FQuartzClockManager::RemoveClock(const FName& InName, bool bForceSynchronous)
	{
		if (!bForceSynchronous && MixerDevice && !MixerDevice->IsAudioRenderingThread())
		{
			MixerDevice->AudioRenderThreadCommand([this, InName]()
			{
				RemoveClock(InName);
			});

			return;
		}

		// Anything below is being executed on the Audio Render Thread
		FScopeLock Lock(&ActiveClockCritSec);
		int32 NumClocks = ActiveClocks.Num();
		for (int32 i = NumClocks - 1; i >= 0; --i)
		{
			if (ActiveClocks[i]->GetName() == InName)
			{
				UE_LOGF(LogAudioQuartz, Verbose, "Removing Clock: %ls", *InName.ToString());
				ActiveClocks.RemoveAtSwap(i);
			}
		}

	}

	FQuartzClockTickRate FQuartzClockManager::GetTickRateForClock(const FName& InName)
	{
		FScopeLock Lock(&ActiveClockCritSec);

		TSharedPtr<FQuartzClock> Clock = FindClock(InName);

		if (Clock)
		{
			return Clock->GetTickRate();
		}

		return FQuartzClockTickRate();
	}

	void FQuartzClockManager::SetTickRateForClock(const FQuartzClockTickRate& InNewTickRate, const FName& InName)
	{
		if (MixerDevice && !MixerDevice->IsAudioRenderingThread())
		{
			MixerDevice->AudioRenderThreadCommand([this, InNewTickRate, InName]()
			{
				SetTickRateForClock(InNewTickRate, InName);
			});

			return;
		}

		// Anything below is being executed on the Audio Render Thread
		FScopeLock Lock(&ActiveClockCritSec);
		TSharedPtr<FQuartzClock> Clock = FindClock(InName);
		if (Clock)
		{
			Clock->ChangeTickRate(InNewTickRate);
		}
	}

	void FQuartzClockManager::ResumeClock(const FName& InName, int32 NumFramesToDelayStart)
	{
		if (MixerDevice && !MixerDevice->IsAudioRenderingThread())
		{
			MixerDevice->AudioRenderThreadCommand([this, InName]()
			{
				ResumeClock(InName);
			});

			return;
		}

		// Anything below is being executed on the Audio Render Thread
		FScopeLock Lock(&ActiveClockCritSec);
		TSharedPtr<FQuartzClock> Clock = FindClock(InName);
		if (Clock)
		{
			Clock->AddToTickDelay(NumFramesToDelayStart);
			Clock->Resume();
		}
	}

	void FQuartzClockManager::StopClock(const FName& InName, bool CancelPendingEvents)
	{
		if (MixerDevice && !MixerDevice->IsAudioRenderingThread())
		{
			MixerDevice->AudioRenderThreadCommand([this, InName, CancelPendingEvents]()
			{
				StopClock(InName, CancelPendingEvents);
			});

			return;
		}

		// Anything below is being executed on the Audio Render Thread
		FScopeLock Lock(&ActiveClockCritSec);
		TSharedPtr<FQuartzClock> Clock = FindClock(InName);
		if (Clock)
		{
			Clock->Stop(CancelPendingEvents);
		}
	}

	void FQuartzClockManager::PauseClock(const FName& InName)
	{
		if (MixerDevice && !MixerDevice->IsAudioRenderingThread())
		{
			MixerDevice->AudioRenderThreadCommand([this, InName]()
			{
				PauseClock(InName);
			});

			return;
		}

		// Anything below is being executed on the Audio Render Thread
		FScopeLock Lock(&ActiveClockCritSec);
		TSharedPtr<FQuartzClock> Clock = FindClock(InName);
		if (Clock)
		{
			Clock->Pause();
		}
	}

	void FQuartzClockManager::Flush()
	{
		FScopeLock Lock(&ActiveClockCritSec);

		int32 NumClocks = ActiveClocks.Num();

		for (int32 i = NumClocks - 1; i >= 0; --i)
		{
			if (!MixerDevice || !ActiveClocks[i]->IgnoresFlush())
			{
				ActiveClocks.RemoveAtSwap(i);
			}
		}
	}

	void FQuartzClockManager::Shutdown()
	{
		FScopeLock Lock(&ActiveClockCritSec);
		ActiveClocks.Reset();
	}

	FQuartzQuantizedCommandHandle FQuartzClockManager::AddCommandToClock(FQuartzQuantizedCommandInitInfo& InQuantizationCommandInitInfo)
	{
		if (!ensure(InQuantizationCommandInitInfo.QuantizedCommandPtr))
		{
			return {};
		}

		FScopeLock Lock(&ActiveClockCritSec);

		// Can this command run without an Audio Device?
		if (!MixerDevice && InQuantizationCommandInitInfo.QuantizedCommandPtr->RequiresAudioDevice())
		{
			InQuantizationCommandInitInfo.QuantizedCommandPtr->Cancel();
		}
		// does the target clock exist?
		else if (TSharedPtr<FQuartzClock> Clock = FindClock(InQuantizationCommandInitInfo.ClockName))
		{
			// pass the quantized command to it's clock
			InQuantizationCommandInitInfo.SetOwningClockPtr(Clock);
			InQuantizationCommandInitInfo.QuantizedCommandPtr->OnQueued(InQuantizationCommandInitInfo);
			Clock->AddQuantizedCommand(InQuantizationCommandInitInfo.QuantizationBoundary, InQuantizationCommandInitInfo.QuantizedCommandPtr);

			// initialize the handle the audio source can use to cancel this quantized command
			FQuartzQuantizedCommandHandle Handle;
			Handle.OwningClockName = InQuantizationCommandInitInfo.ClockName;
			Handle.CommandPtr = InQuantizationCommandInitInfo.QuantizedCommandPtr;
			Handle.MixerDevice = MixerDevice;

			return Handle;
		}

		return {};
	}

	void FQuartzClockManager::SubscribeToTimeDivision(FName InClockName, MetronomeCommandQueuePtr InListenerQueue, EQuartzCommandQuantization InQuantizationBoundary)
	{
		if (MixerDevice && !MixerDevice->IsAudioRenderingThread())
		{
			MixerDevice->AudioRenderThreadCommand([this, InClockName, InListenerQueue, InQuantizationBoundary]()
			{
				SubscribeToTimeDivision(InClockName, InListenerQueue, InQuantizationBoundary);
			});

			return;
		}

		// Anything below is being executed on the Audio Render Thread
		FScopeLock Lock(&ActiveClockCritSec);
		TSharedPtr<FQuartzClock> Clock = FindClock(InClockName);
		if (Clock)
		{
			Clock->SubscribeToTimeDivision(FQuartzGameThreadSubscriber(InListenerQueue), InQuantizationBoundary);
		}
	}

	void FQuartzClockManager::SubscribeToAllTimeDivisions(FName InClockName, MetronomeCommandQueuePtr InListenerQueue)
	{
		if (MixerDevice && !MixerDevice->IsAudioRenderingThread())
		{
			MixerDevice->AudioRenderThreadCommand([this, InClockName, InListenerQueue]()
			{
				SubscribeToAllTimeDivisions(InClockName, InListenerQueue);
			});

			return;
		}

		// Anything below is being executed on the Audio Render Thread
		FScopeLock Lock(&ActiveClockCritSec);
		TSharedPtr<FQuartzClock> Clock = FindClock(InClockName);
		if (Clock)
		{
			Clock->SubscribeToAllTimeDivisions(FQuartzGameThreadSubscriber(InListenerQueue));
		}
	}

	void FQuartzClockManager::UnsubscribeFromTimeDivision(FName InClockName, MetronomeCommandQueuePtr InListenerQueue, EQuartzCommandQuantization InQuantizationBoundary)
	{
		if (MixerDevice && !MixerDevice->IsAudioRenderingThread())
		{
			MixerDevice->AudioRenderThreadCommand([this, InClockName, InListenerQueue, InQuantizationBoundary]()
			{
				UnsubscribeFromTimeDivision(InClockName, InListenerQueue, InQuantizationBoundary);
			});

			return;
		}

		// Anything below is being executed on the Audio Render Thread
		FScopeLock Lock(&ActiveClockCritSec);
		TSharedPtr<FQuartzClock> Clock = FindClock(InClockName);
		if (Clock)
		{
			Clock->UnsubscribeFromTimeDivision(FQuartzGameThreadSubscriber(InListenerQueue), InQuantizationBoundary);
		}
	}

	void FQuartzClockManager::UnsubscribeFromAllTimeDivisions(FName InClockName, MetronomeCommandQueuePtr InListenerQueue)
	{
		if (MixerDevice && !MixerDevice->IsAudioRenderingThread())
		{
			MixerDevice->AudioRenderThreadCommand([this, InClockName, InListenerQueue]()
			{
				UnsubscribeFromAllTimeDivisions(InClockName, InListenerQueue);
			});

			return;
		}

		// Anything below is being executed on the Audio Render Thread
		FScopeLock Lock(&ActiveClockCritSec);
		TSharedPtr<FQuartzClock> Clock = FindClock(InClockName);
		if (Clock)
		{
			Clock->UnsubscribeFromAllTimeDivisions(FQuartzGameThreadSubscriber(InListenerQueue));
		}
	}

	bool FQuartzClockManager::CancelCommandOnClock(FName InOwningClockName, TSharedPtr<IQuartzQuantizedCommand> InCommandPtr)
	{
		// This function should only be called on the Audio Render Thread
		if (MixerDevice)
		{
			check(MixerDevice->IsAudioRenderingThread());
		}

		FScopeLock Lock(&ActiveClockCritSec);
		TSharedPtr<FQuartzClock> Clock = FindClock(InOwningClockName);

		if (Clock && InCommandPtr)
		{
			return Clock->CancelQuantizedCommand(InCommandPtr);
		}

		return false;
	}

	bool FQuartzClockManager::HasClockBeenTickedThisUpdate(FName InClockName)
	{
		FScopeLock Lock(&ActiveClockCritSec);
		int32 NumClocks = ActiveClocks.Num();

		for (int32 i = 0; i < NumClocks; ++i)
		{
			TSharedPtr<FQuartzClock> ClockPtr = ActiveClocks[i];
			check(ClockPtr.IsValid());

			if (ClockPtr->GetName() == InClockName)
			{
				// if this clock is earlier in the array than the last clock we ticked,
				// then it has already been ticked this update
				if (i < LastClockTickedIndex.GetValue())
				{
					return true;
				}

				return false;
			}
		}

		return false;
	}

	FMixerDevice* FQuartzClockManager::GetMixerDevice() const
	{
		return MixerDevice;
	}

	void FQuartzClockManager::TickClocks(int32 NumFramesToTick)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(QuartzClockManager::TickClocks);
		int32 TotalNumPendingCommands = 0;

		if (MixerDevice)
		{
			// This function should only be called on the Audio Render Thread
			check(MixerDevice->IsAudioRenderingThread());
		}

		FScopeLock Lock(&ActiveClockCritSec);
		for (auto& Clock : ActiveClocks)
		{
			TotalNumPendingCommands += Clock->NumPendingEvents();
			Clock->Tick(NumFramesToTick);
			LastClockTickedIndex.Increment();
		}

		TRACE_INT_VALUE(TEXT("QuartzClockManager::NumActiveClocks"), ActiveClocks.Num());
		TRACE_INT_VALUE(TEXT("QuartzClockManager::NumTotalPendingCommands"), TotalNumPendingCommands);

		LastClockTickedIndex.Reset();
	}

	TSharedPtr<FQuartzClock> FQuartzClockManager::FindClock(const FName& InName)
	{
		FScopeLock Lock(&ActiveClockCritSec);
		for (auto& Clock : ActiveClocks)
		{
			if (Clock->GetName() == InName)
			{
				return Clock;
			}
		}

		// didn't exist
		return nullptr;
	}
} // namespace Audio
