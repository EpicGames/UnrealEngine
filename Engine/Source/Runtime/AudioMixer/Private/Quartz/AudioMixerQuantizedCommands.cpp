// Copyright Epic Games, Inc. All Rights Reserved.

#include "Quartz/AudioMixerQuantizedCommands.h"
#include "AudioMixerSourceManager.h"

namespace Audio
{
	
	FQuantizedPlayCommand::FQuantizedPlayCommand()
	{
	}

	TSharedPtr<IQuartzQuantizedCommand> FQuantizedPlayCommand::GetDeepCopyOfDerivedObject() const
	{
		TSharedPtr<FQuantizedPlayCommand> NewCopy = MakeShared<FQuantizedPlayCommand>();

		NewCopy->OwningClockPtr = OwningClockPtr;
		NewCopy->SourceID = SourceID;

		return NewCopy;
	}

	void FQuantizedPlayCommand::OnQueuedCustom(const FQuartzQuantizedCommandInitInfo& InCommandInitInfo)
	{
		OwningClockPtr = InCommandInitInfo.OwningClockPointer;
		SourceID = InCommandInitInfo.SourceID;
		bIsCanceled = false;

		// access source manager through owning clock (via clock manager)
		FMixerSourceManager* SourceManager = OwningClockPtr->GetSourceManager();
		if (SourceManager)
		{
			SourceManager->PauseSoundForQuantizationCommand(SourceID);
		}
		else
		{
			// cancel ourselves (no source manager may mean we are running without an audio device)
			if (ensure(OwningClockPtr))
			{
				OwningClockPtr->CancelQuantizedCommand(TSharedPtr<IQuartzQuantizedCommand>(this));
			}
		}
		
	}

	// TODO: think about playback progress of a sound source
	// TODO: AudioComponent "waiting to play" state (cancel-able)
	void FQuantizedPlayCommand::OnFinalCallbackCustom(int32 InNumFramesLeft)
	{
		// Access source manager through owning clock (via clock manager)
		check(OwningClockPtr && OwningClockPtr->GetSourceManager());

		// This was canceled before the active sound hit the source manager.
		// Calling CancelCustom() make sure we stop the associated sound.
		if (bIsCanceled)
		{
			CancelCustom();
			return;
		}

		// access source manager through owning clock (via clock manager)
		// Owning Clock Ptr may be nullptr if this command was canceled.
		if (OwningClockPtr)
		{
			FMixerSourceManager* SourceManager = OwningClockPtr->GetSourceManager();
			if (SourceManager)
			{
				SourceManager->SetSubBufferDelayForSound(SourceID, InNumFramesLeft);
				SourceManager->UnPauseSoundForQuantizationCommand(SourceID);
			}
			else
			{
				// cancel ourselves (no source manager may mean we are running without an audio device)
				OwningClockPtr->CancelQuantizedCommand(TSharedPtr<IQuartzQuantizedCommand>(this));
			}
		}

	}

	void FQuantizedPlayCommand::CancelCustom()
	{
		bIsCanceled = true;

		if (OwningClockPtr)
		{
			FMixerSourceManager* SourceManager = OwningClockPtr->GetSourceManager();
			FMixerDevice* MixerDevice = OwningClockPtr->GetMixerDevice();

			if (MixerDevice && SourceManager && MixerDevice->IsAudioRenderingThread())
			{
				// if we don't UnPause first, this function will be called by FMixerSourceManager::StopInternal()
				SourceManager->UnPauseSoundForQuantizationCommand(SourceID); // (avoid infinite recursion)
				SourceManager->CancelQuantizedSound(SourceID);
			}
		}
	}

	static const FName PlayCommandName("Play Command");
	FName FQuantizedPlayCommand::GetCommandName() const
	{
		return PlayCommandName;
	}

	int32 FQuantizedPlayCommand::OverrideFramesUntilExec(int32 NumFramesUntilExec)
	{
//		if (OwningClockPtr)
//		{
//				if(const FMixerDevice* MixerDevice = OwningClockPtr->GetMixerDevice())
//				{
//					return NumFramesUntilExec + 2 * MixerDevice->GetBufferLength();
//				}
//		}
//		
		return NumFramesUntilExec;
	}

	void FQuantizedQueueCommand::SetQueueCommand(const FAudioComponentCommandInfo& InAudioComponentData)
	{
		AudioComponentData = InAudioComponentData;
	}

	TSharedPtr<IQuartzQuantizedCommand> FQuantizedQueueCommand::GetDeepCopyOfDerivedObject() const
	{
		return MakeShared<FQuantizedQueueCommand>(*this);
	}

	void FQuantizedQueueCommand::OnQueuedCustom(const FQuartzQuantizedCommandInitInfo& InCommandInitInfo)
	{
		OwningClockPtr = InCommandInitInfo.OwningClockPointer;
	}

	int32 FQuantizedQueueCommand::OverrideFramesUntilExec(int32 NumFramesUntilExec)
	{
		// Calculate the amount of time before taking up a voice slot
		int32 NumFramesBeforeVoiceSlot = NumFramesUntilExec - static_cast<int32>(OwningClockPtr->GetTickRate().GetFramesPerDuration(AudioComponentData.AnticipatoryBoundary.Quantization));

		//If NumFramesBeforeVoiceSlot is less than 0, change the boundary back to the original, and mark this command as having 0 frames till exec
		if (NumFramesBeforeVoiceSlot < 0)
		{
			return 0;
		}
	
		return NumFramesBeforeVoiceSlot;
	}

	void FQuantizedQueueCommand::OnFinalCallbackCustom(int32 InNumFramesLeft)
	{
		if (OwningClockPtr)
		{
			FName ClockName = OwningClockPtr->GetName();
			Audio::FQuartzQueueCommandData CommandData(AudioComponentData, ClockName);

			AudioComponentData.Subscriber.PushEvent(CommandData);
		}
	}

	static const FName QueueCommandName("Queue Command");
	FName FQuantizedQueueCommand::GetCommandName() const
	{
		return QueueCommandName;
	}
	
	TSharedPtr<IQuartzQuantizedCommand> FQuantizedTickRateChange::GetDeepCopyOfDerivedObject() const
	{
		TSharedPtr<FQuantizedTickRateChange> NewCopy = MakeShared<FQuantizedTickRateChange>();

		NewCopy->OwningClockPtr = OwningClockPtr;
		NewCopy->TickRate = TickRate;

		return NewCopy;
	}
	
	void FQuantizedTickRateChange::OnQueuedCustom(const FQuartzQuantizedCommandInitInfo& InCommandInitInfo)
	{
		OwningClockPtr = InCommandInitInfo.OwningClockPointer;
	}
	
	void FQuantizedTickRateChange::OnFinalCallbackCustom(int32 InNumFramesLeft)
	{
		OwningClockPtr->ChangeTickRate(TickRate, InNumFramesLeft);
	}

	static const FName TickRateChangeCommandName("Tick Rate Change Command");
	FName FQuantizedTickRateChange::GetCommandName() const
	{
		return TickRateChangeCommandName;
	}

	TSharedPtr<IQuartzQuantizedCommand> FQuantizedTransportReset::GetDeepCopyOfDerivedObject() const
	{
		TSharedPtr<FQuantizedTransportReset> NewCopy = MakeShared<FQuantizedTransportReset>();

		NewCopy->OwningClockPtr = OwningClockPtr;

		return NewCopy;
	}

	void FQuantizedTransportReset::OnQueuedCustom(const FQuartzQuantizedCommandInitInfo& InCommandInitInfo)
	{
		OwningClockPtr = InCommandInitInfo.OwningClockPointer;
	}

	void FQuantizedTransportReset::OnFinalCallbackCustom(int32 InNumFramesLeft)
	{
		// todo: guard against multiple reset commands executing in the same clock tick
		// OwningClockPtr->ResetTransport(InNumFramesLeft - 1); // - 1 to triggering events w/o double trigger of events on the last frame
		OwningClockPtr->ResetTransport(0);
		OwningClockPtr->AddToTickDelay(InNumFramesLeft); // next metronome tick will be less by InNumFramesLeft
	}

	static const FName TransportResetCommandName("Transport Reset Command");
	FName FQuantizedTransportReset::GetCommandName() const
	{
		return TransportResetCommandName;
	}


	TSharedPtr<IQuartzQuantizedCommand> FQuantizedOtherClockStart::GetDeepCopyOfDerivedObject() const
	{
		TSharedPtr<FQuantizedOtherClockStart> NewCopy = MakeShared<FQuantizedOtherClockStart>();

		NewCopy->OwningClockPtr = OwningClockPtr;
		NewCopy->NameOfClockToStart = NameOfClockToStart;

		return NewCopy;
	}

	void FQuantizedOtherClockStart::OnQueuedCustom(const FQuartzQuantizedCommandInitInfo& InCommandInitInfo)
	{
		OwningClockPtr = InCommandInitInfo.OwningClockPointer;
		check(OwningClockPtr.IsValid());

		NameOfClockToStart = InCommandInitInfo.OtherClockName;
	}

	void FQuantizedOtherClockStart::OnFinalCallbackCustom(int32 InNumFramesLeft)
	{
		if (!ensureMsgf(OwningClockPtr.IsValid(), TEXT("Quantized Other Clock Start is early exiting (invalid/missing Owning Clock Pointer)")))
		{
			return;
		}

		// get access to the clock manager
		FQuartzClockManager* ClockManager = OwningClockPtr->GetClockManager();

		bool bShouldStart = ClockManager && !ClockManager->IsClockRunning(NameOfClockToStart);

		if (bShouldStart)
		{
			// ...start the clock
			ClockManager->ResumeClock(NameOfClockToStart, InNumFramesLeft);

			if (ClockManager->HasClockBeenTickedThisUpdate(NameOfClockToStart))
			{
				ClockManager->UpdateClock(NameOfClockToStart, ClockManager->GetLastUpdateSizeInFrames());
			}
		}
	}

	static const FName StartOtherClockName("Start Other Clock Command");
	FName FQuantizedOtherClockStart::GetCommandName() const
	{
		return StartOtherClockName;
	}


	FQuantizedNotify::FQuantizedNotify(float InMsOffset)  : OffsetInMs(InMsOffset)
	{
	}

	int32 FQuantizedNotify::OverrideFramesUntilExec(int32 NumFramesUntilExec)
	{
		constexpr float MsToSec = 1000.f;
		return FMath::Max(0, NumFramesUntilExec + (OffsetInMs / MsToSec) * SampleRate);
	}

	void FQuantizedNotify::OnQueuedCustom(const FQuartzQuantizedCommandInitInfo& InCommandInitInfo)
	{
		SampleRate = InCommandInitInfo.SampleRate;
	}

	TSharedPtr<IQuartzQuantizedCommand> FQuantizedNotify::GetDeepCopyOfDerivedObject() const
	{
		return MakeShared<FQuantizedNotify>();
	}

	static const FName FQuantizedNotifyName("Notify Command");
	FName FQuantizedNotify::GetCommandName() const
	{
		return FQuantizedNotifyName;
	}

} // namespace Audio
