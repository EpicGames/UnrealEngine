// Copyright Epic Games, Inc. All Rights Reserved.

#include "Quartz/QuartzMetronome.h"
#include "Sound/QuartzSubscription.h"

namespace Audio
{
	FQuartzMetronome::FQuartzMetronome(FName InClockName)
		: TimeSinceStart(0), ClockName(InClockName)
	{
		SetTickRate(CurrentTickRate);
	}

	FQuartzMetronome::FQuartzMetronome(const FQuartzTimeSignature& InTimeSignature, FName InClockName)
		: CurrentTimeSignature(InTimeSignature), TimeSinceStart(0), ClockName(InClockName)
	{
		SetTickRate(CurrentTickRate);
	}

	FQuartzMetronome::~FQuartzMetronome()
	{
	}

	void FQuartzMetronome::Tick(int32 InNumSamples, int32 FramesOfLatency)
	{
		LastTickCpuCycles64 = FPlatformTime::Cycles64();


		for (int i = 0; i < static_cast<int32>(EQuartzCommandQuantization::Count); ++i)
		{
			const EQuartzCommandQuantization DurationType = static_cast<EQuartzCommandQuantization>(i);
			int32 EventFrame = FramesLeftInMusicalDuration[DurationType]; 
			FramesLeftInMusicalDuration[DurationType] -= InNumSamples;
			
			if (FramesLeftInMusicalDuration[DurationType] < 0)
			{
				// the beat value is constant
				if (!(DurationType == EQuartzCommandQuantization::Beat && PulseDurations.Num()))
				{
					do
					{
						PendingMetronomeEvents.Add(DurationType, EventFrame);
						
						EventFrame += MusicalDurationsInFrames[DurationType];
						FramesLeftInMusicalDuration[DurationType] += MusicalDurationsInFrames[DurationType];
					}
					while (FramesLeftInMusicalDuration[DurationType] <= 0);
				}
				else
				{
					// the beat value can change
					do
					{
						PendingMetronomeEvents.Add(DurationType, EventFrame);
						if (++PulseDurationIndex == PulseDurations.Num())
						{
							PulseDurationIndex = 0;
						}

						EventFrame += MusicalDurationsInFrames[DurationType];
						FramesLeftInMusicalDuration[DurationType] += PulseDurations[PulseDurationIndex];
						MusicalDurationsInFrames[DurationType] = PulseDurations[PulseDurationIndex];
					}
					while (FramesLeftInMusicalDuration[DurationType] <= 0);
				}
			}
		}

		// update transport
		if (PendingMetronomeEvents.HasPendingEvent(EQuartzCommandQuantization::Bar))
		{
			++CurrentTimeStamp.Bars;
			CurrentTimeStamp.Beat = 1;
		}
		else if (PendingMetronomeEvents.HasPendingEvent(EQuartzCommandQuantization::Beat))
		{
			++CurrentTimeStamp.Beat;
		}

		if (PulseDurations.Num())
		{
			CurrentTimeStamp.BeatFraction = 1.f - (FramesLeftInMusicalDuration[EQuartzCommandQuantization::Beat] / static_cast<float>(PulseDurations[PulseDurationIndex]));
		}
		else
		{
			CurrentTimeStamp.BeatFraction = 1.f - (FramesLeftInMusicalDuration[EQuartzCommandQuantization::Beat] / static_cast<float>(MusicalDurationsInFrames[EQuartzCommandQuantization::Beat]));
		}

		TimeSinceStart += double(InNumSamples) / CurrentTickRate.GetSampleRate(); 
		CurrentTimeStamp.Seconds = TimeSinceStart;
		FireEvents();
		PendingMetronomeEvents.Reset();
	}

	void FQuartzMetronome::SetTickRate(FQuartzClockTickRate InNewTickRate, int32 NumFramesLeft)
	{
		// early exit?
		const bool bSameAsOldTickRate = FMath::IsNearlyEqual(InNewTickRate.GetFramesPerTick(), CurrentTickRate.GetFramesPerTick());
		const bool bIsInitialized = (MusicalDurationsInFrames[0] > 0);

		if (bSameAsOldTickRate && bIsInitialized)
		{
			return;
		}

		// ratio between new and old rates
		const double Ratio = InNewTickRate.GetFramesPerTick() / CurrentTickRate.GetFramesPerTick();

		for (double& Value : FramesLeftInMusicalDuration.FramesInTimeValueInternal)
		{
			Value = NumFramesLeft + Ratio * (Value - NumFramesLeft);
		}

		CurrentTickRate = InNewTickRate;
		RecalculateDurations();
	}

	void FQuartzMetronome::SetSampleRate(float InNewSampleRate)
	{
		CurrentTickRate.SetSampleRate(InNewSampleRate);
		RecalculateDurations();
	}

	void FQuartzMetronome::SetTimeSignature(const FQuartzTimeSignature& InNewTimeSignature)
	{
		CurrentTimeSignature = InNewTimeSignature;
		RecalculateDurations();
	}

	double FQuartzMetronome::GetFramesUntilBoundary(FQuartzQuantizationBoundary InQuantizationBoundary) const
	{
		if (!ensure(InQuantizationBoundary.Quantization != EQuartzCommandQuantization::None))
		{
			return 0; // Metronome's should not have to deal w/ Quartization == None
		}

		if (InQuantizationBoundary.Multiplier < 1.0f)
		{
			UE_LOGF(LogAudioQuartz, Warning, "Quantization Boundary being clamped to 1.0 (from %f)", InQuantizationBoundary.Multiplier);
			InQuantizationBoundary.Multiplier = 1.f;
		}

		// number of frames until the next occurrence of this boundary
		double FramesUntilBoundary = FramesLeftInMusicalDuration[InQuantizationBoundary.Quantization];

		// how many multiples actually exist until the boundary we care about?
		int32 NumDurationsLeft = static_cast<int32>(InQuantizationBoundary.Multiplier) - 1;

		// in the simple case that's all we need to know
		bool bIsSimpleCase = FMath::IsNearlyEqual(InQuantizationBoundary.Multiplier, 1.f);

		// it is NOT the simple case if we are in Bar-Relative. // i.e. 1.f Beat here means "Beat 1 of the bar"
		bIsSimpleCase &= (InQuantizationBoundary.CountingReferencePoint != EQuarztQuantizationReference::BarRelative);

		if (CurrentTimeStamp.IsZero() && !InQuantizationBoundary.bFireOnClockStart)
		{
			FramesUntilBoundary = MusicalDurationsInFrames[InQuantizationBoundary.Quantization];

			if (NumDurationsLeft == 0)
			{
				return FramesUntilBoundary;
			}
		}
		else if (bIsSimpleCase || CurrentTimeStamp.IsZero())
		{
			return FramesUntilBoundary;
		}

		// counting from the current point in time
		if (InQuantizationBoundary.CountingReferencePoint == EQuarztQuantizationReference::CurrentTimeRelative)
		{
			// continue
		}
		// counting from the beginning of the of the current transport
		else if (InQuantizationBoundary.CountingReferencePoint == EQuarztQuantizationReference::TransportRelative)
		{
			// how many of these subdivisions have happened since in the transport lifespan
			int32 CurrentCount = CountNumSubdivisionsSinceStart(InQuantizationBoundary.Quantization);
			 
			// find the remainder
			if (CurrentCount >= InQuantizationBoundary.Multiplier)
			{
				CurrentCount %= static_cast<int32>(InQuantizationBoundary.Multiplier);
			}

			NumDurationsLeft -= CurrentCount;
		}
		// counting from the current bar
		else if (InQuantizationBoundary.CountingReferencePoint == EQuarztQuantizationReference::BarRelative)
		{
			const float NumSubdivisionsPerBar = CountNumSubdivisionsPerBar(InQuantizationBoundary.Quantization);
			const float NumSubdivisionsAlreadyOccuredInCurrentBar = CountNumSubdivisionsSinceBarStart(InQuantizationBoundary.Quantization);

			// the requested duration is longer than our current bar
			// do the math in bars instead
			if (NumSubdivisionsPerBar < 1.f && ensure(!FMath::IsNearlyZero(NumSubdivisionsPerBar)))
			{
				const float NumBarsPerSubdivision = 1.f / NumSubdivisionsPerBar;
				const float NumBarsRemaining = NumBarsPerSubdivision - (NumSubdivisionsAlreadyOccuredInCurrentBar - 1.f);

				InQuantizationBoundary.Multiplier = NumBarsRemaining;
				InQuantizationBoundary.Quantization = EQuartzCommandQuantization::Bar;
				
				NumDurationsLeft = static_cast<int32>(InQuantizationBoundary.Multiplier) - 1;
			}
			else
			{
				NumDurationsLeft = NumDurationsLeft % static_cast<int32>(NumSubdivisionsPerBar) - static_cast<int32>(NumSubdivisionsAlreadyOccuredInCurrentBar);
			}

			// if NumDurationsLeft is negative, it means the target has already passed this bar.
			// instead we will schedule the sound for the same target in the next bar
			if (NumDurationsLeft < 0)
			{
				NumDurationsLeft += NumSubdivisionsPerBar;
			}
		}

		const double FractionalPortion = FMath::Fractional(InQuantizationBoundary.Multiplier);

		// for Beats, the lengths are not uniform for complex meters
		if ((InQuantizationBoundary.Quantization == EQuartzCommandQuantization::Beat) && PulseDurations.Num())
		{
			// if the metronome hasn't ticked yet, then PulseDurationIndex is -1 (treat as index zero)
			int32 TempPulseDurationIndex = (PulseDurationIndex < 0) ? 0 : PulseDurationIndex;

			for (int i = 0; i < NumDurationsLeft; ++i)
			{
				// need to increment now because FramesUntilBoundary already
				// represents the current (fractional) pulse duration
				if (++TempPulseDurationIndex == PulseDurations.Num())
				{
					TempPulseDurationIndex = 0;
				}

				FramesUntilBoundary += PulseDurations[TempPulseDurationIndex];
			}

			if (++TempPulseDurationIndex == PulseDurations.Num())
			{
				TempPulseDurationIndex = 0;
			}

			FramesUntilBoundary += FractionalPortion * PulseDurations[TempPulseDurationIndex];
		}
		else
		{
			const float Multiplier = NumDurationsLeft + FractionalPortion;
			const float Duration = static_cast<float>(MusicalDurationsInFrames[InQuantizationBoundary.Quantization]);
			FramesUntilBoundary += Multiplier * Duration;
		}

		return FramesUntilBoundary;
	}

	float FQuartzMetronome::CountNumSubdivisionsPerBar(EQuartzCommandQuantization InSubdivision) const
	{
		if (InSubdivision == EQuartzCommandQuantization::Beat && PulseDurations.Num())
		{
			return static_cast<float>(PulseDurations.Num());
		}

		return static_cast<float>(MusicalDurationsInFrames[EQuartzCommandQuantization::Bar] / MusicalDurationsInFrames[InSubdivision]);
	}

	float FQuartzMetronome::CountNumSubdivisionsSinceBarStart(EQuartzCommandQuantization InSubdivision) const
	{
		// for our own counting, we don't say that "one bar has occurred since the start of the bar"
		if (InSubdivision == EQuartzCommandQuantization::Bar)
		{
			return 0.0f;
		}

		// Count starts at 1.0f since all musical subdivisions occur once at beat 0 in a bar
		float Count = 1.f;
		if ((InSubdivision == EQuartzCommandQuantization::Beat) && PulseDurations.Num())
		{
			Count += static_cast<float>(PulseDurationIndex);
		}
		else
		{
			const float BarProgress = 1.0f - (FramesLeftInMusicalDuration[EQuartzCommandQuantization::Bar] / static_cast<float>(MusicalDurationsInFrames[EQuartzCommandQuantization::Bar]));
			Count += (BarProgress * CountNumSubdivisionsPerBar(InSubdivision));
		}

		return Count;
	}

	float FQuartzMetronome::CountNumSubdivisionsSinceStart(EQuartzCommandQuantization InSubdivision) const
	{
		const int32 NumPerBar = CountNumSubdivisionsPerBar(InSubdivision);
		const int32 NumInThisBar = CountNumSubdivisionsSinceBarStart(InSubdivision);

		return (CurrentTimeStamp.Bars - 1) * NumPerBar + NumInThisBar;
	}

	void FQuartzMetronome::FFramesInTimeValue::Reset()
	{
		for (double& FrameCount : FramesInTimeValueInternal)
		{
			FrameCount = 0.0;
		}
	}

	double& FQuartzMetronome::FFramesInTimeValue::operator[](EQuartzCommandQuantization InTimeValue)
	{
		return FramesInTimeValueInternal[static_cast<int32>(InTimeValue)];
	}

	const double& FQuartzMetronome::FFramesInTimeValue::operator[](EQuartzCommandQuantization InTimeValue) const
	{
		return FramesInTimeValueInternal[static_cast<int32>(InTimeValue)];
	}

	double& FQuartzMetronome::FFramesInTimeValue::operator[](int32 Index)
	{
		return FramesInTimeValueInternal[Index];
	}

	const double& FQuartzMetronome::FFramesInTimeValue::operator[](int32 Index) const
	{
		return FramesInTimeValueInternal[Index];
	}

	void FQuartzMetronome::FMetronomeEventEntry::Reset()
	{
		EventFrames.Reset();
	}

	void FQuartzMetronome::FPendingMetronomeEvents::Reset()
	{
		for (FMetronomeEventEntry& Entry : CurrentMetronomeEvents)
		{
			Entry.Reset();
		}
	}

	bool FQuartzMetronome::FPendingMetronomeEvents::HasPendingEvent(const EQuartzCommandQuantization InDuration) const
	{
		return CurrentMetronomeEvents[static_cast<int32>(InDuration)].EventFrames.Num() != 0;
	}

	void FQuartzMetronome::FPendingMetronomeEvents::Add(const EQuartzCommandQuantization InDuration,
	                                                    const int32 InFrame)
	{
		CurrentMetronomeEvents[static_cast<int32>(InDuration)].EventFrames.Add(InFrame);
	}

	void FQuartzMetronome::CalculateDurationPhases(float (&OutPhases)[static_cast<int32>(EQuartzCommandQuantization::Count)]) const
	{
		for (int i = 0; i < static_cast<int32>(EQuartzCommandQuantization::Count); ++i)
		{
			OutPhases[i] = 1.f - FramesLeftInMusicalDuration[i] / static_cast<float>(MusicalDurationsInFrames[i]);
		}
	}

	void FQuartzMetronome::SubscribeToTimeDivision(MetronomeCommandQueuePtr InListenerQueue, EQuartzCommandQuantization InQuantizationBoundary)
	{
		MetronomeSubscriptionMatrix[(int32)InQuantizationBoundary].AddUnique(InListenerQueue);
		ListenerFlags |= 1 << (int32)InQuantizationBoundary;
	}

	void FQuartzMetronome::SubscribeToAllTimeDivisions(MetronomeCommandQueuePtr InListenerQueue)
	{
		int32 i = 0;
		for (TArray<MetronomeCommandQueuePtr>& QuantizationBoundarySubscribers : MetronomeSubscriptionMatrix)
		{
			QuantizationBoundarySubscribers.AddUnique(InListenerQueue);
			ListenerFlags |= (1 << i++);
		}
	}

	void FQuartzMetronome::UnsubscribeFromTimeDivision(MetronomeCommandQueuePtr InListenerQueue, EQuartzCommandQuantization InQuantizationBoundary)
	{
		MetronomeSubscriptionMatrix[(int32)InQuantizationBoundary].RemoveSingleSwap(InListenerQueue);

		if (MetronomeSubscriptionMatrix[(int32)InQuantizationBoundary].Num() == 0)
		{
			ListenerFlags &= ~(1 << (int32)InQuantizationBoundary);
		}
	}

	void FQuartzMetronome::UnsubscribeFromAllTimeDivisions(MetronomeCommandQueuePtr InListenerQueue)
	{
		int32 i = 0;
		for (TArray<MetronomeCommandQueuePtr>& QuantizationBoundarySubscribers : MetronomeSubscriptionMatrix)
		{
			QuantizationBoundarySubscribers.RemoveSingleSwap(InListenerQueue);
			
			if (QuantizationBoundarySubscribers.Num() == 0)
			{
				ListenerFlags &= ~(1 << i);
			}

			++i;
		}
	}

	void FQuartzMetronome::ResetTransport()
	{
		CurrentTimeStamp.Reset();

		FramesLeftInMusicalDuration.Reset();

		TimeSinceStart = 0.0;
		PulseDurationIndex = -1;
	}

	void FQuartzMetronome::RecalculateDurations()
	{
		PulseDurations.Reset();

		// get default values for each boundary:
		for (int32 i = 0; i < static_cast<int32>(EQuartzCommandQuantization::Count); ++i)
		{
			MusicalDurationsInFrames[i] = CurrentTickRate.GetFramesPerDuration(static_cast<EQuartzCommandQuantization>(i));
		}

		// determine actual length of a bar
		const double BarLength = CurrentTimeSignature.NumBeats * CurrentTickRate.GetFramesPerDuration(CurrentTimeSignature.BeatType);
		MusicalDurationsInFrames[EQuartzCommandQuantization::Bar] = BarLength;

		// default beat value to the denominator of our time signature
		MusicalDurationsInFrames[EQuartzCommandQuantization::Beat] = CurrentTickRate.GetFramesPerDuration(CurrentTimeSignature.BeatType);

		// potentially update the durations of BEAT and BAR
		if (CurrentTimeSignature.OptionalPulseOverride.Num() != 0)
		{
			// determine the length of each beat
			double LengthCounter = 0.0;
			double StepLength = 0.0;

			for (const FQuartzPulseOverrideStep& PulseStep : CurrentTimeSignature.OptionalPulseOverride)
			{
				for (int i = 0; i < PulseStep.NumberOfPulses; ++i)
				{
					StepLength = CurrentTickRate.GetFramesPerDuration(PulseStep.PulseDuration);
					LengthCounter += StepLength;

					PulseDurations.Add(StepLength);
				}
			}

			if (LengthCounter > BarLength)
			{
				UE_LOGF(LogAudioQuartz, Warning
					, "Pulse override array on Time Signature reperesents more than a bar. The provided list will be trunctaed to 1 Bar in length");

				return;
			}

			// extend the last duration to the length of the bar if needed
			while ((LengthCounter + StepLength) <= BarLength)
			{
				PulseDurations.Add(StepLength);
				LengthCounter += StepLength;
			}

			// check to see if all our pulses are the same length
			const double FirstValue = PulseDurations[0];
			bool bBeatDurationsAreConstant = true;

			for (const double& Values : PulseDurations)
			{
				if (!FMath::IsNearlyEqual(Values, FirstValue))
				{
					bBeatDurationsAreConstant = false;
					break;
				}
			}

			// we can just overwrite the duration array with the single value
			if (bBeatDurationsAreConstant)
			{
				MusicalDurationsInFrames[EQuartzCommandQuantization::Beat] = FirstValue;
				PulseDurations.Reset();
			}
		}
	}

	void FQuartzMetronome::FireEvents()
	{
		FQuartzMetronomeDelegateData Data;
		Data.Bar = (CurrentTimeStamp.Bars);
		Data.Beat = (CurrentTimeStamp.Beat);
		Data.BeatFraction = (CurrentTimeStamp.BeatFraction);
		Data.ClockName = ClockName;
		
		// loop through subscribers of each quantization boundary
		int32 i = -1;
		for (TArray<MetronomeCommandQueuePtr>& QuantizationBoundarySubscribers : MetronomeSubscriptionMatrix)
		{
			Data.Quantization = static_cast<EQuartzCommandQuantization>(++i);

			// if this quantization boundary had any events...
			if (PendingMetronomeEvents.HasPendingEvent(Data.Quantization))
			{
				// ...for each subscriber...
				for (const MetronomeCommandQueuePtr& Subscriber : QuantizationBoundarySubscribers)
				{
					// fire an event for each instance of that metronome event in this buffer
					for (const int32& MetronomeEventFrame : PendingMetronomeEvents.CurrentMetronomeEvents[i].EventFrames)
					{
						Data.FrameOffset = MetronomeEventFrame;
						Subscriber->PushLambda<Quartz::IMetronomeEventListener>(
							[=](Quartz::IMetronomeEventListener& InListener)
							{
								InListener.OnMetronomeEvent(Data);
							});
					}
				}
			}
		}
	}
} // namespace Audio