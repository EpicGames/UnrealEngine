// Copyright Epic Games, Inc. All Rights Reserved.

#include "Quartz/QuartzSubsystem.h"

#include "Engine/World.h"
#include "Quartz/AudioMixerClockHandle.h"
#include "Quartz/QuartzMetronome.h"
#include "Quartz/AudioMixerClockManager.h"
#include "Sound/QuartzQuantizationUtilities.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Stats/Stats.h"

#include "AudioDevice.h"
#include "AudioMixerDevice.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(QuartzSubsystem)

static int32 MaxQuartzSubscribersToUpdatePerTickCvar = -1;
FAutoConsoleVariableRef CVarMaxQuartzSubscribersToUpdatePerTick(
	TEXT("au.Quartz.MaxSubscribersToUpdatePerTick"),
	MaxQuartzSubscribersToUpdatePerTickCvar,
	TEXT("Limits the number of Quartz subscribers to update per Tick.\n")
	TEXT("<= 0: No Limit, >= 1: Limit"),
	ECVF_Default);

static int32 SimulateNoAudioDeviceCvar = 0;
FAutoConsoleVariableRef CVarSimulateNoAudioDevice(
	TEXT("au.Quartz.SimulateNoAudioDevice"),
	SimulateNoAudioDeviceCvar,
	TEXT("If enabled, the QuartzSubsystem will assume no audio device, and will run new clocks in headless mode.\n")
	TEXT("0: Not Enabled, 1: Enabled"),
	ECVF_Default);

static Audio::FMixerDevice* GetAudioMixerDevice(const UWorld* InWorld)
{
	if(!InWorld || SimulateNoAudioDeviceCvar)
	{
		return nullptr;
	}

	FAudioDevice* AudioDevicePtr = InWorld->GetAudioDeviceRaw();
	if(AudioDevicePtr)
	{
		return static_cast<Audio::FMixerDevice*>(AudioDevicePtr);
	}

	return nullptr;
}

static TUniquePtr<FScopeLock> GetPersistentStateScopeLock(const UWorld* InWorld)
{
	if(Audio::FMixerDevice* MixerDevicePtr = GetAudioMixerDevice(InWorld))
	{
		return MakeUnique<FScopeLock>(&MixerDevicePtr->QuartzPersistentStateCritSec);
	}

	return {};
}


void UQuartzSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);
	TUniquePtr<FScopeLock> Lock(GetPersistentStateScopeLock(GetWorld()));

	if(const UWorld* WorldPtr = GetWorld())
	{
		if(Audio::FMixerDevice const* MixerDevice = static_cast<Audio::FMixerDevice*>(WorldPtr->GetAudioDeviceRaw()))
		{
			ClockManagerDataPtr = MixerDevice->QuartzSubsystemData;
		}
	}

	if(!ClockManagerDataPtr)
	{
		ClockManagerDataPtr = MakeShared<Audio::FPersistentQuartzSubsystemData>();
	}
}

void UQuartzSubsystem::Deinitialize()
{
	Super::Deinitialize();
	TUniquePtr<FScopeLock> Lock(GetPersistentStateScopeLock(GetWorld()));

	if(Audio::FMixerDevice* MixerDevice = GetAudioMixerDevice(GetWorld()))
	{
		MixerDevice->QuartzSubsystemData = ClockManagerDataPtr;
	}
}

void UQuartzSubsystem::BeginDestroy()
{
	Super::BeginDestroy();

	// force un-subscribe all Quartz tickable objects
	if (ClockManagerDataPtr)
	{
		ClockManagerDataPtr->SubsystemClockManager.Flush();
	}
}

void FQuartzTickableObjectsManager::Tick(float DeltaTime)
{
	const int32 NumSubscribers = QuartzTickSubscribers.Num();
	if (MaxQuartzSubscribersToUpdatePerTickCvar <= 0 || NumSubscribers <= MaxQuartzSubscribersToUpdatePerTickCvar)
	{
		TArray<FQuartzTickableObject*> SubscribersCopy = QuartzTickSubscribers;

		// we can afford to update ALL subscribers
		for (FQuartzTickableObject* Entry : SubscribersCopy)
		{
			if (Entry && Entry->QuartzIsTickable())
			{
				Entry->QuartzTick(DeltaTime);
			}
		}

		UpdateIndex = 0;
	}
	else
	{
		// only update up to our limit
		for (int i = 0; i < MaxQuartzSubscribersToUpdatePerTickCvar; ++i)
		{
			FQuartzTickableObject* CurrentSubscriber = QuartzTickSubscribers[UpdateIndex];
			if (!ensure(CurrentSubscriber))
			{
				continue;
			}

			if (CurrentSubscriber->QuartzIsTickable())
			{
				CurrentSubscriber->QuartzTick(DeltaTime);
			}

			if (++UpdateIndex == NumSubscribers)
			{
				UpdateIndex = 0;
			}
		}
	}
}

bool FQuartzTickableObjectsManager::IsTickable() const
{
	const int32 NumSubscribers = QuartzTickSubscribers.Num();
	const bool bHasTickSubscribers = NumSubscribers > 0;
	TRACE_INT_VALUE(TEXT("QuartzSubsystem::NumSubscribers"), NumSubscribers);

	// if our manager has no clocks, and we have no ClockHandle subscribers, we don't need to tick
	if (!bHasTickSubscribers)
	{
		return false;
	}

	// if our manager has no clocks, and none of our subscribers are tickable, we don't need to tick
	for (const FQuartzTickableObject * Entry : QuartzTickSubscribers)
	{
		if (Entry && Entry->QuartzIsTickable())
		{
			return true;
		}
	}

	return false;
}

void FQuartzTickableObjectsManager::SubscribeToQuartzTick(FQuartzTickableObject* InObjectToTick)
{
	if (!InObjectToTick)
	{
		return;
	}

	QuartzTickSubscribers.AddUnique(InObjectToTick);
}

void FQuartzTickableObjectsManager::UnsubscribeFromQuartzTick(FQuartzTickableObject* InObjectToTick)
{
	if (!InObjectToTick)
	{
		return;
	}

	QuartzTickSubscribers.RemoveSingleSwap(InObjectToTick);
}


bool UQuartzSubsystem::DoesSupportWorldType(EWorldType::Type WorldType) const
{
	return Super::DoesSupportWorldType(WorldType) || WorldType == EWorldType::EditorPreview;
}


void UQuartzSubsystem::Tick(float DeltaTime)
{
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Audio);
	Super::Tick(DeltaTime);
	TRACE_CPUPROFILER_EVENT_SCOPE(QuartzSubsystem::Tick);

	check(TickableObjectManagerPtr);
	check(ClockManagerDataPtr);
	TUniquePtr<FScopeLock> Lock(GetPersistentStateScopeLock(GetWorld()));

	PruneStaleProxies();
	ClockManagerDataPtr->SubsystemClockManager.LowResoultionUpdate(DeltaTime);
	TickableObjectManagerPtr->Tick(DeltaTime);
}

bool UQuartzSubsystem::IsTickableWhenPaused() const
{
	return bTickEvenWhenPaused;
}

bool UQuartzSubsystem::IsTickable() const
{
	check(TickableObjectManagerPtr);
	check(ClockManagerDataPtr);
	TUniquePtr<FScopeLock> Lock(GetPersistentStateScopeLock(GetWorld()));

	const int32 NumClocks = ClockManagerDataPtr->SubsystemClockManager.GetNumClocks();
	TRACE_INT_VALUE(TEXT("QuartzSubsystem::NumClocks"), NumClocks);

	// IsTickable() updates unreal insights values
	const bool bSubscribersNeedUpdate = TickableObjectManagerPtr->IsTickable();
	const bool bIsManagingClocks = NumClocks > 0;

	return bIsManagingClocks || bSubscribersNeedUpdate;
}

TStatId UQuartzSubsystem::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(UQuartzSubsystem, STATGROUP_Tickables);
}

 void UQuartzSubsystem::SubscribeToQuartzTick(FQuartzTickableObject * InObjectToTick)
 {
	check(TickableObjectManagerPtr.IsValid());
 	TickableObjectManagerPtr->SubscribeToQuartzTick(InObjectToTick);
 }


 void UQuartzSubsystem::UnsubscribeFromQuartzTick(FQuartzTickableObject * InObjectToTick)
 {
	check(TickableObjectManagerPtr.IsValid());
 	TickableObjectManagerPtr->UnsubscribeFromQuartzTick(InObjectToTick);
 }


UQuartzSubsystem* UQuartzSubsystem::Get(const UWorld* const World)
{
	if (World)
	{
		return World->GetSubsystem<UQuartzSubsystem>();
	}

	return nullptr;
}


Audio::FQuartzQuantizedRequestData UQuartzSubsystem::CreateRequestDataForSchedulePlaySound(UQuartzClockHandle* InClockHandle, const FOnQuartzCommandEventBP& InDelegate, const FQuartzQuantizationBoundary& InQuantizationBoundary)
{
	Audio::FQuartzQuantizedRequestData CommandInitInfo;

	if (!InClockHandle)
	{
		return {};
	}

	CommandInitInfo.ClockName = InClockHandle->GetClockName();
	CommandInitInfo.QuantizationBoundary = InQuantizationBoundary;
	CommandInitInfo.QuantizedCommandPtr = MakeShared<Audio::FQuantizedPlayCommand>();
	CommandInitInfo.GameThreadSubscribers.Append(InQuantizationBoundary.GameThreadSubscribers);
	CommandInitInfo.GameThreadSubscribers.Add(InClockHandle->GetQuartzSubscriber());

	if (InDelegate.IsBound())
	{
		CommandInitInfo.GameThreadDelegateID = InClockHandle->AddCommandDelegate(InDelegate);
	}

	return CommandInitInfo;
}

bool UQuartzSubsystem::IsQuartzEnabled()
{
	return true;
}


Audio::FQuartzQuantizedRequestData UQuartzSubsystem::CreateRequestDataForTickRateChange(UQuartzClockHandle* InClockHandle, const FOnQuartzCommandEventBP& InDelegate, const Audio::FQuartzClockTickRate& InNewTickRate, const FQuartzQuantizationBoundary& InQuantizationBoundary)
{
	if (!ensure(InClockHandle))
	{
		return { };
	}

	const TSharedPtr<Audio::FQuantizedTickRateChange> TickRateChangeCommandPtr = MakeShared<Audio::FQuantizedTickRateChange>();
	TickRateChangeCommandPtr->SetTickRate(InNewTickRate);

	Audio::FQuartzQuantizedRequestData CommandInitInfo;

	CommandInitInfo.ClockName = InClockHandle->GetClockName();
	CommandInitInfo.QuantizationBoundary = InQuantizationBoundary;
	CommandInitInfo.QuantizedCommandPtr = TickRateChangeCommandPtr;
	CommandInitInfo.GameThreadSubscribers.Append(InQuantizationBoundary.GameThreadSubscribers);
	CommandInitInfo.GameThreadSubscribers.Add(InClockHandle->GetQuartzSubscriber());

	if (InDelegate.IsBound())
	{
		CommandInitInfo.GameThreadDelegateID = InClockHandle->AddCommandDelegate(InDelegate);
	}

	return CommandInitInfo;
}

Audio::FQuartzQuantizedRequestData UQuartzSubsystem::CreateRequestDataForTransportReset(UQuartzClockHandle* InClockHandle, const FQuartzQuantizationBoundary& InQuantizationBoundary, const FOnQuartzCommandEventBP& InDelegate)
{
	if (!ensure(InClockHandle))
	{
		return { };
	}

	const TSharedPtr<Audio::FQuantizedTransportReset> TransportResetCommandPtr = MakeShared<Audio::FQuantizedTransportReset>();

	Audio::FQuartzQuantizedRequestData CommandInitInfo;

	CommandInitInfo.ClockName = InClockHandle->GetClockName();
	CommandInitInfo.QuantizationBoundary = InQuantizationBoundary;
	CommandInitInfo.QuantizedCommandPtr = TransportResetCommandPtr;
	CommandInitInfo.GameThreadSubscribers.Append(InQuantizationBoundary.GameThreadSubscribers);
	CommandInitInfo.GameThreadSubscribers.Add(InClockHandle->GetQuartzSubscriber());

	if (InDelegate.IsBound())
	{
		CommandInitInfo.GameThreadDelegateID = InClockHandle->AddCommandDelegate(InDelegate);
	}

	return CommandInitInfo;
}

Audio::FQuartzQuantizedRequestData UQuartzSubsystem::CreateRequestDataForStartOtherClock(UQuartzClockHandle* InClockHandle, FName InClockToStart, const FQuartzQuantizationBoundary& InQuantizationBoundary, const FOnQuartzCommandEventBP& InDelegate)
{
	if (!ensure(InClockHandle))
	{
		return { };
	}

	const TSharedPtr<Audio::FQuantizedOtherClockStart> TransportResetCommandPtr = MakeShared<Audio::FQuantizedOtherClockStart>();

	Audio::FQuartzQuantizedRequestData CommandInitInfo;

	CommandInitInfo.ClockName = InClockHandle->GetClockName();
	CommandInitInfo.OtherClockName = InClockToStart;
	CommandInitInfo.QuantizationBoundary = InQuantizationBoundary;
	CommandInitInfo.QuantizedCommandPtr = TransportResetCommandPtr;
	CommandInitInfo.GameThreadSubscribers.Append(InQuantizationBoundary.GameThreadSubscribers);
	CommandInitInfo.GameThreadSubscribers.Add(InClockHandle->GetQuartzSubscriber());

	if (InDelegate.IsBound())
	{
		CommandInitInfo.GameThreadDelegateID = InClockHandle->AddCommandDelegate(InDelegate);
	}

	return CommandInitInfo;
}

Audio::FQuartzQuantizedRequestData UQuartzSubsystem::CreateRequestDataForQuantizedNotify(UQuartzClockHandle* InClockHandle, const FQuartzQuantizationBoundary& InQuantizationBoundary, const FOnQuartzCommandEventBP& InDelegate, float InMsOffset)
{
	if (!ensure(InClockHandle))
	{
		return { };
	}

	const TSharedPtr<Audio::FQuantizedNotify> NotifyCommandPtr = MakeShared<Audio::FQuantizedNotify>(InMsOffset);

	Audio::FQuartzQuantizedRequestData CommandInitInfo;

	CommandInitInfo.ClockName = InClockHandle->GetClockName();
	CommandInitInfo.QuantizationBoundary = InQuantizationBoundary;
	CommandInitInfo.QuantizedCommandPtr = NotifyCommandPtr;
	CommandInitInfo.GameThreadSubscribers.Append(InQuantizationBoundary.GameThreadSubscribers);
	CommandInitInfo.GameThreadSubscribers.Add(InClockHandle->GetQuartzSubscriber());

	if (InDelegate.IsBound())
	{
		CommandInitInfo.GameThreadDelegateID = InClockHandle->AddCommandDelegate(InDelegate);
	}

	return CommandInitInfo;
}

Audio::FQuartzClockManager* UQuartzSubsystem::GetClockManager(const UObject* WorldContextObject, bool bUseAudioEngineClockManager)
{
	// decide if the clock should be managed by the AudioDevice (audio engine) or the Subsystem (this object)
	Audio::FQuartzClockManager* ClockManager;
	Audio::FMixerDevice* MixerDevice = GetAudioMixerDevice(WorldContextObject->GetWorld());
	if(!bUseAudioEngineClockManager || !MixerDevice)
	{
		check(ClockManagerDataPtr);
		TUniquePtr<FScopeLock> Lock(GetPersistentStateScopeLock(GetWorld()));
		ClockManager = &ClockManagerDataPtr->SubsystemClockManager;
	}
	else
	{
		ClockManager = &MixerDevice->QuantizedEventClockManager;
	}

	// we should have fallen back to this object
	return ClockManager;
}

UQuartzClockHandle* UQuartzSubsystem::CreateNewClock(const UObject* WorldContextObject, FName ClockName, FQuartzClockSettings InSettings, bool bOverrideSettingsIfClockExists, bool bUseAudioEngineClockManager)
{
	if (ClockName.IsNone() || !WorldContextObject)
	{
		return nullptr;
	}

	Audio::FQuartzClockManager* ClockManager = GetClockManager(WorldContextObject, bUseAudioEngineClockManager);
	check(ClockManager); // should have at least fallen back to "this" object as a manager

	// numerator of time signature must be >= 1
	if (InSettings.TimeSignature.NumBeats < 1)
	{
		UE_LOGF(LogAudioQuartz, Warning, "Clock: (%ls) is attempting to set a time signature with a Numerator < 1.  Clamping to 1 beat per bar", *ClockName.ToString());
		InSettings.TimeSignature.NumBeats = 1;
	}

	Audio::FQuartzClockProxy ClockProxy = ClockManager->GetOrCreateClock(ClockName, InSettings, bOverrideSettingsIfClockExists);
	UQuartzClockHandle* ClockHandle = static_cast<UQuartzClockHandle*>(NewObject<UQuartzClockHandle>()->Init(WorldContextObject->GetWorld()));
	ClockHandle->SubscribeToClock(WorldContextObject, ClockName, &ClockProxy);

	// if we are not the manager for the clock, it means the FAudioDevice is,
	// so we hold onto our own copy of the proxy
	check(ClockManagerDataPtr);
	TUniquePtr<FScopeLock> Lock(GetPersistentStateScopeLock(GetWorld()));
	ClockManagerDataPtr->ActiveAudioMixerClockProxies.Add(ClockProxy);

	return ClockHandle;
}


void UQuartzSubsystem::DeleteClockByName(const UObject* WorldContextObject, FName ClockName)
{
	// first look for the clock on the audio device's clock manager
	bool bShouldDeleteSynchronous = false;
	Audio::FQuartzClockManager* ClockManager = GetClockManager(WorldContextObject, /*bUseAudioEngineClockManager*/ true);

	if (ClockManager && !ClockManager->DoesClockExist(ClockName))
	{
		// if we didn't find it, assume the clock is on the subsystem's clock manager
		ClockManager = GetClockManager(WorldContextObject, /*bUseAudioEngineClockManager*/ false);
		bShouldDeleteSynchronous = true;
	}

	// if the clock is managed by the audio mixer device,
	// bShouldDeleteSynchronous is false, and it will be deleted on the correct thread.
	// if its managed by the subsystem, bShouldDeleteSynchronous will be true
	if(ClockManager)
	{
		ClockManager->RemoveClock(ClockName, bShouldDeleteSynchronous);
		PruneStaleProxies();
	}
}

void UQuartzSubsystem::DeleteClockByHandle(const UObject* WorldContextObject, UQuartzClockHandle*& InClockHandle)
{
	if (InClockHandle)
	{
		DeleteClockByName(WorldContextObject, InClockHandle->GetClockName());
	}
}

UQuartzClockHandle* UQuartzSubsystem::GetHandleForClock(const UObject* WorldContextObject, FName ClockName)
{
	Audio::FQuartzClockManager* ClockManager = GetClockManager(WorldContextObject);
	if (!ClockManager || !ClockManager->DoesClockExist(ClockName))
	{
		return nullptr;
	}

	Audio::FQuartzClockProxy ClockHandle = ClockManager->GetClock(ClockName);

	UQuartzClockHandle* ClockHandlePtr = static_cast<UQuartzClockHandle*>(NewObject<UQuartzClockHandle>()->Init(WorldContextObject->GetWorld()));
	return ClockHandlePtr->SubscribeToClock(WorldContextObject, ClockName, &ClockHandle);
}


Audio::FQuartzClockProxy UQuartzSubsystem::GetProxyForClock(FName ClockName) const
{
	if(Audio::FQuartzClockProxy const* ProxyPtr = FindProxyByName(ClockName))
	{
		return *ProxyPtr; // caller gets their own copy
	}

	return {};
}

void UQuartzSubsystem::AddProxyForExternalClock(const Audio::FQuartzClockProxy& InProxy)
{
	// make sure we aren't adding a duplicate name
	if(FindProxyByName(InProxy.GetClockName()))
	{
		UE_LOGF(LogAudioQuartz, Warning, "Received request to add external Clock: (%ls) when a clock of that name already exists (Ignoring Request)", *InProxy.GetClockName().ToString());
		return;
	}
}


bool UQuartzSubsystem::DoesClockExist(const UObject* WorldContextObject, FName ClockName)
{
	Audio::FQuartzClockProxy const* Proxy = FindProxyByName(ClockName);
	if(Proxy && Proxy->IsValid())
	{
		return Proxy->DoesClockExist();
	}

	return {};
}

bool UQuartzSubsystem::IsClockRunning(const UObject* WorldContextObject, FName ClockName)
{
	Audio::FQuartzClockProxy const* Proxy = FindProxyByName(ClockName);
	if(Proxy && Proxy->IsValid())
	{
		return Proxy->IsClockRunning();
	}

	return {};
}

float UQuartzSubsystem::GetDurationOfQuantizationTypeInSeconds(const UObject* WorldContextObject, FName ClockName, const EQuartzCommandQuantization& QuantizationType, float Multiplier)
{
	Audio::FQuartzClockProxy const* Proxy = FindProxyByName(ClockName);
	if(Proxy && Proxy->IsValid())
	{
		return Proxy->GetDurationOfQuantizationTypeInSeconds(QuantizationType, Multiplier);
	}

	return {};
}

FQuartzTransportTimeStamp UQuartzSubsystem::GetCurrentClockTimestamp(const UObject* WorldContextObject, const FName& InClockName)
{
	Audio::FQuartzClockProxy const* Proxy = FindProxyByName(InClockName);
	if(Proxy && Proxy->IsValid())
	{
		return Proxy->GetCurrentClockTimestamp();
	}

	return {};
}

float UQuartzSubsystem::GetEstimatedClockRunTime(const UObject* WorldContextObject, const FName& InClockName)
{
	Audio::FQuartzClockProxy const* Proxy = FindProxyByName(InClockName);
	if(Proxy && Proxy->IsValid())
	{
		return Proxy->GetEstimatedClockRunTimeSeconds();
	}

	return {};
}

// todo: move FQuartLatencyTracker off the AudioMixerClockManager? (GameThread->AudioRenderThread tracking)
float UQuartzSubsystem::GetGameThreadToAudioRenderThreadAverageLatency(const UObject* WorldContextObject)
{
	Audio::FQuartzClockManager* ClockManager = GetClockManager(WorldContextObject);
	if (!ClockManager)
	{
		return { };
	}
	return ClockManager->GetLifetimeAverageLatency();
}


float UQuartzSubsystem::GetGameThreadToAudioRenderThreadMinLatency(const UObject* WorldContextObject)
{
	Audio::FQuartzClockManager* ClockManager = GetClockManager(WorldContextObject);
	if (!ClockManager)
	{
		return { };
	}
	return ClockManager->GetMinLatency();
}


float UQuartzSubsystem::GetGameThreadToAudioRenderThreadMaxLatency(const UObject* WorldContextObject)
{
	Audio::FQuartzClockManager* ClockManager = GetClockManager(WorldContextObject);
	if (!ClockManager)
	{
		return { };
	}
	return ClockManager->GetMinLatency();
}


float UQuartzSubsystem::GetAudioRenderThreadToGameThreadAverageLatency()
{
	check(TickableObjectManagerPtr.IsValid());
	return TickableObjectManagerPtr->GetLifetimeAverageLatency();
}


float UQuartzSubsystem::GetAudioRenderThreadToGameThreadMinLatency()
{
	check(TickableObjectManagerPtr.IsValid());
	return TickableObjectManagerPtr->GetMinLatency();
}


float UQuartzSubsystem::GetAudioRenderThreadToGameThreadMaxLatency()
{
	check(TickableObjectManagerPtr.IsValid());
	return TickableObjectManagerPtr->GetMaxLatency();
}


float UQuartzSubsystem::GetRoundTripAverageLatency(const UObject* WorldContextObject)
{
	// very much an estimate
	return GetAudioRenderThreadToGameThreadAverageLatency() + GetGameThreadToAudioRenderThreadAverageLatency(WorldContextObject);
}


float UQuartzSubsystem::GetRoundTripMinLatency(const UObject* WorldContextObject)
{
	return GetAudioRenderThreadToGameThreadMaxLatency() + GetGameThreadToAudioRenderThreadMaxLatency(WorldContextObject);
}


float UQuartzSubsystem::GetRoundTripMaxLatency(const UObject* WorldContextObject)
{
	return GetAudioRenderThreadToGameThreadMinLatency() + GetGameThreadToAudioRenderThreadMinLatency(WorldContextObject);
}

void UQuartzSubsystem::SetQuartzSubsystemTickableWhenPaused(const bool bInTickableWhenPaused)
{
	bTickEvenWhenPaused = bInTickableWhenPaused;
}

TWeakPtr<FQuartzTickableObjectsManager> UQuartzSubsystem::GetTickableObjectManager() const
{
	return TickableObjectManagerPtr;
}

void UQuartzSubsystem::PruneStaleProxies()
{
	check(ClockManagerDataPtr);
	TUniquePtr<FScopeLock> Lock(GetPersistentStateScopeLock(GetWorld()));
	PruneStaleProxiesInternal(ClockManagerDataPtr->ActiveExternalClockProxies);
	PruneStaleProxiesInternal(ClockManagerDataPtr->ActiveAudioMixerClockProxies);
}

void UQuartzSubsystem::PruneStaleProxiesInternal(TArray<Audio::FQuartzClockProxy>& ContainerToPrune)
{
	 for(int32 i = 0; i < ContainerToPrune.Num(); ++i)
	 {
		 if(ContainerToPrune[i].IsValid() == false)
		 {
		 	ContainerToPrune.RemoveAtSwap(i--, EAllowShrinking::No);
		 }
	 }
}

Audio::FQuartzClockProxy* UQuartzSubsystem::FindProxyByName(const FName& ClockName)
{
	check(ClockManagerDataPtr);
	TUniquePtr<FScopeLock> Lock(GetPersistentStateScopeLock(GetWorld()));
	Audio::FQuartzClockProxy* Result = ClockManagerDataPtr->ActiveAudioMixerClockProxies.FindByKey(ClockName);

	// if the subsystem doesn't have a match, check the externally-registered clock proxies
	if(!Result)
	{
		Result = ClockManagerDataPtr->ActiveExternalClockProxies.FindByKey(ClockName);
	}

	return Result;
}

Audio::FQuartzClockProxy const* UQuartzSubsystem::FindProxyByName(const FName& ClockName) const
{
	check(ClockManagerDataPtr);
	TUniquePtr<FScopeLock> Lock(GetPersistentStateScopeLock(GetWorld()));

	Audio::FQuartzClockProxy const* Result = ClockManagerDataPtr->ActiveAudioMixerClockProxies.FindByKey(ClockName);

	// if the subsystem doesn't have a match, check the externally-registered clock proxies
	if(!Result)
	{
		Result = ClockManagerDataPtr->ActiveExternalClockProxies.FindByKey(ClockName);
	}

	return Result;
}

