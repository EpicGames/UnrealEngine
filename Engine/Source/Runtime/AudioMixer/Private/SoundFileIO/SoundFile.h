// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "SoundFileIOEnums.h"


namespace Audio
{
	using SoundFileCount = int64;

	/**
	 * Specifies a sound file description.
	 * 
	 * Note that libsndfile reads some of these fields (noteably FormatFlags and bIsSeekable)
	 * at file open time so we zero them out at construction time to avoid unexpected/intermintent issues.
	 */
	struct FSoundFileDescription
	{
		/** The number of frames (interleaved samples) in the sound file. */
		int64 NumFrames = 0;

		/** The sample rate of the sound file. */
		int32 SampleRate = 0;

		/** The number of channels of the sound file. */
		int32 NumChannels = 0;

		/** The format flags of the sound file. */
		int32 FormatFlags = 0;

		/** The number of sections of the sound file. */
		int32 NumSections = 0;

		/** Whether or not the sound file is seekable. */
		int32 bIsSeekable = 0;
	};

	struct FSoundFileConvertFormat
	{
		/** Desired convert format. */
		int32 Format;

		/** Desired convert sample rate. */
		uint32 SampleRate;

		/** For compression-type target formats that used an encoding quality (0.0 = low, 1.0 = high). */
		double EncodingQuality;

		/** Whether or not to peak-normalize the audio file during import. */
		bool bPerformPeakNormalization;

		/** Creates audio engine's default source format */
		static FSoundFileConvertFormat CreateDefault()
		{
			FSoundFileConvertFormat Default = FSoundFileConvertFormat();
			Default.Format = Audio::ESoundFileFormat::WAV | Audio::ESoundFileFormat::PCM_SIGNED_16;
			Default.SampleRate = 48000;
			Default.EncodingQuality = 1.0;
			Default.bPerformPeakNormalization = false;

			return MoveTemp(Default);
		}
	};
	
	/**
	 * FSoundFileChunkInfo which maps to libsndfile SF_CHUNK_INFO struct.
	 */

	struct FSoundFileChunkInfo
	{
		/** Chunk Id **/
		ANSICHAR ChunkId[64];

		/** Size of the Chunk Id **/
		uint32 ChunkIdSize = 0;

		/** Size of the data in this chunk **/
		uint32 DataLength = 0;

		/** Pointer to chunk data **/
		void* DataPtr = nullptr;
	};

	/**
	 * FSoundFileCuePoint which maps to libsndfile SF_CUE_POINT struct.
	 */
	struct FSoundFileCuePoint
	{
		int32 CueId = 0;        
		uint32 Position = 0;    
		int32 ChunkId = 0;      
		int32 ChunkStart = 0;   
		int32 BlockStart = 0;   
		uint32 SampleOffset = 0;
		char Name[256];         
	};

	/**
	 * FSoundFileCues which maps to libsndfile SF_CUES struct.
	 */
	struct FSoundFileCues
	{
		uint32 CueCount = 0;
		FSoundFileCuePoint CuePoints[100];
	};
	
	// ChunkID: 'cue ' 
	// A cue chunk specifies one or more sample offsets which are often used to mark noteworthy sections of audio. For example, 
	// the beginning and end of a verse in a song may have cue points to make them easier to find. The cue chunk is optional and 
	// if included, a single cue chunk should specify all cue points for the "WAVE" chunk. 
	// No more than one cue chunk is allowed in a "WAVE" chunk.
	struct FRiffCueChunk
	{
		uint32 ChunkID;			// 'cue '
		uint32 ChunkDataSize;	// Depends on the number of cue points
		uint32 NumCuePoints;	// Number of cue points in the list
	};
	
	struct FRiffCuePointChunk
	{
		uint32 CueID;			// Unique ID value for the cue point
		uint32 Position;		// Play order position
		uint32 DataChunkID;		// RIFF ID of corresponding data chunk
		uint32 ChunkStart;		// Byte offset of data chunk
		uint32 BlockStart;		// Byte offset of sample of first channel
		uint32 SampleOffset;	// Byte offset to sample byte of first channel
	};
	
	struct FRiffSampleChunk
	{
		uint32 ChunkID;				// 'smpl'
		uint32 ChunkDataSize;		// Depends on the number of sample loops
		uint32 ManufacturerCode;	// The MIDI Manufacturers Association manufacturer code
		uint32 Product;				// The Product / Model ID of the target device, specific to the manufacturer
		uint32 SamplePeriod;		// The period of one sample in nanoseconds.
		uint32 MidiUnityNote;		// The MIDI note that will play when this sample is played at its current pitch
		uint32 MidiPitchFraction;	// The fraction of a semitone up from the specified note. 
		uint32 SmpteFormat;			// The SMPTE format. Possible values: 0, 24, 25, 29, 30
		uint32 SmpteOffset;			// Specifies a time offset for the sample, if the sample should start at a later time and not immediately.
		uint32 NumSampleLoops;		// Number of sample loops contained in this chunks data
		uint32 NumSampleDataBytes;	// Number of bytes of optional sampler specific data that follows the sample loops. zero if there is no such data.
	};
	
	struct FRiffSampleLoopChunk
	{
		uint32 LoopID;			// A unique ID of the loop, which could be a cue point
		uint32 LoopType;		// The loop type. 0: Forward Looping, 1: Ping-Pong, 2: Backward, 3-31: future standard types. >=32: manufacturer specific types
		uint32 StartFrame;		// Start point of the loop in samples
		uint32 EndFrame;		// End point of the loop in samples. The end sample is also played.
		uint32 Fraction;		// The resolution at which this loop should be fine tuned.
		uint32 NumPlayTimes;	// The number of times to play the loop. A value of zero means inifity. In a Midi sampler, that may mean infinite sustain.
	};
	
	struct FRiffListChunk
	{
		uint32 ChunkID;			// 'list'
		uint32 ChunkDataSize;	// Depends on contained text
		uint32 TypeID;			// always 'adtl'
	};
	
	struct FRiffLabelChunk
	{
		uint32 ChunkID;			// 'labl'
		uint32 ChunkDataSize;	// depends on contained text
		uint32 CuePointID;		// Cue Point ID associated with the label
	};

	/**
	 * FSoundFileChunkInfoWrapper wraps FSoundFileChunkInfo and manages
	 * chunk data memory.
	 */
	class FSoundFileChunkInfoWrapper
	{
	public:
		FSoundFileChunkInfoWrapper() = default;
		~FSoundFileChunkInfoWrapper() = default;
		FSoundFileChunkInfoWrapper(const FSoundFileChunkInfoWrapper& Other) = delete;
		FSoundFileChunkInfoWrapper& operator=(const FSoundFileChunkInfoWrapper& Other) = delete;

		FSoundFileChunkInfoWrapper(FSoundFileChunkInfoWrapper&& Other) noexcept
		{
			if (Other.ChunkInfo.ChunkIdSize)
			{
				FMemory::Memcpy(ChunkInfo.ChunkId, Other.ChunkInfo.ChunkId, sizeof(ChunkInfo.ChunkId));
			}
			ChunkInfo.ChunkIdSize = Other.ChunkInfo.ChunkIdSize;
			ChunkInfo.DataLength = Other.ChunkInfo.DataLength;
			ChunkInfo.DataPtr = Other.ChunkInfo.DataPtr;

			ChunkData = MoveTemp(Other.ChunkData);
		}

		FSoundFileChunkInfoWrapper& operator=(FSoundFileChunkInfoWrapper&& Other) noexcept
		{
			if (Other.ChunkInfo.ChunkIdSize)
			{
				FMemory::Memcpy(ChunkInfo.ChunkId, Other.ChunkInfo.ChunkId, sizeof(ChunkInfo.ChunkId));
			}
			ChunkInfo.ChunkIdSize = Other.ChunkInfo.ChunkIdSize;
			ChunkInfo.DataLength = Other.ChunkInfo.DataLength;
			ChunkInfo.DataPtr = Other.ChunkInfo.DataPtr;

			ChunkData = MoveTemp(Other.ChunkData);
			
			return *this;
		}

		void AllocateChunkData()
		{
			if (ChunkInfo.DataLength > 0 && ensure(ChunkInfo.DataPtr == nullptr))
			{
				ChunkData = MakeUnique<uint8[]>(ChunkInfo.DataLength);
				ChunkInfo.DataPtr = ChunkData.Get();
			}
		}

		FSoundFileChunkInfo* GetPtr()
		{
			return &ChunkInfo;
		}

		const FSoundFileChunkInfo* GetPtr() const
		{
			return &ChunkInfo;
		}

	private:
		FSoundFileChunkInfo	ChunkInfo;
		TUniquePtr<uint8[]>	ChunkData;
	};
	typedef TArray<FSoundFileChunkInfoWrapper> FSoundFileChunkArray;
	
	/**
	 * ISoundFile
	 */
	class ISoundFile
	{
	public:
		virtual ~ISoundFile() = default;
		virtual ESoundFileError::Type GetState(ESoundFileState::Type& OutState) const = 0;
		virtual ESoundFileError::Type GetError() const = 0;
		virtual ESoundFileError::Type GetId(uint32& OutId) const = 0;
		virtual ESoundFileError::Type GetPath(FName& OutPath) const = 0;
		virtual ESoundFileError::Type GetBulkData(TArray<uint8>** OutData) const = 0;
		virtual ESoundFileError::Type GetDataSize(int32& DataSize) const = 0;
		virtual ESoundFileError::Type GetDescription(FSoundFileDescription& OutDescription) const = 0;
		virtual ESoundFileError::Type GetChannelMap(TArray<ESoundFileChannelMap::Type>& OutChannelMap) const = 0;
		virtual ESoundFileError::Type IsStreamed(bool& bOutIsStreamed) const = 0;
	};

	class ISoundFileReader
	{
	public:
		virtual ~ISoundFileReader() = default;

		virtual ESoundFileError::Type Init(TSharedPtr<ISoundFile> InSoundFileData, bool bIsStreamed) = 0;
		virtual ESoundFileError::Type Init(const TArray<uint8>* InData) = 0;
		virtual ESoundFileError::Type Release() = 0;
		virtual ESoundFileError::Type SeekFrames(SoundFileCount Offset, ESoundFileSeekMode::Type SeekMode, SoundFileCount& OutOffset) = 0;
		virtual ESoundFileError::Type ReadFrames(float* DataPtr, SoundFileCount NumFrames, SoundFileCount& OutNumFramesRead) = 0;
		virtual ESoundFileError::Type ReadFrames(double* DataPtr, SoundFileCount NumFrames, SoundFileCount& OutNumFramesRead) = 0;
		virtual ESoundFileError::Type ReadSamples(float* DataPtr, SoundFileCount NumSamples, SoundFileCount& OutNumSamplesRead) = 0;
		virtual ESoundFileError::Type ReadSamples(double* DataPtr, SoundFileCount NumSamples, SoundFileCount& OutNumSamplesRead) = 0;
		virtual ESoundFileError::Type GetDescription(FSoundFileDescription& OutputDescription, TArray<ESoundFileChannelMap::Type>& OutChannelMap) = 0;
		virtual ESoundFileError::Type GetOptionalChunks(FSoundFileChunkArray& OutChunkInfoArray, const TSet<uint32>& ChunkIdsToSkip = {}) = 0;
	};

	class ISoundFileWriter
	{
	public:
		virtual ~ISoundFileWriter() = default;

		virtual ESoundFileError::Type Init(const FSoundFileDescription& FileDescription, const TArray<ESoundFileChannelMap::Type>& InChannelMap, double EncodingQuality) = 0;
		virtual ESoundFileError::Type Release() = 0;
		virtual ESoundFileError::Type SeekFrames(SoundFileCount Offset, ESoundFileSeekMode::Type SeekMode, SoundFileCount& OutOffset) = 0;
		virtual ESoundFileError::Type WriteFrames(const float* Data, SoundFileCount NumFrames, SoundFileCount& OutNumFramesWritten) = 0;
		virtual ESoundFileError::Type WriteFrames(const double* Data, SoundFileCount NumFrames, SoundFileCount& OutNumFramesWritten) = 0;
		virtual ESoundFileError::Type WriteSamples(const float* DataPtr, SoundFileCount NumSamples, SoundFileCount& OutNumSampleWritten) = 0;
		virtual ESoundFileError::Type WriteSamples(const double* DataPtr, SoundFileCount NumSamples, SoundFileCount& OutNumSampleWritten) = 0;
		virtual ESoundFileError::Type GetData(TArray<uint8>** OutData) = 0;
		virtual ESoundFileError::Type WriteOptionalChunks(const FSoundFileChunkArray& ChunkInfoArray) = 0;
		virtual ESoundFileError::Type WriteByteArrayChunk(TArray<uint8>& InSoundFileChunk) = 0;
		virtual ESoundFileError::Type WriteCueCommandData(FSoundFileCues& InSoundFileData) = 0;
	};
} // namespace Audio
