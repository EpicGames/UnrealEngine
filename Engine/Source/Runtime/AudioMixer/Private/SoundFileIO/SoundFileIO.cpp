// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundFileIO/SoundFileIO.h"

#include "CoreMinimal.h"

#include "Async/AsyncWork.h"
#include "AudioMixerDevice.h"
#include "Engine/Engine.h"
#include "EngineGlobals.h"
#include "Misc/FileHelper.h"
#include "Modules/ModuleManager.h"
#include "SoundFileIOManager.h"
#include "SoundFile.h"
#include "SoundFileIOEnums.h"
#include "Stats/Stats.h"


namespace Audio::SoundFileUtils
{
	static void CopyOptionalWavChunks(TSharedPtr<ISoundFileReader>& InSoundDataReader, const int32 InInputFormat, TSharedPtr<ISoundFileWriter>& InSoundFileWriter, const int32 InOutputFormat, const TSet<uint32>& ChunkIdsToSkip = {});

	bool AUDIOMIXER_API InitSoundFileIOManager()
	{
		return Audio::SoundFileIOManagerInit();
	}

	bool AUDIOMIXER_API ShutdownSoundFileIOManager()
	{
		return Audio::SoundFileIOManagerShutdown();
	}

	uint32 AUDIOMIXER_API GetNumSamples(const TArray<uint8>& InAudioData)
	{
		FSoundFileIOManager SoundIOManager;
		TSharedPtr<ISoundFileReader> InputSoundDataReader = SoundIOManager.CreateSoundDataReader();

		ESoundFileError::Type Error = InputSoundDataReader->Init(&InAudioData);
		if (Error != ESoundFileError::Type::NONE)
		{
			return 0;
		}

		TArray<ESoundFileChannelMap::Type> ChannelMap;

		FSoundFileDescription InputDescription;
		InputSoundDataReader->GetDescription(InputDescription, ChannelMap);
		InputSoundDataReader->Release();

		return InputDescription.NumFrames * InputDescription.NumChannels;
	}

	bool AUDIOMIXER_API ConvertAudioToWav(const TArray<uint8>& InAudioData, TArray<uint8>& OutWaveData)
	{
		const FSoundFileConvertFormat ConvertFormat = FSoundFileConvertFormat::CreateDefault();

		FSoundFileIOManager SoundIOManager;
		TSharedPtr<ISoundFileReader> InputSoundDataReader = SoundIOManager.CreateSoundDataReader();
		
		ESoundFileError::Type Error = InputSoundDataReader->Init(&InAudioData);
		if (Error != ESoundFileError::Type::NONE)
		{
			return false;
		}

		TArray<ESoundFileChannelMap::Type> ChannelMap;
		
		FSoundFileDescription InputDescription;
		InputSoundDataReader->GetDescription(InputDescription, ChannelMap);

		FSoundFileDescription NewSoundFileDescription;
		NewSoundFileDescription.NumChannels = InputDescription.NumChannels;
		NewSoundFileDescription.NumFrames = InputDescription.NumFrames;
		NewSoundFileDescription.FormatFlags = ConvertFormat.Format;
		NewSoundFileDescription.SampleRate = InputDescription.SampleRate;
		NewSoundFileDescription.NumSections = InputDescription.NumSections;
		NewSoundFileDescription.bIsSeekable = InputDescription.bIsSeekable;

		TSharedPtr<ISoundFileWriter> SoundFileWriter = SoundIOManager.CreateSoundFileWriter();
		Error = SoundFileWriter->Init(NewSoundFileDescription, ChannelMap, ConvertFormat.EncodingQuality);
		if (Error != ESoundFileError::Type::NONE)
		{
			return false;
		}

		// Copy optional chunks before writing data chunk which libsndfile assumes will be the last chunk
		CopyOptionalWavChunks(InputSoundDataReader, InputDescription.FormatFlags, SoundFileWriter, NewSoundFileDescription.FormatFlags);

		// Create a buffer to do the processing 
		SoundFileCount ProcessBufferSamples = static_cast<SoundFileCount>(1024) * NewSoundFileDescription.NumChannels;
		TArray<float> ProcessBuffer;
		ProcessBuffer.Init(0.0f, ProcessBufferSamples);

		// Find the max value if we've been told to do peak normalization on import
		float MaxValue = 0.0f;
		SoundFileCount InputSamplesRead = 0;
		bool bPerformPeakNormalization = ConvertFormat.bPerformPeakNormalization;
		if (bPerformPeakNormalization)
		{
			Error = InputSoundDataReader->ReadSamples(ProcessBuffer.GetData(), ProcessBufferSamples, InputSamplesRead);
			check(Error == ESoundFileError::Type::NONE);

			while (InputSamplesRead)
			{
				for (SoundFileCount Sample = 0; Sample < InputSamplesRead; ++Sample)
				{
					if (ProcessBuffer[Sample] > FMath::Abs(MaxValue))
					{
						MaxValue = ProcessBuffer[Sample];
					}
				}

				Error = InputSoundDataReader->ReadSamples(ProcessBuffer.GetData(), ProcessBufferSamples, InputSamplesRead);
				check(Error == ESoundFileError::Type::NONE);
			}

			// If this happens, it means we have a totally silent file
			if (MaxValue == 0.0)
			{
				bPerformPeakNormalization = false;
			}

			// Seek the file back to the beginning
			SoundFileCount OutOffset;
			InputSoundDataReader->SeekFrames(0, ESoundFileSeekMode::FROM_START, OutOffset);
		}

		bool SamplesProcessed = true;

		// Read the first block of samples
		Error = InputSoundDataReader->ReadSamples(ProcessBuffer.GetData(), ProcessBufferSamples, InputSamplesRead);
		check(Error == ESoundFileError::Type::NONE);

		// Normalize and clamp the input decoded audio
		if (bPerformPeakNormalization)
		{
			for (int32 Sample = 0; Sample < InputSamplesRead; ++Sample)
			{
				ProcessBuffer[Sample] = FMath::Clamp(ProcessBuffer[Sample] / MaxValue, -1.0f, 1.0f);
			}
		}
		else
		{
			for (int32 Sample = 0; Sample < InputSamplesRead; ++Sample)
			{
				ProcessBuffer[Sample] = FMath::Clamp(ProcessBuffer[Sample], -1.0f, 1.0f);
			}
		}

		SoundFileCount SamplesWritten = 0;
		SoundFileCount TotalSamplesRead = InputSamplesRead;

		while (InputSamplesRead == ProcessBuffer.Num())
		{
			Error = SoundFileWriter->WriteSamples((const float*)ProcessBuffer.GetData(), InputSamplesRead, SamplesWritten);
			check(Error == ESoundFileError::Type::NONE);
			check(SamplesWritten == InputSamplesRead);

			// read more samples
			Error = InputSoundDataReader->ReadSamples(ProcessBuffer.GetData(), ProcessBufferSamples, InputSamplesRead);
			check(Error == ESoundFileError::Type::NONE);

			// Normalize and clamp the samples
			if (bPerformPeakNormalization)
			{
				for (int32 Sample = 0; Sample < InputSamplesRead; ++Sample)
				{
					ProcessBuffer[Sample] = FMath::Clamp(ProcessBuffer[Sample] / MaxValue, -1.0f, 1.0f);
				}
			}
			else
			{
				for (int32 Sample = 0; Sample < InputSamplesRead; ++Sample)
				{
					ProcessBuffer[Sample] = FMath::Clamp(ProcessBuffer[Sample], -1.0f, 1.0f);
				}
			}

			TotalSamplesRead += InputSamplesRead;

			if (TotalSamplesRead > static_cast<SoundFileCount>(INT32_MAX))
			{
				UE_LOGF(LogAudioMixer, Error, "Error encountered while converting audio to wav, file too large... skipping");

				return false;
			}
		}

		// Write final block of samples
		Error = SoundFileWriter->WriteSamples((const float*)ProcessBuffer.GetData(), InputSamplesRead, SamplesWritten);
		check(Error == ESoundFileError::Type::NONE);

		// Release the sound file handles as soon as we finished converting the file
		InputSoundDataReader->Release();
		SoundFileWriter->Release();

		// Get the raw binary data.....
		TArray<uint8>* Data = nullptr;
		SoundFileWriter->GetData(&Data);

		OutWaveData.Init(0, Data->Num());
		FMemory::Memcpy(OutWaveData.GetData(), (const void*)&(*Data)[0], OutWaveData.Num());

		return true;
	}

	void CopyOptionalWavChunks(TSharedPtr<ISoundFileReader>& InSoundDataReader, const int32 InInputFormat, TSharedPtr<ISoundFileWriter>& InSoundFileWriter, const int32 InOutputFormat, const TSet<uint32>& ChunkIdsToSkip)
	{
		// libsndfile only supports chunk operations with wave file formats
		if ((InInputFormat & ESoundFileFormat::WAV) && (InOutputFormat & ESoundFileFormat::WAV))
		{
			// Get the optional chunks from the input data
			FSoundFileChunkArray OptionalChunks;
			ESoundFileError::Type Error = InSoundDataReader->GetOptionalChunks(OptionalChunks, ChunkIdsToSkip);
			if (Error != ESoundFileError::Type::NONE)
			{
				UE_LOGF(LogAudioMixer, Error, "Error encountered while reading optional chunk data...skipping");
			}
			else
			{
				// Copy any chunks found over to the output file
				Error = InSoundFileWriter->WriteOptionalChunks(OptionalChunks);
				if (Error != ESoundFileError::Type::NONE)
				{
					UE_LOGF(LogAudioMixer, Error, "Error encountered while writing optional chunk data...skipping");
				}
			}
		}
	}

	static void WriteUInt32ToByteArray(TArray<uint8>& InOutByteArray, int32& InOutIndex, const uint32 InValue)
	{
		check(InOutByteArray.Num() >= InOutIndex + 4);
		
		InOutByteArray[InOutIndex++] = (uint8)(InValue >> 0);
		InOutByteArray[InOutIndex++] = (uint8)(InValue >> 8);
		InOutByteArray[InOutIndex++] = (uint8)(InValue >> 16);
		InOutByteArray[InOutIndex++] = (uint8)(InValue >> 24);
	}

	bool AUDIOMIXER_API CreateCueAndSampleChunks(const TArray<uint8>& InAudioData, TArray<uint8>& OutWaveData, const TArray<FWaveCue>& InWaveCues, const TArray<FWaveSampleLoop>& InSampleLoops)
	{
		const FSoundFileConvertFormat ConvertFormat = FSoundFileConvertFormat::CreateDefault();

		FSoundFileIOManager SoundIOManager;
		TSharedPtr<ISoundFileReader> InputSoundDataReader = SoundIOManager.CreateSoundDataReader();
		check(InputSoundDataReader);

		ESoundFileError::Type Error = InputSoundDataReader->Init(&InAudioData);
		if (Error != ESoundFileError::Type::NONE)
		{
			UE_LOGF(LogAudioMixer, Error, "Error encountered while initializing InputSoundDataReader... returning false");
	
			return false;
		}

		TArray<ESoundFileChannelMap::Type> ChannelMap;

		FSoundFileDescription InputDescription;
		InputSoundDataReader->GetDescription(InputDescription, ChannelMap);

		FSoundFileDescription NewSoundFileDescription;
		NewSoundFileDescription.NumChannels = InputDescription.NumChannels;
		NewSoundFileDescription.NumFrames = InputDescription.NumFrames;
		NewSoundFileDescription.FormatFlags = ConvertFormat.Format;
		NewSoundFileDescription.SampleRate = InputDescription.SampleRate;
		NewSoundFileDescription.NumSections = InputDescription.NumSections;
		NewSoundFileDescription.bIsSeekable = InputDescription.bIsSeekable;

		TSharedPtr<ISoundFileWriter> SoundFileWriter = SoundIOManager.CreateSoundFileWriter();
		check(SoundFileWriter);
		
		Error = SoundFileWriter->Init(NewSoundFileDescription, ChannelMap, ConvertFormat.EncodingQuality);
		if (Error != ESoundFileError::Type::NONE)
		{
			UE_LOGF(LogAudioMixer, Error, "Error encountered while initializing SoundFileWriter... returning false");
	
			return false;
		}

		//Skip these chunkIds because we are overwriting them
		TSet<uint32> ChunkIdsToSkip;
		ChunkIdsToSkip.Reserve(3);

		if(!InWaveCues.IsEmpty())
		{
			ChunkIdsToSkip.Add(FWaveModInfo::GetChunkId("cue "));
			ChunkIdsToSkip.Add(FWaveModInfo::GetChunkId("LIST"));
			ChunkIdsToSkip.Add(FWaveModInfo::GetChunkId("labl"));
		}

		if(!InSampleLoops.IsEmpty())
		{
			ChunkIdsToSkip.Add(FWaveModInfo::GetChunkId("smpl"));
		}

		// Copy optional chunks before writing data chunk which libsndfile assumes will be the last chunk
		CopyOptionalWavChunks(InputSoundDataReader, InputDescription.FormatFlags, SoundFileWriter, NewSoundFileDescription.FormatFlags, ChunkIdsToSkip);
		
		if (!InWaveCues.IsEmpty())
		{
			FSoundFileCues Cues;
			
			// Limit number of cue points to 100 for libsndfile compatibility
			Cues.CueCount = InWaveCues.Num() <= 100 ? InWaveCues.Num() : 100;
			
			for (uint32 Index = 0; Index < Cues.CueCount; ++Index)
			{
				const FWaveCue& WaveCue = InWaveCues[Index];
				
				FSoundFileCuePoint CuePoint;
				
				CuePoint.CueId = WaveCue.CuePointID;
				CuePoint.ChunkId = FWaveModInfo::GetChunkId("data");
				CuePoint.Position = WaveCue.Position;
				CuePoint.ChunkStart = 0;
				CuePoint.BlockStart = 0;
				CuePoint.SampleOffset = WaveCue.Position;

				// CuePoint.Name is currently unused by libsndfile, but retaining info in case it is updated
				FCStringAnsi::Strncpy(CuePoint.Name, TCHAR_TO_ANSI(*WaveCue.Label), sizeof(CuePoint.Name) - 1);
				
				Cues.CuePoints[Index] = CuePoint;
			}

			Error = SoundFileWriter->WriteCueCommandData(Cues);
			if (Error != ESoundFileError::Type::NONE)
			{
				UE_LOGF(LogAudioMixer, Error, "Error encountered while writing cue data... returning false");

				return false;
			}

			TArray<uint8> ListChunkData;
			int32 ChunkDataByteIndex = 0;
			int32 ListChunkSize = sizeof(FRiffListChunk);

			TMap<uint32, FRiffLabelChunk> LabelChunks;

			for (const FWaveCue& WaveCue : InWaveCues)
			{
				// If no label, set label size to 2 so that the label chunk includes a blank label or some DAWs won't render the marker 
				const uint32 LabelSize = WaveCue.Label.GetCharArray().Num() > 0 ? WaveCue.Label.GetCharArray().Num() : 2;

				FRiffLabelChunk LabelChunk;
				LabelChunk.ChunkID = FWaveModInfo::GetChunkId("labl");
				LabelChunk.ChunkDataSize = sizeof(FRiffLabelChunk::CuePointID) + LabelSize;
				LabelChunk.CuePointID = WaveCue.CuePointID;

				LabelChunks.Emplace(WaveCue.CuePointID, LabelChunk);

				ListChunkSize += LabelChunk.ChunkDataSize + sizeof(FRiffLabelChunk::ChunkID) + sizeof(FRiffLabelChunk::ChunkDataSize);

				// Padding parent chunk size, libsndfile does this for us, but it does not pad the child chunk so we need to account for
				// the size of the child chunk padding in the TArray.
				if (LabelSize % 2 != 0)
				{
					ListChunkSize++;
				}
			}

			ListChunkData.AddZeroed(ListChunkSize);

			// FieldName: chunk ID
			// FieldSize: 4 bytes
			// FieldValue: "LIST" (big endian)
			ListChunkData[ChunkDataByteIndex++] = 'L';
			ListChunkData[ChunkDataByteIndex++] = 'I';
			ListChunkData[ChunkDataByteIndex++] = 'S';
			ListChunkData[ChunkDataByteIndex++] = 'T';

			// FieldName: size
			// FieldSize: 4 bytes
			// FieldValue: The size of the LIST chunk(number of bytes) less 8 (less the "chunk ID" and the "size")
			WriteUInt32ToByteArray(ListChunkData, ChunkDataByteIndex, ListChunkSize - sizeof(FRiffCueChunk::ChunkID)
				- sizeof(FRiffCueChunk::ChunkDataSize));

			// FieldName: list type ID
			// FieldSize: 4 bytes
			// FieldValue: The ASCII character string "adtl"
			ListChunkData[ChunkDataByteIndex++] = 'a';
			ListChunkData[ChunkDataByteIndex++] = 'd';
			ListChunkData[ChunkDataByteIndex++] = 't';
			ListChunkData[ChunkDataByteIndex++] = 'l';

			for (const FWaveCue& WaveCue : InWaveCues)
			{
				FRiffLabelChunk& LabelChunk = LabelChunks[WaveCue.CuePointID];

				WriteUInt32ToByteArray(ListChunkData, ChunkDataByteIndex, LabelChunk.ChunkID);
				WriteUInt32ToByteArray(ListChunkData, ChunkDataByteIndex, LabelChunk.ChunkDataSize);
				WriteUInt32ToByteArray(ListChunkData, ChunkDataByteIndex, LabelChunk.CuePointID);

				TArray<TCHAR> Label = WaveCue.Label.GetCharArray();

				// If Label is empty, add two null characters (to ensure character alignment)
				// so that the label chunk includes a blank label or some DAWs won't render the marker
				if (Label.IsEmpty())
				{
					Label.Append({'\0', '\0'});
				}
				
				for (int32 Index = 0; Index < Label.Num(); Index++)
				{
					ListChunkData[ChunkDataByteIndex++] = static_cast<uint8>(Label[Index]);
				}

				// if the label with null terminator has an odd bytelength, pad with 1 more \0 to ensure 2 byte alignment
				// libsndfile does not pad children chunks when writing a parent chunk.
				if (Label.Num() % 2 != 0)
				{
					ListChunkData[ChunkDataByteIndex++] = '\0';
				}
			}

			Error = SoundFileWriter->WriteByteArrayChunk(ListChunkData);
			if (Error != ESoundFileError::Type::NONE)
			{
				UE_LOGF(LogAudioMixer, Error, "Error encountered while writing list chunk data... returning false");

				return false;
			}
		}

		if (!InSampleLoops.IsEmpty())
		{
			TArray<uint8> SampleChunkData;
			int32 ChunkDataByteIndex = 0;
			int32 SizeOfSampleChunk = sizeof(FRiffSampleChunk) + sizeof(FRiffSampleLoopChunk) * InSampleLoops.Num();
			SampleChunkData.AddZeroed(SizeOfSampleChunk);

			// FieldName: chunk ID
			// FieldSize: 4 bytes
			// FieldValue: "cue " (big endian)
			SampleChunkData[ChunkDataByteIndex++] = 's';
			SampleChunkData[ChunkDataByteIndex++] = 'm';
			SampleChunkData[ChunkDataByteIndex++] = 'p';
			SampleChunkData[ChunkDataByteIndex++] = 'l';

			// FieldName: size
			// FieldSize: 4 bytes
			// FieldValue: The size of the chunk(number of bytes) less 8 (less the "chunk ID" and the "size")
			WriteUInt32ToByteArray(SampleChunkData, ChunkDataByteIndex, SizeOfSampleChunk - sizeof(FRiffSampleChunk::ChunkID)
				- sizeof(FRiffSampleChunk::ChunkDataSize));

			// FieldName: manufacturer 
			// FieldSize: 4 bytes
			// FieldValue: The MIDI Manufacturers Association manufacturer code (see MIDI System Exclusive message). A value of zero implies that there is no specific manufacturer. The first byte of the four bytes specifies the number of bytes in the manufacturer code that are relevant (1 or 3). For example, Roland would be specified as 0x01000041 (0x41), where as Microsoft would be 0x03000041 (0x00 0x00 0x41)
			WriteUInt32ToByteArray(SampleChunkData, ChunkDataByteIndex, 0);

			// FieldName: product 
			// FieldSize: 4 bytes
			// FieldValue: The product / model ID of the target device, specific to the manufacturer. A value of zero means no specific product	
			WriteUInt32ToByteArray(SampleChunkData, ChunkDataByteIndex, 0);

			// FieldName: sample period 
			// FieldSize: 4 bytes
			// FieldValue: The period of one sample in nanoseconds. For example, at the sampling rate 44.1 KHz the size of one sample is (1 / 44100) * 1,000,000,000 = 22675 nanoseconds = 0x00005893	
			if (NewSoundFileDescription.SampleRate <= 0)
			{
				UE_LOGF(LogAudioMixer, Error, "Invalid sample rate (%d) in audio data; cannot write smpl chunk", NewSoundFileDescription.SampleRate);
				return false;
			}
			const int32 SamplePeriod = 1000000000 / NewSoundFileDescription.SampleRate;
			WriteUInt32ToByteArray(SampleChunkData, ChunkDataByteIndex, SamplePeriod);

			// FieldName: MIDI unity note 
			// FieldSize: 4 bytes
			// FieldValue: The MIDI note that will play when this sample is played at its current pitch. The values are between 0 and 127 (see MIDI Note On message)
			WriteUInt32ToByteArray(SampleChunkData, ChunkDataByteIndex, 0);

			// FieldName: MIDI pitch fraction 
			// FieldSize: 4 bytes
			// FieldValue: The fraction of a semitone up from the specified note. For example, one-half semitone is 50 cents and will be specified as 0x80
			WriteUInt32ToByteArray(SampleChunkData, ChunkDataByteIndex, 0);

			// FieldName: SMPTE format 
			// FieldSize: 4 bytes
			// FieldValue: The SMPTE format. Possible values are 0, 24, 25, 29, and 30 (see Time division (of a MIDI file))
			WriteUInt32ToByteArray(SampleChunkData, ChunkDataByteIndex, 0);

			// FieldName: SMPTE offset 
			// FieldSize: 4 bytes
			// FieldValue: Specifies a time offset for the sample, if the sample should start at a later time and not immediately. The first byte of this value specifies the number of hours and is in between -23 and 23. The second byte is the number of minutes and is between 0 and 59. The third byte is the number of seconds (0 to 59). The last byte is the number of frames and is between 0 and the frames specified by the SMPTE format. For example, if the SMPTE format is 24, then the number of frames is between 0 and 23
			WriteUInt32ToByteArray(SampleChunkData, ChunkDataByteIndex, 0);

			// FieldName: number of sample loops 
			// FieldSize: 4 bytes
			// FieldValue: Specifies the number of sample loops that are contained in this chunk's data
			WriteUInt32ToByteArray(SampleChunkData, ChunkDataByteIndex, InSampleLoops.Num());

			// FieldName: sample data 
			// FieldSize: 4 bytes
			// FieldValue: The number of bytes of optional sampler specific data that follows the sample loops. If there is no such data, the number of bytes is zero	
			WriteUInt32ToByteArray(SampleChunkData, ChunkDataByteIndex, 0);

			// FieldName: data
			// FieldSize: variable
			// FieldValue: A list of sample loops. Each sample loop uses 24 bytes of data
			for (const FWaveSampleLoop& SampleLoop : InSampleLoops)
			{
				FRiffSampleLoopChunk SampleLoopChunk;
				SampleLoopChunk.LoopID = SampleLoop.LoopID;
				SampleLoopChunk.LoopType = 0;
				SampleLoopChunk.StartFrame = SampleLoop.StartFrame;
				SampleLoopChunk.EndFrame = SampleLoop.EndFrame;
				SampleLoopChunk.Fraction = 0;
				SampleLoopChunk.NumPlayTimes = 0;

				WriteUInt32ToByteArray(SampleChunkData, ChunkDataByteIndex, SampleLoopChunk.LoopID);
				WriteUInt32ToByteArray(SampleChunkData, ChunkDataByteIndex, SampleLoopChunk.LoopType);
				WriteUInt32ToByteArray(SampleChunkData, ChunkDataByteIndex, SampleLoopChunk.StartFrame);
				WriteUInt32ToByteArray(SampleChunkData, ChunkDataByteIndex, SampleLoopChunk.EndFrame);
				WriteUInt32ToByteArray(SampleChunkData, ChunkDataByteIndex, SampleLoopChunk.Fraction);
				WriteUInt32ToByteArray(SampleChunkData, ChunkDataByteIndex, SampleLoopChunk.NumPlayTimes);
			}

			Error = SoundFileWriter->WriteByteArrayChunk(SampleChunkData);
			if (Error != ESoundFileError::Type::NONE)
			{
				UE_LOGF(LogAudioMixer, Error, "Error encountered while writing sample chunk data... returning false");

				return false;
			}
		}

		// Create a buffer to do the processing 
		SoundFileCount InputSamplesRead = 0;
		SoundFileCount ProcessBufferSamples = static_cast<SoundFileCount>(1024) * NewSoundFileDescription.NumChannels;
		TArray<float> ProcessBuffer;
		ProcessBuffer.Init(0.0f, ProcessBufferSamples);

		// Read the first block of samples
		Error = InputSoundDataReader->ReadSamples(ProcessBuffer.GetData(), ProcessBufferSamples, InputSamplesRead);
		if (Error != ESoundFileError::Type::NONE)
		{
			UE_LOGF(LogAudioMixer, Error, "Error encountered while ReadingSamples... returning false");

			return false;
		}

		SoundFileCount SamplesWritten = 0;

		while (InputSamplesRead == ProcessBuffer.Num())
		{
			Error = SoundFileWriter->WriteSamples((const float*)ProcessBuffer.GetData(), InputSamplesRead, SamplesWritten);
			check(Error == ESoundFileError::Type::NONE);
			check(SamplesWritten == InputSamplesRead);

			// read more samples
			Error = InputSoundDataReader->ReadSamples(ProcessBuffer.GetData(), ProcessBufferSamples, InputSamplesRead);
			if (Error != ESoundFileError::Type::NONE)
			{
				UE_LOGF(LogAudioMixer, Error, "Error encountered while ReadingSamples... returning false");

				return false;
			}

			for (int32 Sample = 0; Sample < InputSamplesRead; ++Sample)
			{
				ProcessBuffer[Sample] = FMath::Clamp(ProcessBuffer[Sample], -1.0f, 1.0f);
			}
		}

		// Write final block of samples
		Error = SoundFileWriter->WriteSamples((const float*)ProcessBuffer.GetData(), InputSamplesRead, SamplesWritten);
		check(Error == ESoundFileError::Type::NONE);

		// Release the sound file handles as soon as we finished converting the file
		InputSoundDataReader->Release();
		SoundFileWriter->Release();

		// Get the raw binary data.....
		TArray<uint8>* Data = nullptr;
		SoundFileWriter->GetData(&Data);

		OutWaveData.Init(0, Data->Num());
		FMemory::Memcpy(OutWaveData.GetData(), (const void*)&(*Data)[0], OutWaveData.Num());

		return true;
	}
}
