// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"


namespace Audio
{
	namespace ESoundFileError
	{
		enum class Type : uint8
		{
			NONE = 0,
			INVALID_SOUND_FILE,
			INVALID_SOUND_FILE_HANDLE,
			BAD_ENCODING_QUALITY,
			FAILED_TO_LOAD_BYTE_DATA,
			ALREADY_OPENED,
			ALREADY_HAS_DATA,
			INVALID_DATA,
			FILE_DOESNT_EXIST,
			INVALID_INPUT_FORMAT,
			INVALID_CHANNEL_MAP,
			FAILED_TO_OPEN,
			FAILED_TO_SEEK,
			ALREADY_INITIALIZED,
			LOADING,
			INVALID_STATE,
			INVALID_CHUNK,
			UNKNOWN
		};

		inline const TCHAR* ToString(ESoundFileError::Type SoundFileError)
		{
			switch (SoundFileError)
			{
			case Type::NONE:						return TEXT("NONE");
			case Type::INVALID_SOUND_FILE:			return TEXT("INVALID_SOUND_FILE");
			case Type::INVALID_SOUND_FILE_HANDLE:	return TEXT("INVALID_SOUND_FILE_HANDLE");
			case Type::BAD_ENCODING_QUALITY:		return TEXT("BAD_ENCODING_QUALITY");
			case Type::FAILED_TO_LOAD_BYTE_DATA:	return TEXT("FAILED_TO_LOAD_BYTE_DATA");
			case Type::ALREADY_OPENED:				return TEXT("ALREADY_OPENED");
			case Type::ALREADY_HAS_DATA:			return TEXT("ALREADY_HAS_DATA");
			case Type::INVALID_DATA:				return TEXT("INVALID_DATA");
			case Type::FILE_DOESNT_EXIST:			return TEXT("FILE_DOESNT_EXIST");
			case Type::INVALID_INPUT_FORMAT:		return TEXT("INVALID_INPUT_FORMAT");
			case Type::INVALID_CHANNEL_MAP:			return TEXT("INVALID_CHANNEL_MAP");
			case Type::FAILED_TO_OPEN:				return TEXT("FAILED_TO_OPEN");
			case Type::FAILED_TO_SEEK:				return TEXT("FAILED_TO_SEEK");
			case Type::ALREADY_INITIALIZED:			return TEXT("ALREADY_INITIALIZED");
			case Type::LOADING:						return TEXT("LOADING");
			case Type::INVALID_STATE:				return TEXT("INVALID_STATE");
			default: case Type::UNKNOWN:			return TEXT("UNKNOWN");
			}
		}
	} // namespace ESoundFileError

	namespace ESoundFileSeekMode
	{
		enum Type
		{
			FROM_START = 0,
			FROM_CURRENT = 1,
			FROM_END = 2,
		};
	} // namespace ESoundFileSeekMode

	/**
	 * Specifies the major format type of the sound source.
	 * File formats are fully specified by a major/minor format.
	 *
	 * For example, a Ogg-Vorbis encoding would use:
	 * uint32 FormatFlags = ESoundFormatFlags::OGG | ESoundFormatFlags::VORBIS;
	 */
	namespace ESoundFileFormat
	{
		enum Flags
		{
			// Major Formats
			WAV = 0x010000,		// Microsoft WAV format
			AIFF = 0x020000,		// Apple AIFF format
			FLAC = 0x170000,		// FLAC lossless
			OGG = 0x200000,		// Xiph OGG

			// Uncompressed Minor Formats
			PCM_SIGNED_8 = 0x0001,		// Signed 8 bit PCM
			PCM_SIGNED_16 = 0x0002,		// Signed 16 bit PCM
			PCM_SIGNED_24 = 0x0003,		// Signed 24 bit PCM
			PCM_SIGNED_32 = 0x0004,		// Signed 32 bit PCM
			PCM_UNSIGNED_8 = 0x0005,		// Unsigned 8 bit PCM
			PCM_FLOAT = 0x0006,		// 32 bit float
			PCM_DOUBLE = 0x0007,		// 64 bit float

			// Compressed Minor Formats
			MU_LAW = 0x0010,		// Mu-law encoding
			A_LAW = 0x0011,		// A-law encoding
			IMA_ADPCM = 0x0012,		// IMA ADPCM encoding
			MS_ADPCM = 0x0013,		// Microsoft ADPCM encoding
			GSM_610 = 0x0020,		// GSM 6.10 encoding
			G721_32 = 0x0030,		// 32 kbps G721 ADPCM encoding
			G723_24 = 0x0031,		// 23 kbps G723 ADPCM encoding
			G723_40 = 0x0032,		// 40 kbps G723 ADPCM encoding
			DWVW_12 = 0x0040,		// 12 bit delta-width variable word encoding
			DMVW_16 = 0x0041,		// 16 bit delta-width variable word encoding
			DMVW_24 = 0x0042,		// 24 bit delta-width variable word encoding
			DMVW_N = 0x0043,		// N bit delta-width variable word encoding
			VORBIS = 0x0060,		// Xiph vorbis encoding

			// Endian opts
			ENDIAN_FILE = 0x00000000,	// default file endian
			ENDIAN_LITTLE = 0x10000000,	// little-endian
			ENDIAN_BIG = 0x20000000,	// big-endian
			ENDIAN_CPU = 0x30000000,	// cpu-endian

			// Masks
			MINOR_FORMAT_MASK = 0x0000FFFF,
			MAJOR_FORMAT_MASK = 0x0FFF0000,
			ENDIAN_MASK = 0x30000000,
		};

		inline const TCHAR* ToStringMajor(int32 FormatFlags)
		{
			switch (FormatFlags & ESoundFileFormat::MAJOR_FORMAT_MASK)
			{
			case ESoundFileFormat::WAV:		return TEXT("WAV");
			case ESoundFileFormat::AIFF:	return TEXT("AIFF");
			case ESoundFileFormat::FLAC:	return TEXT("FLAC");
			case ESoundFileFormat::OGG:		return TEXT("OGG");
			default:						return TEXT("INVALID");
			}
		}

		inline const TCHAR* ToStringMinor(int32 FormatFlags)
		{
			switch (FormatFlags & ESoundFileFormat::MINOR_FORMAT_MASK)
			{
			case ESoundFileFormat::PCM_SIGNED_8:	return TEXT("PCM_SIGNED_8");
			case ESoundFileFormat::PCM_SIGNED_16:	return TEXT("PCM_SIGNED_16");
			case ESoundFileFormat::PCM_SIGNED_24:	return TEXT("PCM_SIGNED_24");
			case ESoundFileFormat::PCM_SIGNED_32:	return TEXT("PCM_SIGNED_32");
			case ESoundFileFormat::PCM_UNSIGNED_8:	return TEXT("PCM_UNSIGNED_8");
			case ESoundFileFormat::PCM_FLOAT:		return TEXT("PCM_FLOAT");
			case ESoundFileFormat::PCM_DOUBLE:		return TEXT("PCM_DOUBLE");
			case ESoundFileFormat::MU_LAW:			return TEXT("MU_LAW");
			case ESoundFileFormat::A_LAW:			return TEXT("A_LAW");
			case ESoundFileFormat::IMA_ADPCM:		return TEXT("IMA_ADPCM");
			case ESoundFileFormat::MS_ADPCM:		return TEXT("MS_ADPCM");
			case ESoundFileFormat::GSM_610:			return TEXT("GSM_610");
			case ESoundFileFormat::G721_32:			return TEXT("G721_32");
			case ESoundFileFormat::G723_24:			return TEXT("G723_24");
			case ESoundFileFormat::G723_40:			return TEXT("G723_40");
			case ESoundFileFormat::DWVW_12:			return TEXT("DWVW_12");
			case ESoundFileFormat::DMVW_16:			return TEXT("DMVW_16");
			case ESoundFileFormat::DMVW_24:			return TEXT("DMVW_24");
			case ESoundFileFormat::DMVW_N:			return TEXT("DMVW_N");
			case ESoundFileFormat::VORBIS:			return TEXT("VORBIS");
			default:								return TEXT("INVALID");
			}
		}
	} // namespace ESoundFileFormat;

	/*
	 * Enumeration to specify a sound files intended output channel mapping.
	 * @note These are separated from the device channel mappings purposefully since
	 * the enumeration may not exactly be the same as the output speaker mapping.
	 */
	namespace ESoundFileChannelMap
	{
		// this is used to populate an array which is passed into a sound file API call so must be uint32
		enum class Type : uint32
		{
			INVALID = 0,
			MONO,
			LEFT,
			RIGHT,
			CENTER,
			FRONT_LEFT,
			FRONT_RIGHT,
			FRONT_CENTER,
			BACK_CENTER,
			BACK_LEFT,
			BACK_RIGHT,
			LFE,
			LEFT_CENTER,
			RIGHT_CENTER,
			SIDE_LEFT,
			SIDE_RIGHT,
			TOP_CENTER,
			TOP_FRONT_LEFT,
			TOP_FRONT_RIGHT,
			TOP_FRONT_CENTER,
			TOP_BACK_LEFT,
			TOP_BACK_RIGHT,
			TOP_BACK_CENTER,
		};

		inline const TCHAR* ToString(ESoundFileChannelMap::Type ChannelMap)
		{
			switch (ChannelMap)
			{
			case Type::INVALID:				return TEXT("INVALID");
			case Type::MONO:				return TEXT("MONO");
			case Type::LEFT:				return TEXT("LEFT");
			case Type::RIGHT:				return TEXT("RIGHT");
			case Type::CENTER:				return TEXT("CENTER");
			case Type::FRONT_LEFT:			return TEXT("FRONT_LEFT");
			case Type::FRONT_RIGHT:			return TEXT("FRONT_RIGHT");
			case Type::FRONT_CENTER:		return TEXT("FRONT_CENTER");
			case Type::BACK_CENTER:			return TEXT("BACK_CENTER");
			case Type::BACK_LEFT:			return TEXT("BACK_LEFT");
			case Type::BACK_RIGHT:			return TEXT("BACK_RIGHT");
			case Type::LFE:					return TEXT("LFE");
			case Type::LEFT_CENTER:			return TEXT("LEFT_CENTER");
			case Type::RIGHT_CENTER:		return TEXT("RIGHT_CENTER");
			case Type::SIDE_LEFT:			return TEXT("SIDE_LEFT");
			case Type::SIDE_RIGHT:			return TEXT("SIDE_RIGHT");
			case Type::TOP_CENTER:			return TEXT("TOP_CENTER");
			case Type::TOP_FRONT_LEFT:		return TEXT("TOP_FRONT_LEFT");
			case Type::TOP_FRONT_RIGHT:		return TEXT("TOP_FRONT_RIGHT");
			case Type::TOP_FRONT_CENTER:	return TEXT("TOP_FRONT_CENTER");
			case Type::TOP_BACK_LEFT:		return TEXT("TOP_BACK_LEFT");
			case Type::TOP_BACK_RIGHT:		return TEXT("TOP_BACK_RIGHT");
			case Type::TOP_BACK_CENTER:		return TEXT("TOP_BACK_CENTER");
			default:						return TEXT("UNKNOWN");
			}
		}
	} // namespace ESoundFileChannelMap

	namespace ESoundFileOpenMode
	{
		enum Type
		{
			READING = 0x10,
			WRITING = 0x20,
			UNKNOWN = 0,
		};
	} // namespace ESoundFileOpenMode

	namespace ESoundFileState
	{
		enum Type
		{
			UNINITIALIZED = 0,
			INITIALIZED,
			LOADING,
			LOADED,
			STREAMING,
			WRITING,
			HAS_ERROR,
		};
	} // namespace ESoundFileState
} // namespace Audio