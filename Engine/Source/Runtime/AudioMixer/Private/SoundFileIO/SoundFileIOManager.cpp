// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundFileIOManager.h"
#include "SoundFileIOManagerImpl.h"

#include "CoreMinimal.h"
#include "Templates/UniquePtr.h"

namespace Audio
{
	FSoundFileIOManager::FSoundFileIOManager()
	{
		Impl = TUniquePtr<FSoundFileIOManagerImpl>(new FSoundFileIOManagerImpl());
	}

	FSoundFileIOManager::~FSoundFileIOManager()
	{

	}

	TSharedPtr<ISoundFileReader> FSoundFileIOManager::CreateSoundFileReader()
	{
		if (Impl.IsValid())
		{
			return Impl->CreateSoundFileReader();
		}

		return nullptr;
	}

	TSharedPtr<ISoundFileReader> FSoundFileIOManager::CreateSoundDataReader()
	{
		if (Impl.IsValid())
		{
			return Impl->CreateSoundDataReader();
		}

		return nullptr;
	}

	TSharedPtr<ISoundFileWriter> FSoundFileIOManager::CreateSoundFileWriter()
	{
		if (Impl.IsValid())
		{
			return Impl->CreateSoundFileWriter();
		}

		return nullptr;
	}

	bool FSoundFileIOManager::GetSoundFileDescription(const FString& FilePath, FSoundFileDescription& OutputDescription, TArray<ESoundFileChannelMap::Type>& OutChannelMap)
	{
		if (Impl.IsValid())
		{
			return Impl->GetSoundFileDescription(FilePath, OutputDescription, OutChannelMap);
		}

		return false;
	}

	bool FSoundFileIOManager::GetSoundFileDescription(const FString& FilePath, FSoundFileDescription& OutputDescription)
	{
		if (Impl.IsValid())
		{
			return Impl->GetSoundFileDescription(FilePath, OutputDescription);
		}

		return false;
	}

	bool FSoundFileIOManager::GetFileExtensionForFormatFlags(int32 FormatFlags, FString& OutExtension)
	{
		if (Impl.IsValid())
		{
			return Impl->GetFileExtensionForFormatFlags(FormatFlags, OutExtension);
		}

		return false;
	}

	ESoundFileError::Type FSoundFileIOManager::GetSoundFileInfoFromPath(const FString& FilePath, FSoundFileDescription& Description, TArray<ESoundFileChannelMap::Type>& ChannelMap)
	{
		if (Impl.IsValid())
		{
			return Impl->GetSoundFileInfoFromPath(FilePath, Description, ChannelMap);
		}

		return ESoundFileError::Type::UNKNOWN;
	}

	ESoundFileError::Type FSoundFileIOManager::LoadSoundFileFromPath(const FString& FilePath, FSoundFileDescription& Description, TArray<ESoundFileChannelMap::Type>& ChannelMap, TArray<uint8>& BulkData)
	{
		if (Impl.IsValid())
		{
			return Impl->LoadSoundFileFromPath(FilePath, Description, ChannelMap, BulkData);
		}

		return ESoundFileError::Type::UNKNOWN;
	}
}

