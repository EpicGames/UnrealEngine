// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "SoundFile.h"
#include "SoundFileIOEnums.h"

namespace Audio
{
	class FSoundFileIOManagerImpl;

	class FSoundFileIOManager
	{
	public:
		FSoundFileIOManager();
		~FSoundFileIOManager();

		TSharedPtr<ISoundFileReader> CreateSoundFileReader();
		TSharedPtr<ISoundFileReader> CreateSoundDataReader();
		TSharedPtr<ISoundFileWriter> CreateSoundFileWriter();

		bool GetSoundFileDescription(const FString& FilePath, FSoundFileDescription& OutputDescription, TArray<ESoundFileChannelMap::Type>& OutChannelMap);
		bool GetSoundFileDescription(const FString& FilePath, FSoundFileDescription& OutputDescription);
		bool GetFileExtensionForFormatFlags(int32 FormatFlags, FString& OutExtension);
		ESoundFileError::Type GetSoundFileInfoFromPath(const FString& FilePath, FSoundFileDescription& Description, TArray<ESoundFileChannelMap::Type>& ChannelMap);
		ESoundFileError::Type LoadSoundFileFromPath(const FString& FilePath, FSoundFileDescription& Description, TArray<ESoundFileChannelMap::Type>& ChannelMap, TArray<uint8>& BulkData);

	private:
		TUniquePtr<FSoundFileIOManagerImpl> Impl;
	};

	bool SoundFileIOManagerInit();
	bool SoundFileIOManagerShutdown();
} // namespace Audio
