// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundFileIOManagerImpl.h"

#include "CoreMinimal.h"

#include "Audio.h"
#include "AudioMixer.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"

#ifndef WITH_SNDFILE_IO
#define WITH_SNDFILE_IO (0)
#endif //WITH_SNDFILE_IO


namespace Audio
{
	typedef struct SoundFileHandleOpaque LibSoundFileHandle;
	typedef struct SoundFileChunkIteratorOpaque LibSoundFileChunkIterator;

	//	// Virtual Sound File Function Pointers
	typedef SoundFileCount(*VirtualSoundFileGetLengthFuncPtr)(void* UserData);
	typedef SoundFileCount(*VirtualSoundFileSeekFuncPtr)(SoundFileCount Offset, int32 Mode, void* UserData);
	typedef SoundFileCount(*VirtualSoundFileReadFuncPtr)(void* DataPtr, SoundFileCount ByteCount, void* UserData);
	typedef SoundFileCount(*VirtualSoundFileWriteFuncPtr)(const void* DataPtr, SoundFileCount ByteCount, void* UserData);
	typedef SoundFileCount(*VirtualSoundFileTellFuncPtr)(void* UserData);

	// Struct describing function pointers to call for virtual file IO
	struct FVirtualSoundFileCallbackInfo
	{
		VirtualSoundFileGetLengthFuncPtr VirtualSoundFileGetLength;
		VirtualSoundFileSeekFuncPtr VirtualSoundFileSeek;
		VirtualSoundFileReadFuncPtr VirtualSoundFileRead;
		VirtualSoundFileWriteFuncPtr VirtualSoundFileWrite;
		VirtualSoundFileTellFuncPtr VirtualSoundFileTell;
	};

	// SoundFile Constants
	static const int32 SET_ENCODING_QUALITY = 0x1300;
	static const int32 SET_CHANNEL_MAP_INFO = 0x1101;
	static const int32 GET_CHANNEL_MAP_INFO = 0x1100;
	static const int32 UPDATE_HEADER_NOW = 0x1060; // Always returns 0, ignore return value.
	static const int32 SET_INSTRUMENT = 0x10D1;
	static const int32 SET_CUE = 0x10CF; // Returns TRUE if the markers are written to file.

	// Exported SoundFile Functions
	typedef LibSoundFileHandle*(*SoundFileOpenFuncPtr)(const char* Path, int32 Mode, FSoundFileDescription* Description);
	typedef LibSoundFileHandle*(*SoundFileOpenVirtualFuncPtr)(FVirtualSoundFileCallbackInfo* VirtualFileDescription, int32 Mode, FSoundFileDescription* Description, void* UserData);
	typedef int32(*SoundFileCloseFuncPtr)(LibSoundFileHandle* FileHandle);
	typedef int32(*SoundFileErrorFuncPtr)(LibSoundFileHandle* FileHandle);
	typedef const char*(*SoundFileStrErrorFuncPtr)(LibSoundFileHandle* FileHandle);
	typedef const char*(*SoundFileErrorNumberFuncPtr)(int32 ErrorNumber);
	typedef int32(*SoundFileCommandFuncPtr)(LibSoundFileHandle* FileHandle, int32 Command, void* Data, int32 DataSize);
	typedef int32(*SoundFileFormatCheckFuncPtr)(const FSoundFileDescription* Description);
	typedef SoundFileCount(*SoundFileSeekFuncPtr)(LibSoundFileHandle* FileHandle, SoundFileCount NumFrames, int32 SeekMode);
	typedef const char*(*SoundFileGetVersionFuncPtr)(void);
	typedef SoundFileCount(*SoundFileReadFramesFloatFuncPtr)(LibSoundFileHandle* FileHandle, float* Buffer, SoundFileCount NumFrames);
	typedef SoundFileCount(*SoundFileReadFramesDoubleFuncPtr)(LibSoundFileHandle* FileHandle, double* Buffer, SoundFileCount NumFrames);
	typedef SoundFileCount(*SoundFileWriteFramesFloatFuncPtr)(LibSoundFileHandle* FileHandle, const float* Buffer, SoundFileCount NumFrames);
	typedef SoundFileCount(*SoundFileWriteFramesDoubleFuncPtr)(LibSoundFileHandle* FileHandle, const double* Buffer, SoundFileCount NumFrames);
	typedef SoundFileCount(*SoundFileReadSamplesFloatFuncPtr)(LibSoundFileHandle* FileHandle, float* Buffer, SoundFileCount NumSamples);
	typedef SoundFileCount(*SoundFileReadSamplesDoubleFuncPtr)(LibSoundFileHandle* FileHandle, double* Buffer, SoundFileCount NumSamples);
	typedef SoundFileCount(*SoundFileWriteSamplesFloatFuncPtr)(LibSoundFileHandle* FileHandle, const float* Buffer, SoundFileCount NumSamples);
	typedef SoundFileCount(*SoundFileWriteSamplesDoubleFuncPtr)(LibSoundFileHandle* FileHandle, const double* Buffer, SoundFileCount NumSamples);
	typedef int32(*SoundFileGetChunkSizeFuncPtr)(const LibSoundFileChunkIterator* ChunkIterator, FSoundFileChunkInfo* ChunkInfo);
	typedef int32(*SoundFileGetChunkDataFuncPtr)(const LibSoundFileChunkIterator* ChunkIterator, FSoundFileChunkInfo* ChunkInfo);
	typedef LibSoundFileChunkIterator*(*SoundFileGetChunkIteratorFuncPtr)(LibSoundFileHandle* FileHandle, const FSoundFileChunkInfo* ChunkInfo);
	typedef LibSoundFileChunkIterator*(*SoundFileNextChunkIteratorFuncPtr)(LibSoundFileChunkIterator* ChunkIterator);
	typedef int32(*SoundFileSetChunkFuncPtr)(LibSoundFileHandle* FileHandle, const FSoundFileChunkInfo* ChunkInfo);

	SoundFileOpenFuncPtr SoundFileOpen = nullptr;
	SoundFileOpenVirtualFuncPtr SoundFileOpenVirtual = nullptr;
	SoundFileCloseFuncPtr SoundFileClose = nullptr;
	SoundFileErrorFuncPtr SoundFileError = nullptr;
	SoundFileStrErrorFuncPtr SoundFileStrError = nullptr;
	SoundFileErrorNumberFuncPtr SoundFileErrorNumber = nullptr;
	SoundFileCommandFuncPtr SoundFileCommand = nullptr;
	SoundFileFormatCheckFuncPtr SoundFileFormatCheck = nullptr;
	SoundFileSeekFuncPtr SoundFileSeek = nullptr;
	SoundFileGetVersionFuncPtr SoundFileGetVersion = nullptr;
	SoundFileReadFramesFloatFuncPtr SoundFileReadFramesFloat = nullptr;
	SoundFileReadFramesDoubleFuncPtr SoundFileReadFramesDouble = nullptr;
	SoundFileWriteFramesFloatFuncPtr SoundFileWriteFramesFloat = nullptr;
	SoundFileWriteFramesDoubleFuncPtr SoundFileWriteFramesDouble = nullptr;
	SoundFileReadSamplesFloatFuncPtr SoundFileReadSamplesFloat = nullptr;
	SoundFileReadSamplesDoubleFuncPtr SoundFileReadSamplesDouble = nullptr;
	SoundFileWriteSamplesFloatFuncPtr SoundFileWriteSamplesFloat = nullptr;
	SoundFileWriteSamplesDoubleFuncPtr SoundFileWriteSamplesDouble = nullptr;
	SoundFileGetChunkSizeFuncPtr SoundFileGetChunkSize = nullptr;
	SoundFileGetChunkDataFuncPtr SoundFileGetChunkData = nullptr;
	SoundFileGetChunkIteratorFuncPtr SoundFileGetChunkIterator = nullptr;
	SoundFileNextChunkIteratorFuncPtr SoundFileNextChunkIterator = nullptr;
	SoundFileSetChunkFuncPtr SoundFileSetChunk = nullptr;

	void* SoundFileDllHandle;
	static void* GetSoundFileDllHandle()
	{
		void* DllHandle = nullptr;
#if WITH_SNDFILE_IO 
	#if PLATFORM_WINDOWS
		#if PLATFORM_CPU_ARM_FAMILY && !PLATFORM_WINDOWS_ARM64EC
			const FString PlatformPath = "Win64/arm64/";
		#else
			const FString PlatformPath = "Win64/";
		#endif // PLATFORM_CPU_ARM_FAMILY && !PLATFORM_WINDOWS_ARM64EC
		const FString DllName = "libsndfile-1.dll";
	#elif PLATFORM_MAC //PLATFORM_WINDOWS
		const FString PlatformPath = "Mac/";
		const FString DllName = "libsndfile.1.dylib";
	#elif PLATFORM_LINUX //PLATFORM_MAC
		const FString PlatformPath = "Linux/";
		const FString DllName = ("libsndfile.so.1");
	#else //PLATFORM_LINUX
		#pragma message ("Platform not supported");
		const FString PlatformPath;
		const FString DllName;
	#endif //PLATFORM_LINUX

		const FString Path = FPaths::EngineDir() / FString("Binaries/ThirdParty/libsndfile/") / PlatformPath;
		FPlatformProcess::PushDllDirectory(*Path);
		DllHandle = FPlatformProcess::GetDllHandle(*(Path + DllName));
		FPlatformProcess::PopDllDirectory(*Path);

#endif //WITH_SNDFILE_IO
		return DllHandle;
	}

	static bool LoadSoundFileLib()
	{
		SoundFileDllHandle = GetSoundFileDllHandle();
		if (!SoundFileDllHandle)
		{
			UE_LOGF(LogAudioMixer, Display, "Failed to load Sound File dll");
			return false;
		}

		bool bSuccess = true;

		void* LambdaDLLHandle = SoundFileDllHandle;

		// Helper function to load DLL exports and report warnings
		auto GetSoundFileDllExport = [&LambdaDLLHandle](const TCHAR* FuncName, bool& bInSuccess) -> void*
		{
			if (bInSuccess)
			{
				void* FuncPtr = FPlatformProcess::GetDllExport(LambdaDLLHandle, FuncName);
				if (FuncPtr == nullptr)
				{
					bInSuccess = false;
					UE_LOGF(LogAudioMixer, Warning, "Failed to locate the expected DLL import function '%ls' in the SoundFile DLL.", FuncName);
					FPlatformProcess::FreeDllHandle(LambdaDLLHandle);
					LambdaDLLHandle = nullptr;
				}
				return FuncPtr;
			}
			else
			{
				return nullptr;
			}
		};

		SoundFileOpen = (SoundFileOpenFuncPtr)GetSoundFileDllExport(TEXT("sf_open"), bSuccess);
		SoundFileOpenVirtual = (SoundFileOpenVirtualFuncPtr)GetSoundFileDllExport(TEXT("sf_open_virtual"), bSuccess);
		SoundFileClose = (SoundFileCloseFuncPtr)GetSoundFileDllExport(TEXT("sf_close"), bSuccess);
		SoundFileError = (SoundFileErrorFuncPtr)GetSoundFileDllExport(TEXT("sf_error"), bSuccess);
		SoundFileStrError = (SoundFileStrErrorFuncPtr)GetSoundFileDllExport(TEXT("sf_strerror"), bSuccess);
		SoundFileErrorNumber = (SoundFileErrorNumberFuncPtr)GetSoundFileDllExport(TEXT("sf_error_number"), bSuccess);
		SoundFileCommand = (SoundFileCommandFuncPtr)GetSoundFileDllExport(TEXT("sf_command"), bSuccess);
		SoundFileFormatCheck = (SoundFileFormatCheckFuncPtr)GetSoundFileDllExport(TEXT("sf_format_check"), bSuccess);
		SoundFileSeek = (SoundFileSeekFuncPtr)GetSoundFileDllExport(TEXT("sf_seek"), bSuccess);
		SoundFileGetVersion = (SoundFileGetVersionFuncPtr)GetSoundFileDllExport(TEXT("sf_version_string"), bSuccess);
		SoundFileReadFramesFloat = (SoundFileReadFramesFloatFuncPtr)GetSoundFileDllExport(TEXT("sf_readf_float"), bSuccess);
		SoundFileReadFramesDouble = (SoundFileReadFramesDoubleFuncPtr)GetSoundFileDllExport(TEXT("sf_readf_double"), bSuccess);
		SoundFileWriteFramesFloat = (SoundFileWriteFramesFloatFuncPtr)GetSoundFileDllExport(TEXT("sf_writef_float"), bSuccess);
		SoundFileWriteFramesDouble = (SoundFileWriteFramesDoubleFuncPtr)GetSoundFileDllExport(TEXT("sf_writef_double"), bSuccess);
		SoundFileReadSamplesFloat = (SoundFileReadSamplesFloatFuncPtr)GetSoundFileDllExport(TEXT("sf_read_float"), bSuccess);
		SoundFileReadSamplesDouble = (SoundFileReadSamplesDoubleFuncPtr)GetSoundFileDllExport(TEXT("sf_read_double"), bSuccess);
		SoundFileWriteSamplesFloat = (SoundFileWriteSamplesFloatFuncPtr)GetSoundFileDllExport(TEXT("sf_write_float"), bSuccess);
		SoundFileWriteSamplesDouble = (SoundFileWriteSamplesDoubleFuncPtr)GetSoundFileDllExport(TEXT("sf_write_double"), bSuccess);
		SoundFileGetChunkSize = (SoundFileGetChunkSizeFuncPtr)GetSoundFileDllExport(TEXT("sf_get_chunk_size"), bSuccess);
		SoundFileGetChunkData = (SoundFileGetChunkDataFuncPtr)GetSoundFileDllExport(TEXT("sf_get_chunk_data"), bSuccess);
		SoundFileGetChunkIterator = (SoundFileGetChunkIteratorFuncPtr)GetSoundFileDllExport(TEXT("sf_get_chunk_iterator"), bSuccess);
		SoundFileNextChunkIterator = (SoundFileNextChunkIteratorFuncPtr)GetSoundFileDllExport(TEXT("sf_next_chunk_iterator"), bSuccess);
		SoundFileSetChunk = (SoundFileSetChunkFuncPtr)GetSoundFileDllExport(TEXT("sf_set_chunk"), bSuccess);

		// make sure we're successful
		check(bSuccess);
		return bSuccess;
	}

	static bool ShutdownSoundFileLib()
	{
		if (SoundFileDllHandle)
		{
			FPlatformProcess::FreeDllHandle(SoundFileDllHandle);
			SoundFileDllHandle = nullptr;
			SoundFileOpen = nullptr;
			SoundFileOpenVirtual = nullptr;
			SoundFileClose = nullptr;
			SoundFileError = nullptr;
			SoundFileStrError = nullptr;
			SoundFileErrorNumber = nullptr;
			SoundFileCommand = nullptr;
			SoundFileFormatCheck = nullptr;
			SoundFileSeek = nullptr;
			SoundFileGetVersion = nullptr;
			SoundFileReadFramesFloat = nullptr;
			SoundFileReadFramesDouble = nullptr;
			SoundFileWriteFramesFloat = nullptr;
			SoundFileWriteFramesDouble = nullptr;
			SoundFileReadSamplesFloat = nullptr;
			SoundFileReadSamplesDouble = nullptr;
			SoundFileWriteSamplesFloat = nullptr;
			SoundFileWriteSamplesDouble = nullptr;
			SoundFileGetChunkSize = nullptr;
			SoundFileGetChunkData = nullptr;
			SoundFileGetChunkIterator = nullptr;
			SoundFileNextChunkIterator = nullptr;
			SoundFileSetChunk = nullptr;
		}
		return true;
	}

	/**
	Function implementations of virtual function callbacks
	*/
	class ISoundFileParser
	{
	public:
		virtual ~ISoundFileParser() {}

		virtual ESoundFileError::Type GetLengthBytes(SoundFileCount& OutLength) const = 0;
		virtual ESoundFileError::Type SeekBytes(SoundFileCount Offset, ESoundFileSeekMode::Type SeekMode, SoundFileCount& OutOffset) = 0;
		virtual ESoundFileError::Type ReadBytes(void* DataPtr, SoundFileCount NumBytes, SoundFileCount& NumBytesRead) = 0;
		virtual ESoundFileError::Type WriteBytes(const void* DataPtr, SoundFileCount NumBytes, SoundFileCount& NumBytesWritten) = 0;
		virtual ESoundFileError::Type GetOffsetBytes(SoundFileCount& OutOffset) const = 0;
	};

	///**
	//* Gets the default channel mapping for the given channel number
	//*/
	static void GetDefaultMappingsForChannelNumber(int32 NumChannels, TArray<ESoundFileChannelMap::Type>& ChannelMap)
	{
		check(ChannelMap.Num() == NumChannels);

		switch (NumChannels)
		{
		case 1:	// MONO
			ChannelMap[0] = ESoundFileChannelMap::Type::MONO;
			break;

		case 2:	// STEREO
			ChannelMap[0] = ESoundFileChannelMap::Type::LEFT;
			ChannelMap[1] = ESoundFileChannelMap::Type::RIGHT;
			break;

		case 3:	// 2.1
			ChannelMap[0] = ESoundFileChannelMap::Type::LEFT;
			ChannelMap[1] = ESoundFileChannelMap::Type::RIGHT;
			ChannelMap[2] = ESoundFileChannelMap::Type::LFE;
			break;

		case 4: // Quadraphonic
			ChannelMap[0] = ESoundFileChannelMap::Type::LEFT;
			ChannelMap[1] = ESoundFileChannelMap::Type::RIGHT;
			ChannelMap[2] = ESoundFileChannelMap::Type::BACK_LEFT;
			ChannelMap[3] = ESoundFileChannelMap::Type::BACK_RIGHT;
			break;

		case 5: // 5.0
			ChannelMap[0] = ESoundFileChannelMap::Type::LEFT;
			ChannelMap[1] = ESoundFileChannelMap::Type::RIGHT;
			ChannelMap[2] = ESoundFileChannelMap::Type::CENTER;
			ChannelMap[3] = ESoundFileChannelMap::Type::SIDE_LEFT;
			ChannelMap[4] = ESoundFileChannelMap::Type::SIDE_RIGHT;
			break;

		case 6: // 5.1
			ChannelMap[0] = ESoundFileChannelMap::Type::LEFT;
			ChannelMap[1] = ESoundFileChannelMap::Type::RIGHT;
			ChannelMap[2] = ESoundFileChannelMap::Type::CENTER;
			ChannelMap[3] = ESoundFileChannelMap::Type::LFE;
			ChannelMap[4] = ESoundFileChannelMap::Type::SIDE_LEFT;
			ChannelMap[5] = ESoundFileChannelMap::Type::SIDE_RIGHT;
			break;

		case 7: // 6.1
			ChannelMap[0] = ESoundFileChannelMap::Type::LEFT;
			ChannelMap[1] = ESoundFileChannelMap::Type::RIGHT;
			ChannelMap[2] = ESoundFileChannelMap::Type::CENTER;
			ChannelMap[3] = ESoundFileChannelMap::Type::LFE;
			ChannelMap[4] = ESoundFileChannelMap::Type::SIDE_LEFT;
			ChannelMap[5] = ESoundFileChannelMap::Type::SIDE_RIGHT;
			ChannelMap[6] = ESoundFileChannelMap::Type::BACK_CENTER;
			break;

		case 8: // 7.1
			ChannelMap[0] = ESoundFileChannelMap::Type::LEFT;
			ChannelMap[1] = ESoundFileChannelMap::Type::RIGHT;
			ChannelMap[2] = ESoundFileChannelMap::Type::CENTER;
			ChannelMap[3] = ESoundFileChannelMap::Type::LFE;
			ChannelMap[4] = ESoundFileChannelMap::Type::BACK_LEFT;
			ChannelMap[5] = ESoundFileChannelMap::Type::BACK_RIGHT;
			ChannelMap[6] = ESoundFileChannelMap::Type::SIDE_LEFT;
			ChannelMap[7] = ESoundFileChannelMap::Type::SIDE_RIGHT;
			break;

		default:
			break;
		}
	}

	static ESoundFileError::Type GetSoundDesriptionInternal(LibSoundFileHandle** OutFileHandle, const FString& FilePath, FSoundFileDescription& OutputDescription, TArray<ESoundFileChannelMap::Type>& OutChannelMap)
	{
		*OutFileHandle = nullptr;

		// Check to see if the file exists
		if (!FPaths::FileExists(FilePath))
		{
			UE_LOGF(LogAudioMixer, Error, "Sound file %ls doesn't exist.", *FilePath);
			return ESoundFileError::Type::FILE_DOESNT_EXIST;
		}

		// open a sound file handle to get the description
		if (SoundFileOpen != nullptr)
		{
			*OutFileHandle = SoundFileOpen(TCHAR_TO_ANSI(*FilePath), ESoundFileOpenMode::READING, &OutputDescription);
		}
		else
		{
			UE_LOGF(LogAudioMixer, Error, "LibSoundFile failed to load symbols for SoundFileOpen.");
			*OutFileHandle = nullptr;
		}

		if (!*OutFileHandle)
		{
			if (!SoundFileStrError)
			{
				return ESoundFileError::Type::INVALID_DATA;
			}

			FString StrError = FString(SoundFileStrError(nullptr));
			UE_LOGF(LogAudioMixer, Error, "Failed to open sound file %ls: %ls", *FilePath, *StrError);
			return ESoundFileError::Type::FAILED_TO_OPEN;
		}

		// Try to get a channel mapping
		int32 NumChannels = OutputDescription.NumChannels;
		OutChannelMap.Init(ESoundFileChannelMap::Type::INVALID, NumChannels);

		int32 Result = 0;
		if (SoundFileCommand)
		{
			Result = SoundFileCommand(*OutFileHandle, GET_CHANNEL_MAP_INFO, (int32*)OutChannelMap.GetData(), sizeof(int32)*NumChannels);
		}
		else
		{
			UE_LOGF(LogAudioMixer, Error, "LibSoundFile wasn't properly loaded with symbols for SoundFileCommand.");
		}

		// If we failed to get the file's channel map definition, then we set the default based on the number of channels
		if (Result == 0)
		{
			GetDefaultMappingsForChannelNumber(NumChannels, OutChannelMap);
		}
		else
		{
			// Check to see if the channel map we did get back is filled with INVALID channels
			bool bIsInvalid = false;
			for (ESoundFileChannelMap::Type ChannelType : OutChannelMap)
			{
				if (ChannelType == ESoundFileChannelMap::Type::INVALID)
				{
					bIsInvalid = true;
					break;
				}
			}
			// If invalid, then we need to get the default channel mapping
			if (bIsInvalid)
			{
				GetDefaultMappingsForChannelNumber(NumChannels, OutChannelMap);
			}
		}

		return ESoundFileError::Type::NONE;
	}

	static ESoundFileError::Type GetOptionalChunksInternal(LibSoundFileHandle* FileHandle, FSoundFileChunkArray& OutChunkInfoArray, const TSet<uint32>& ChunkIdsToSkip = {})
	{
		// Verify that the necessary library function pointers have been properly set
		if (SoundFileGetChunkIterator != nullptr && SoundFileGetChunkSize != nullptr &&
			SoundFileGetChunkData != nullptr && SoundFileNextChunkIterator != nullptr)
		{
			const TArray<uint32>& OptionalChunkIds = FWaveModInfo::GetOptionalWaveChunkIds();

			for (const uint32 Id : OptionalChunkIds)
			{
				if (ChunkIdsToSkip.Contains(Id))
				{
					continue;
				}
				FSoundFileChunkInfo ChunkLookup;				
				// Copy chunk ID over. DWORD (4 bytes, each is ANSI char)
				*reinterpret_cast<uint32*>(ChunkLookup.ChunkId) = Id;
				ChunkLookup.ChunkId[4] = 0;		// Null terminate the string just in case.
				
				ChunkLookup.ChunkIdSize = 5;	// 4 bytes, + null.
				

				// Lookup chunk of given Id. Multiple chunks can exist of a given type
				// so we loop here.
				LibSoundFileChunkIterator* ChunkItr = SoundFileGetChunkIterator(FileHandle, &ChunkLookup);
				while (ChunkItr)
				{
					FSoundFileChunkInfoWrapper ChunkInfo;
					// SoundFileGetChunkSize retrieves the chunk data size. Oddly,
					// it does not fill in the chunk Id.
					int32 Result = SoundFileGetChunkSize(ChunkItr, ChunkInfo.GetPtr());
					if (Result == 0 && ChunkInfo.GetPtr()->DataLength > 0)
					{
						ChunkInfo.AllocateChunkData();
						// SoundFileGetChunkData copies in the chunk data and fills
						// in the ChunkId.
						Result = SoundFileGetChunkData(ChunkItr, ChunkInfo.GetPtr());

						if (Result == 0)
						{
							OutChunkInfoArray.Add(MoveTemp(ChunkInfo));
						}
						else
						{
							UE_LOGF(LogAudioMixer, Error, "LibSoundFile unable to read invalid chunk: %ls", *FString((ANSICHAR*)ChunkLookup.ChunkId));
							return ESoundFileError::Type::INVALID_CHUNK;
						}
					}

					ChunkItr = SoundFileNextChunkIterator(ChunkItr);
				}
			}
		}
		else
		{
			UE_LOGF(LogAudioMixer, Error, "LibSoundFile wasn't properly loaded with symbols for accessing wav chunk data.");
		}

		return ESoundFileError::Type::NONE;
	}

	static ESoundFileError::Type WriteOptionalChunksInternal(LibSoundFileHandle* FileHandle, const FSoundFileChunkArray& ChunkInfoArray)
	{
		// Verify that the necessary library function pointers have been properly set
		if (SoundFileSetChunk != nullptr && SoundFileCommand != nullptr)
		{
			for (const FSoundFileChunkInfoWrapper& ChunkInfo : ChunkInfoArray)
			{
				// Note, libsndfile uses 4-byte pad when writing chunk data
				int32 Result = SoundFileSetChunk(FileHandle, ChunkInfo.GetPtr());
				if (Result)
				{
					UE_LOGF(LogAudioMixer, Error, "LibSoundFile failed to write chunk data; Result = %d", Result);
					return ESoundFileError::Type::INVALID_STATE;
				}
				else
				{
					UE_LOGF(LogAudioMixer, VeryVerbose, "Wrote ChunkId: %ls, chunk DataLength: %d", *FString((ANSICHAR*)ChunkInfo.GetPtr()->ChunkId), ChunkInfo.GetPtr()->DataLength);

					// Update file header after adding a new chunk - UPDATE_HEADER_NOW command always returns 0
					SoundFileCommand(FileHandle, UPDATE_HEADER_NOW, nullptr, 0);
				}
			}
		}
		else
		{
			UE_LOGF(LogAudioMixer, Error, "LibSoundFile wasn't properly loaded with symbols for accessing wav chunk data.");
		}

		return ESoundFileError::Type::NONE;
	}

	static ESoundFileError::Type WriteByteArrayChunkInternal(LibSoundFileHandle* FileHandle, TArray<uint8>& InSoundFileChunk)
	{
		// Verify that the necessary library function pointers have been properly set
		if (SoundFileCommand != nullptr && SoundFileSetChunk != nullptr)
		{
			check(InSoundFileChunk.Num() > 8)
			uint8* SoundFileChunk = InSoundFileChunk.GetData();
			check(SoundFileChunk);

			FSoundFileChunkArray ChunkInfoArray;
			FSoundFileChunkInfo ChunkInfo;

			ChunkInfo.ChunkId[0] = InSoundFileChunk[0];
			ChunkInfo.ChunkId[1] = InSoundFileChunk[1];
			ChunkInfo.ChunkId[2] = InSoundFileChunk[2];
			ChunkInfo.ChunkId[3] = InSoundFileChunk[3];
			ChunkInfo.ChunkIdSize = sizeof(uint32);

			ChunkInfo.DataLength = 
				((uint32)InSoundFileChunk[4] << 0) |
				((uint32)InSoundFileChunk[5] << 8) |
				((uint32)InSoundFileChunk[6] << 16)|
				((uint32)InSoundFileChunk[7] << 24);

			ChunkInfo.DataPtr = SoundFileChunk + 8;
			
			// Note, libsndfile uses 4-byte pad when writing chunk data
			int32 Result = SoundFileSetChunk(FileHandle, &ChunkInfo);
			if (Result)
			{
				UE_LOGF(LogAudioMixer, Error, "LibSoundFile failed to write chunk data; Result = %d", Result);

				return ESoundFileError::Type::INVALID_STATE;
			}
			else
			{
				int32 DataLength = ChunkInfo.DataLength;
				UE_LOGF(LogAudioMixer, VeryVerbose, "Wrote ChunkId: %ls, chunk DataLength: %d", *FString((ANSICHAR*)&ChunkInfo.ChunkId), DataLength);

				// Update file header after adding a new chunk - UPDATE_HEADER_NOW command always returns 0
				SoundFileCommand(FileHandle, UPDATE_HEADER_NOW, nullptr, 0);
			}
		}
		else
		{
			UE_LOGF(LogAudioMixer, Error, "LibSoundFile wasn't properly loaded with symbols for accessing wav data.");

			return ESoundFileError::Type::INVALID_STATE;
		}

		return ESoundFileError::Type::NONE;
	}

	// Only works for SoundFile Commands that return a bool (ex. SET_CUE)
	static ESoundFileError::Type WriteCommandDataInternal(LibSoundFileHandle* FileHandle, const int32 Command, void* InSoundFileData, size_t InSoundFileDataLength)
	{
		// Verify that the necessary library function pointers have been properly set
		if (SoundFileCommand != nullptr)
		{
			check(InSoundFileData);

			if (SoundFileCommand(FileHandle, Command, InSoundFileData, sizeof(FSoundFileCues)))
			{
				int32 DataLength = 0;
				UE_LOGF(LogAudioMixer, VeryVerbose, "SoundFileCommand Completed... Updating Header Now");

				// Update file header after adding a new chunk - UPDATE_HEADER_NOW command always returns 0
				SoundFileCommand(FileHandle, UPDATE_HEADER_NOW, nullptr, 0);
			}
			else
			{
				UE_LOGF(LogAudioMixer, Error, "LibSoundFile failed to write command data; Result = %s", SoundFileStrError(FileHandle));

				return ESoundFileError::Type::INVALID_STATE;
			}
		}
		else
		{
			UE_LOGF(LogAudioMixer, Error, "LibSoundFile wasn't properly loaded with symbols for accessing wav data.");

			return ESoundFileError::Type::INVALID_STATE;
		}

		return ESoundFileError::Type::NONE;
	}
	
	static SoundFileCount OnSoundFileGetLengthBytes(void* UserData)
	{
		SoundFileCount Length = 0;
		((ISoundFileParser*)UserData)->GetLengthBytes(Length);
		return Length;
	}

	static SoundFileCount OnSoundFileSeekBytes(SoundFileCount Offset, int32 Mode, void* UserData)
	{
		SoundFileCount OutOffset = 0;
		((ISoundFileParser*)UserData)->SeekBytes(Offset, (ESoundFileSeekMode::Type)Mode, OutOffset);
		return OutOffset;
	}

	static SoundFileCount OnSoundFileReadBytes(void* DataPtr, SoundFileCount ByteCount, void* UserData)
	{
		SoundFileCount OutBytesRead = 0;
		((ISoundFileParser*)UserData)->ReadBytes(DataPtr, ByteCount, OutBytesRead);
		return OutBytesRead;
	}

	static SoundFileCount OnSoundFileWriteBytes(const void* DataPtr, SoundFileCount ByteCount, void* UserData)
	{
		SoundFileCount OutBytesWritten = 0;
		((ISoundFileParser*)UserData)->WriteBytes(DataPtr, ByteCount, OutBytesWritten);
		return OutBytesWritten;
	}

	static SoundFileCount OnSoundFileTell(void* UserData)
	{
		SoundFileCount OutOffset = 0;
		((ISoundFileParser*)UserData)->GetOffsetBytes(OutOffset);
		return OutOffset;
	}

	/************************************************************************/
	/* FSoundFileReader														*/
	/************************************************************************/
	class FSoundFileReader final : public ISoundFileParser, public ISoundFileReader
	{
	public:
		FSoundFileReader()
			: CurrentIndexBytes(0)
			, FileHandle(nullptr)
			, State(ESoundFileState::UNINITIALIZED)
			, CurrentError(static_cast<int32>(ESoundFileError::Type::NONE))
		{
		}

		~FSoundFileReader()
		{
			Release();
			check(FileHandle == nullptr);
		}

		ESoundFileError::Type GetLengthBytes(SoundFileCount& OutLength) const override
		{
			if (!SoundFileData.IsValid())
			{
				return ESoundFileError::Type::INVALID_DATA;
			}

			int32 DataSize;
			ESoundFileError::Type Error = SoundFileData->GetDataSize(DataSize);
			if (Error == ESoundFileError::Type::NONE)
			{
				OutLength = DataSize;
				return ESoundFileError::Type::NONE;
			}
			return Error;
		}

		ESoundFileError::Type SeekBytes(SoundFileCount Offset, ESoundFileSeekMode::Type SeekMode, SoundFileCount& OutOffset) override
		{
			if (!SoundFileData.IsValid())
			{
				return ESoundFileError::Type::INVALID_DATA;
			}

			int32 DataSize;
			ESoundFileError::Type Error = SoundFileData->GetDataSize(DataSize);
			if (Error != ESoundFileError::Type::NONE)
			{
				return Error;
			}

			SoundFileCount MaxBytes = DataSize;
			if (MaxBytes == 0)
			{
				OutOffset = 0;
				CurrentIndexBytes = 0;
				return ESoundFileError::Type::NONE;
			}

			switch (SeekMode)
			{
			case ESoundFileSeekMode::FROM_START:
				CurrentIndexBytes = Offset;
				break;

			case ESoundFileSeekMode::FROM_CURRENT:
				CurrentIndexBytes += Offset;
				break;

			case ESoundFileSeekMode::FROM_END:
				CurrentIndexBytes = MaxBytes + Offset;
				break;

			default:
				checkf(false, TEXT("Uknown seek mode!"));
				break;
			}

			// Wrap the byte index to fall between 0 and MaxBytes
			while (CurrentIndexBytes < 0)
			{
				CurrentIndexBytes += MaxBytes;
			}

			while (CurrentIndexBytes > MaxBytes)
			{
				CurrentIndexBytes -= MaxBytes;
			}

			OutOffset = CurrentIndexBytes;
			return ESoundFileError::Type::NONE;
		}

		ESoundFileError::Type ReadBytes(void* DataPtr, SoundFileCount NumBytes, SoundFileCount& OutNumBytesRead) override
		{
			if (!SoundFileData.IsValid())
			{
				return ESoundFileError::Type::INVALID_DATA;
			}

			SoundFileCount EndByte = CurrentIndexBytes + NumBytes;

			int32 DataSize;
			ESoundFileError::Type Error = SoundFileData->GetDataSize(DataSize);
			if (Error != ESoundFileError::Type::NONE)
			{
				return Error;
			}
			SoundFileCount MaxBytes = DataSize;
			if (EndByte >= MaxBytes)
			{
				NumBytes = MaxBytes - CurrentIndexBytes;
			}

			if (NumBytes > 0)
			{
				TArray<uint8>* BulkData = nullptr;
				Error = SoundFileData->GetBulkData(&BulkData);
				if (Error != ESoundFileError::Type::NONE)
				{
					return Error;
				}

				check(BulkData != nullptr);

				FMemory::Memcpy(DataPtr, (const void*)&(*BulkData)[CurrentIndexBytes], NumBytes);
				CurrentIndexBytes += NumBytes;
			}
			OutNumBytesRead = NumBytes;
			return ESoundFileError::Type::NONE;
		}

		ESoundFileError::Type WriteBytes(const void* DataPtr, SoundFileCount NumBytes, SoundFileCount& OutNumBytesWritten) override
		{
			// This should never get called in the reader class
			check(false);
			return ESoundFileError::Type::NONE;
		}

		ESoundFileError::Type GetOffsetBytes(SoundFileCount& OutOffset) const override
		{
			OutOffset = CurrentIndexBytes;
			return ESoundFileError::Type::NONE;
		}

		ESoundFileError::Type Init(TSharedPtr<ISoundFile> InSoundFileData, bool bIsStreamed) override
		{
			if (bIsStreamed)
			{
				return InitStreamed(InSoundFileData);
			}
			else
			{
				return InitLoaded(InSoundFileData);
			}
		}

		ESoundFileError::Type Init(const TArray<uint8>* InData)
		{
			return ESoundFileError::Type::NONE;
		}

		ESoundFileError::Type Release() override
		{
			if (!SoundFileClose)
			{
				UE_LOGF(LogAudioMixer, Error, "LibSoundFile failed to load symbols for SoundFileClose");
				FileHandle = nullptr;
				return ESoundFileError::Type::INVALID_STATE;
			}

			if (FileHandle)
			{
				SoundFileClose(FileHandle);
				FileHandle = nullptr;
			}
			return ESoundFileError::Type::NONE;
		}

		ESoundFileError::Type GetDescription(FSoundFileDescription& OutputDescription, TArray<ESoundFileChannelMap::Type>& OutChannelMap)
		{
			SoundFileData->GetDescription(OutputDescription);
			SoundFileData->GetChannelMap(OutChannelMap);

			return ESoundFileError::Type::NONE;
		}

		ESoundFileError::Type SeekFrames(SoundFileCount Offset, ESoundFileSeekMode::Type SeekMode, SoundFileCount& OutOffset) override
		{
			if (!SoundFileSeek)
			{
				UE_LOGF(LogAudioMixer, Error, "LibSoundFile failed to load symbols for SoundFileSeek");
				return ESoundFileError::Type::INVALID_STATE;
			}

			SoundFileCount Pos = SoundFileSeek(FileHandle, Offset, (int32)SeekMode);
			if (Pos == -1)
			{
				if (!SoundFileStrError)
				{
					return SetError(ESoundFileError::Type::INVALID_STATE);
				}

				FString StrErr = SoundFileStrError(FileHandle);
				UE_LOGF(LogAudioMixer, Error, "Failed to seek file: %ls", *StrErr);
				return SetError(ESoundFileError::Type::FAILED_TO_SEEK);
			}
			return ESoundFileError::Type::NONE;
		}

		ESoundFileError::Type ReadFrames(float* DataPtr, SoundFileCount NumFrames, SoundFileCount& OutNumFramesRead) override
		{
			if (!SoundFileReadFramesFloat)
			{
				UE_LOGF(LogAudioMixer, Error, "LibSoundFile failed to load symbols for SoundFileReadFramesFloat");
				return ESoundFileError::Type::INVALID_STATE;
			}

			OutNumFramesRead = SoundFileReadFramesFloat(FileHandle, DataPtr, NumFrames);
			return ESoundFileError::Type::NONE;
		}

		ESoundFileError::Type ReadFrames(double* DataPtr, SoundFileCount NumFrames, SoundFileCount& OutNumFramesRead) override
		{
			if (!SoundFileReadFramesDouble)
			{
				UE_LOGF(LogAudioMixer, Error, "LibSoundFile failed to load symbols for SoundFileReadFramesDouble");
				return ESoundFileError::Type::INVALID_STATE;
			}

			OutNumFramesRead = SoundFileReadFramesDouble(FileHandle, DataPtr, NumFrames);
			return ESoundFileError::Type::NONE;
		}

		ESoundFileError::Type ReadSamples(float* DataPtr, SoundFileCount NumSamples, SoundFileCount& OutNumSamplesRead) override
		{
			if (!SoundFileReadSamplesFloat)
			{
				UE_LOGF(LogAudioMixer, Error, "LibSoundFile failed to load symbols for SoundFileReadSamplesFloat");
				return ESoundFileError::Type::INVALID_STATE;
			}

			OutNumSamplesRead = SoundFileReadSamplesFloat(FileHandle, DataPtr, NumSamples);
			return ESoundFileError::Type::NONE;
		}

		ESoundFileError::Type ReadSamples(double* DataPtr, SoundFileCount NumSamples, SoundFileCount& OutNumSamplesRead) override
		{
			if (!SoundFileReadSamplesDouble)
			{
				UE_LOGF(LogAudioMixer, Error, "LibSoundFile failed to load symbols for SoundFileReadSamplesDouble");
				return ESoundFileError::Type::INVALID_STATE;
			}

			OutNumSamplesRead = SoundFileReadSamplesDouble(FileHandle, DataPtr, NumSamples);
			return ESoundFileError::Type::NONE;
		}

		ESoundFileError::Type GetOptionalChunks(FSoundFileChunkArray& OutChunkInfoArray, const TSet<uint32>& ChunkIdsToSkip = {}) override
		{
			return GetOptionalChunksInternal(FileHandle, OutChunkInfoArray, ChunkIdsToSkip);
		}

	private:

		ESoundFileError::Type InitLoaded(TSharedPtr<ISoundFile> InSoundFileData)
		{
			if (!(State.GetValue() == ESoundFileState::UNINITIALIZED || State.GetValue() == ESoundFileState::LOADING))
			{
				return SetError(ESoundFileError::Type::ALREADY_INITIALIZED);
			}

			check(InSoundFileData.IsValid());
			check(FileHandle == nullptr);

			// Setting sound file data initializes this sound file
			SoundFileData = InSoundFileData;
			check(SoundFileData.IsValid());

			bool bIsStreamed;
			ESoundFileError::Type Error = SoundFileData->IsStreamed(bIsStreamed);
			if (Error != ESoundFileError::Type::NONE)
			{
				return Error;
			}

			if (bIsStreamed)
			{
				return ESoundFileError::Type::INVALID_DATA;
			}

			ESoundFileState::Type SoundFileState;
			Error = SoundFileData->GetState(SoundFileState);
			if (Error != ESoundFileError::Type::NONE)
			{
				return Error;
			}

			if (SoundFileState != ESoundFileState::LOADED)
			{
				return ESoundFileError::Type::INVALID_STATE;
			}

			// Open up a virtual file handle with this data
			FVirtualSoundFileCallbackInfo VirtualSoundFileInfo;
			VirtualSoundFileInfo.VirtualSoundFileGetLength = OnSoundFileGetLengthBytes;
			VirtualSoundFileInfo.VirtualSoundFileSeek = OnSoundFileSeekBytes;
			VirtualSoundFileInfo.VirtualSoundFileRead = OnSoundFileReadBytes;
			VirtualSoundFileInfo.VirtualSoundFileWrite = OnSoundFileWriteBytes;
			VirtualSoundFileInfo.VirtualSoundFileTell = OnSoundFileTell;

			FSoundFileDescription Description;
			SoundFileData->GetDescription(Description);

			if (!SoundFileFormatCheck)
			{
				UE_LOGF(LogAudioMixer, Error, "LibSoundFile failed to load symbols for SoundFileFormatCheck");
				return SetError(ESoundFileError::Type::INVALID_STATE);
			}

			if (!SoundFileFormatCheck(&Description))
			{
				return SetError(ESoundFileError::Type::INVALID_INPUT_FORMAT);
			}

			if (!SoundFileOpenVirtual)
			{
				UE_LOGF(LogAudioMixer, Error, "LibSoundFile failed to load symbols for SoundFileOpenVirtual");
				FileHandle = nullptr;
			}
			else
			{
				FileHandle = SoundFileOpenVirtual(&VirtualSoundFileInfo, ESoundFileOpenMode::READING, &Description, (void*)this);
			}
			
			if (!FileHandle)
			{
				if (!SoundFileStrError)
				{
					return SetError(ESoundFileError::Type::INVALID_DATA);
				}

				FString StrErr = SoundFileStrError(nullptr);
				UE_LOGF(LogAudioMixer, Error, "Failed to intitialize sound file: %ls", *StrErr);
				return SetError(ESoundFileError::Type::FAILED_TO_OPEN);
			}

			State.Set(ESoundFileState::INITIALIZED);
			return ESoundFileError::Type::NONE;
		}

		ESoundFileError::Type InitStreamed(TSharedPtr<ISoundFile> InSoundFileData)
		{
			if (!(State.GetValue() == ESoundFileState::UNINITIALIZED || State.GetValue() == ESoundFileState::LOADING))
			{
				return SetError(ESoundFileError::Type::ALREADY_INITIALIZED);
			}

			check(InSoundFileData.IsValid());
			check(FileHandle == nullptr);

			// Setting sound file data initializes this sound file
			SoundFileData = InSoundFileData;
			check(SoundFileData.IsValid());

			bool bIsStreamed;
			ESoundFileError::Type Error = SoundFileData->IsStreamed(bIsStreamed);
			if (Error != ESoundFileError::Type::NONE)
			{
				return Error;
			}

			if (!bIsStreamed)
			{
				return ESoundFileError::Type::INVALID_DATA;
			}

			ESoundFileState::Type SoundFileState;
			Error = SoundFileData->GetState(SoundFileState);
			if (Error != ESoundFileError::Type::NONE)
			{
				return Error;
			}

			if (SoundFileState != ESoundFileState::STREAMING)
			{
				return ESoundFileError::Type::INVALID_STATE;
			}

			FName NamePath;
			Error = SoundFileData->GetPath(NamePath);
			if (Error != ESoundFileError::Type::NONE)
			{
				return Error;
			}

			FString FilePath = NamePath.GetPlainNameString();

			FSoundFileDescription Description;
			TArray<ESoundFileChannelMap::Type> ChannelMap;
			Error = GetSoundDesriptionInternal(&FileHandle, FilePath, Description, ChannelMap);
			if (Error == ESoundFileError::Type::NONE)
			{
				// Tell this reader that we're in streaming mode.
				State.Set(ESoundFileState::STREAMING);
				return ESoundFileError::Type::NONE;
			}
			else
			{
				return SetError(Error);
			}
		}

		ESoundFileError::Type SetError(ESoundFileError::Type InError)
		{
			if (InError != ESoundFileError::Type::NONE)
			{
				State.Set(ESoundFileState::HAS_ERROR);
			}
			CurrentError.Set(static_cast<int32>(InError));
			return InError;
		}

		TSharedPtr<ISoundFile>	SoundFileData;
		SoundFileCount			CurrentIndexBytes;
		LibSoundFileHandle*		FileHandle;
		FThreadSafeCounter		State;
		FThreadSafeCounter		CurrentError;
	};


	/************************************************************************/
	/* FSoundDataReader														*/
	/************************************************************************/
	class FSoundDataReader : public ISoundFileParser, public ISoundFileReader
	{
	public:
		FSoundDataReader()
			: CurrentIndexBytes(0)
			, State(ESoundFileState::UNINITIALIZED)
			, CurrentError(static_cast<int32>(ESoundFileError::Type::NONE))
			, ChannelMap()
		{
		}

		~FSoundDataReader()
		{
			Release();
		}

		ESoundFileError::Type GetLengthBytes(SoundFileCount& OutLength) const override
		{
			if (SoundData == nullptr)
			{
				return ESoundFileError::Type::INVALID_DATA;
			}

			OutLength = SoundData->GetAllocatedSize();

			return ESoundFileError::Type::NONE;
		}

		ESoundFileError::Type SeekBytes(SoundFileCount Offset, ESoundFileSeekMode::Type SeekMode, SoundFileCount& OutOffset) override
		{
			if (SoundData == nullptr)
			{
				return ESoundFileError::Type::INVALID_DATA;
			}

			int32 DataSize = SoundData->GetAllocatedSize();

			SoundFileCount MaxBytes = DataSize;
			if (MaxBytes == 0)
			{
				OutOffset = 0;
				CurrentIndexBytes = 0;
				return ESoundFileError::Type::NONE;
			}

			switch (SeekMode)
			{
			case ESoundFileSeekMode::FROM_START:
				CurrentIndexBytes = Offset;
				break;

			case ESoundFileSeekMode::FROM_CURRENT:
				CurrentIndexBytes += Offset;
				break;

			case ESoundFileSeekMode::FROM_END:
				CurrentIndexBytes = MaxBytes + Offset;
				break;

			default:
				checkf(false, TEXT("Uknown seek mode!"));
				break;
			}

			// Wrap the byte index to fall between 0 and MaxBytes
			while (CurrentIndexBytes < 0)
			{
				CurrentIndexBytes += MaxBytes;
			}

			while (CurrentIndexBytes > MaxBytes)
			{
				CurrentIndexBytes -= MaxBytes;
			}

			OutOffset = CurrentIndexBytes;
			return ESoundFileError::Type::NONE;
		}

		ESoundFileError::Type ReadBytes(void* DataPtr, SoundFileCount NumBytes, SoundFileCount& OutNumBytesRead) override
		{
			if (SoundData == nullptr)
			{
				return ESoundFileError::Type::INVALID_DATA;
			}

			SoundFileCount EndByte = CurrentIndexBytes + NumBytes;

			int32 DataSize = SoundData->GetAllocatedSize();
			
			SoundFileCount MaxBytes = DataSize;
			if (EndByte >= MaxBytes)
			{
				NumBytes = MaxBytes - CurrentIndexBytes;
			}

			if (NumBytes > 0)
			{
				FMemory::Memcpy(DataPtr, (const void*)&(*SoundData)[CurrentIndexBytes], NumBytes);
				CurrentIndexBytes += NumBytes;
			}
			OutNumBytesRead = NumBytes;
			return ESoundFileError::Type::NONE;
		}

		ESoundFileError::Type WriteBytes(const void* DataPtr, SoundFileCount NumBytes, SoundFileCount& OutNumBytesWritten) override
		{
			// This should never get called in the reader class
			check(false);
			return ESoundFileError::Type::NONE;
		}

		ESoundFileError::Type GetOffsetBytes(SoundFileCount& OutOffset) const override
		{
			OutOffset = CurrentIndexBytes;
			return ESoundFileError::Type::NONE;
		}

		ESoundFileError::Type Init(TSharedPtr<ISoundFile> InSoundFileData, bool bIsStreamed) override
		{
			return ESoundFileError::Type::NONE;
		}

		ESoundFileError::Type Init(const TArray<uint8>* InData)
		{
			SoundData = InData;

			// Open up a virtual file handle with this data
			FVirtualSoundFileCallbackInfo VirtualSoundFileInfo;
			VirtualSoundFileInfo.VirtualSoundFileGetLength = OnSoundFileGetLengthBytes;
			VirtualSoundFileInfo.VirtualSoundFileSeek = OnSoundFileSeekBytes;
			VirtualSoundFileInfo.VirtualSoundFileRead = OnSoundFileReadBytes;
			VirtualSoundFileInfo.VirtualSoundFileWrite = OnSoundFileWriteBytes;
			VirtualSoundFileInfo.VirtualSoundFileTell = OnSoundFileTell;

			if (SoundFileOpenVirtual)
			{
				FileHandle = SoundFileOpenVirtual(&VirtualSoundFileInfo, ESoundFileOpenMode::READING, &Description, (void*)this);
			}
			else
			{
				UE_LOGF(LogAudioMixer, Error, "LibSoundFile failed to load symbols for SoundFileOpenVirtual.");
				FileHandle = nullptr;
			}

			if (!FileHandle)
			{
				if (!SoundFileStrError)
				{
					return SetError(ESoundFileError::Type::INVALID_DATA);
				}

				FString StrErr = SoundFileStrError(nullptr);
				UE_LOGF(LogAudioMixer, Error, "Failed to initialize sound file: %ls", *StrErr);
				return SetError(ESoundFileError::Type::FAILED_TO_OPEN);
			}

			// Try to get a channel mapping
			int32 NumChannels = Description.NumChannels;
			ChannelMap.Init(ESoundFileChannelMap::Type::INVALID, NumChannels);
			
			int32 Result = 0;
			if (SoundFileCommand)
			{
				Result = SoundFileCommand(FileHandle, GET_CHANNEL_MAP_INFO, (int32*)ChannelMap.GetData(), sizeof(int32)*NumChannels);
			}
			else
			{
				UE_LOGF(LogAudioMixer, Error, "LibSoundFile failed to load symbols for SoundFileCommand.");
			}

			// If we failed to get the file's channel map definition, then we set the default based on the number of channels
			if (Result == 0)
			{
				GetDefaultMappingsForChannelNumber(NumChannels, ChannelMap);
			}
			else
			{
				// Check to see if the channel map we did get back is filled with INVALID channels
				bool bIsInvalid = false;
				for (ESoundFileChannelMap::Type ChannelType : ChannelMap)
				{
					if (ChannelType == ESoundFileChannelMap::Type::INVALID)
					{
						bIsInvalid = true;
						break;
					}
				}
				// If invalid, then we need to get the default channel mapping
				if (bIsInvalid)
				{
					GetDefaultMappingsForChannelNumber(NumChannels, ChannelMap);
				}
			}

			State.Set(ESoundFileState::INITIALIZED);

			return ESoundFileError::Type::NONE;
		}

		ESoundFileError::Type Release() override final
		{
			SoundData = nullptr;

			return ESoundFileError::Type::NONE;
		}

		ESoundFileError::Type GetDescription(FSoundFileDescription& OutputDescription, TArray<ESoundFileChannelMap::Type>& OutChannelMap)
		{
			OutputDescription = Description;
			OutChannelMap = ChannelMap;

			return ESoundFileError::Type::NONE;
		}


		ESoundFileError::Type SeekFrames(SoundFileCount Offset, ESoundFileSeekMode::Type SeekMode, SoundFileCount& OutOffset) override
		{
			if (!SoundFileSeek)
			{
				UE_LOGF(LogAudioMixer, Error, "LibSoundFile failed to load.");
				return ESoundFileError::Type::INVALID_STATE;
			}

			SoundFileCount Pos = SoundFileSeek(FileHandle, Offset, (int32)SeekMode);
			if (Pos == -1)
			{
				if (!SoundFileStrError)
				{
					return SetError(ESoundFileError::Type::INVALID_DATA);
				}

				FString StrErr = SoundFileStrError(FileHandle);
				UE_LOGF(LogAudioMixer, Error, "Failed to seek file: %ls", *StrErr);
				return SetError(ESoundFileError::Type::FAILED_TO_SEEK);
			}
			return ESoundFileError::Type::NONE;
		}

		ESoundFileError::Type ReadFrames(float* DataPtr, SoundFileCount NumFrames, SoundFileCount& OutNumFramesRead) override
		{
			if (!SoundFileReadFramesFloat)
			{
				UE_LOGF(LogAudioMixer, Error, "LibSoundFile failed to load symbols for SoundFileReadFramesFloat");
				return ESoundFileError::Type::INVALID_STATE;
			}

			OutNumFramesRead = SoundFileReadFramesFloat(FileHandle, DataPtr, NumFrames);
			return ESoundFileError::Type::NONE;
		}

		ESoundFileError::Type ReadFrames(double* DataPtr, SoundFileCount NumFrames, SoundFileCount& OutNumFramesRead) override
		{
			if (!SoundFileReadFramesDouble)
			{
				UE_LOGF(LogAudioMixer, Error, "LibSoundFile failed to load symbols for SoundFileReadFramesDouble");
				return ESoundFileError::Type::INVALID_STATE;
			}

			OutNumFramesRead = SoundFileReadFramesDouble(FileHandle, DataPtr, NumFrames);
			return ESoundFileError::Type::NONE;
		}

		ESoundFileError::Type ReadSamples(float* DataPtr, SoundFileCount NumSamples, SoundFileCount& OutNumSamplesRead) override
		{
			if (!SoundFileReadSamplesFloat)
			{
				UE_LOGF(LogAudioMixer, Error, "LibSoundFile failed to load symbols for SoundFileReadSamplesFloat");
				return ESoundFileError::Type::INVALID_STATE;
			}

			OutNumSamplesRead = SoundFileReadSamplesFloat(FileHandle, DataPtr, NumSamples);
			return ESoundFileError::Type::NONE;
		}

		ESoundFileError::Type ReadSamples(double* DataPtr, SoundFileCount NumSamples, SoundFileCount& OutNumSamplesRead) override
		{
			if (!SoundFileReadSamplesDouble)
			{
				UE_LOGF(LogAudioMixer, Error, "LibSoundFile failed to load symbols for SoundFileReadSamplesDouble");
				return ESoundFileError::Type::INVALID_STATE;
			}

			OutNumSamplesRead = SoundFileReadSamplesDouble(FileHandle, DataPtr, NumSamples);
			return ESoundFileError::Type::NONE;
		}

		ESoundFileError::Type GetOptionalChunks(FSoundFileChunkArray& OutChunkInfoArray, const TSet<uint32>& ChunkIdsToSkip = {}) override
		{
			return GetOptionalChunksInternal(FileHandle, OutChunkInfoArray, ChunkIdsToSkip);
		}

	private:

		ESoundFileError::Type SetError(ESoundFileError::Type InError)
		{
			if (InError != ESoundFileError::Type::NONE)
			{
				State.Set(ESoundFileState::HAS_ERROR);
			}
			CurrentError.Set(static_cast<int32>(InError));
			return InError;
		}

		const TArray<uint8>*	SoundData;
		SoundFileCount			CurrentIndexBytes;
		FThreadSafeCounter		State;
		FThreadSafeCounter		CurrentError;
		FSoundFileDescription	Description;
		TArray<ESoundFileChannelMap::Type> ChannelMap;
		LibSoundFileHandle*		FileHandle;
	};


	/************************************************************************/
	/* FSoundFileWriter														*/
	/************************************************************************/
	class FSoundFileWriter : public ISoundFileParser, public ISoundFileWriter
	{
	public:

		FSoundFileWriter()
			: CurrentIndexBytes(0)
			, FileHandle(nullptr)
			, EncodingQuality(0.0)
			, State(ESoundFileState::UNINITIALIZED)
			, CurrentError(static_cast<int32>(ESoundFileError::Type::NONE))
		{

		}

		~FSoundFileWriter()
		{

		}

		ESoundFileError::Type GetLengthBytes(SoundFileCount& OutLength) const override
		{
			OutLength = BulkData.Num();
			return ESoundFileError::Type::NONE;
		}

		ESoundFileError::Type SeekBytes(SoundFileCount Offset, ESoundFileSeekMode::Type SeekMode, SoundFileCount& OutOffset) override
		{
			int32 DataSize = BulkData.Num();

			if (DataSize == 0)
			{
				OutOffset = 0;
				CurrentIndexBytes = 0;
				return ESoundFileError::Type::NONE;
			}

			switch (SeekMode)
			{
			case ESoundFileSeekMode::FROM_START:
				CurrentIndexBytes = Offset;
				break;

			case ESoundFileSeekMode::FROM_CURRENT:
				CurrentIndexBytes += Offset;
				break;

			case ESoundFileSeekMode::FROM_END:
				CurrentIndexBytes = DataSize + Offset;
				break;

			default:
				checkf(false, TEXT("Uknown seek mode!"));
				break;
			}

			OutOffset = CurrentIndexBytes;
			return ESoundFileError::Type::NONE;
		}

		ESoundFileError::Type ReadBytes(void* DataPtr, SoundFileCount NumBytes, SoundFileCount& OutNumBytesRead) override
		{
			// This shouldn't get called in the writer
			check(false);
			return ESoundFileError::Type::NONE;
		}

		ESoundFileError::Type WriteBytes(const void* DataPtr, SoundFileCount NumBytes, SoundFileCount& OutNumBytesWritten) override
		{
			const uint8* InDataBytes = (const uint8*)DataPtr;

			SoundFileCount BulkDataLength = BulkData.Num();

			// If we need more room, we add it here. 
			int64 NumExtraBytesNeeded = (CurrentIndexBytes + NumBytes) - BulkDataLength;

			if (BulkData.Num() + NumExtraBytesNeeded > INT32_MAX)
			{
				UE_LOGF(LogAudioMixer, Error, "Error encountered while writing bytes, file too large... skipping");

				return ESoundFileError::Type::INVALID_DATA;
			}

			if (NumExtraBytesNeeded > 0)
			{
				check(BulkData.Num() + NumExtraBytesNeeded <= INT32_MAX);
				BulkData.AddUninitialized(NumExtraBytesNeeded);
			}

			// Copy the input data into our current place in the BulkData.
			uint8* BulkDataPtr = BulkData.GetData();
			FMemory::Memcpy(&BulkDataPtr[CurrentIndexBytes], InDataBytes, NumBytes);

			// Seek our cursor forward accordingly.
			CurrentIndexBytes += NumBytes;
			OutNumBytesWritten = NumBytes;

			return ESoundFileError::Type::NONE;
		}

		ESoundFileError::Type GetOffsetBytes(SoundFileCount& OutOffset) const override
		{
			OutOffset = CurrentIndexBytes;
			return ESoundFileError::Type::NONE;
		}

		ESoundFileError::Type Init(const FSoundFileDescription& InDescription, const TArray<ESoundFileChannelMap::Type>& InChannelMap, double InEncodingQuality) override
		{
			State.Set(ESoundFileState::INITIALIZED);

			BulkData.Reset();
			Description = InDescription;
			ChannelMap = InChannelMap;
			EncodingQuality = InEncodingQuality;

			if (!SoundFileFormatCheck)
			{
				UE_LOGF(LogAudioMixer, Error, "LibSoundFile failed to load symbols for SoundFileFormatCheck");
				return SetError(ESoundFileError::Type::INVALID_STATE);
			}

			// First check the input format to make sure it's valid
			if (!SoundFileFormatCheck(&InDescription))
			{
				UE_LOGF(LogAudioMixer,
					Error,
					"Sound file input format (%ls - %ls) is invalid.",
					ESoundFileFormat::ToStringMajor(InDescription.FormatFlags),
					ESoundFileFormat::ToStringMinor(InDescription.FormatFlags));

				return SetError(ESoundFileError::Type::INVALID_INPUT_FORMAT);
			}

			// Make sure we have the right number of channels and our channel map size
			if (InChannelMap.Num() != InDescription.NumChannels)
			{
				UE_LOGF(LogAudioMixer, Error, "Channel map didn't match the input NumChannels");
				return SetError(ESoundFileError::Type::INVALID_CHANNEL_MAP);
			}

			FVirtualSoundFileCallbackInfo VirtualSoundFileInfo;
			VirtualSoundFileInfo.VirtualSoundFileGetLength = OnSoundFileGetLengthBytes;
			VirtualSoundFileInfo.VirtualSoundFileSeek = OnSoundFileSeekBytes;
			VirtualSoundFileInfo.VirtualSoundFileRead = OnSoundFileReadBytes;
			VirtualSoundFileInfo.VirtualSoundFileWrite = OnSoundFileWriteBytes;
			VirtualSoundFileInfo.VirtualSoundFileTell = OnSoundFileTell;

			if (SoundFileOpenVirtual)
			{
				FileHandle = SoundFileOpenVirtual(&VirtualSoundFileInfo, ESoundFileOpenMode::WRITING, &Description, (void*)this);
			}
			else
			{
				UE_LOGF(LogAudioMixer, Error, "LibSoundFile failed to load symbols for SoundFileOpenVirtual");
				FileHandle = nullptr;
			}

			if (!FileHandle)
			{
				if (!SoundFileStrError)
				{
					return SetError(ESoundFileError::Type::INVALID_DATA);
				}

				FString StrErr = FString(SoundFileStrError(nullptr));
				UE_LOGF(LogAudioMixer, Error, "Failed to open empty sound file: %ls", *StrErr);
				return SetError(ESoundFileError::Type::FAILED_TO_OPEN);
			}

			int32 Result = 0;
			if (SoundFileCommand)
			{
				Result = SoundFileCommand(FileHandle, SET_CHANNEL_MAP_INFO, (int32*)InChannelMap.GetData(), sizeof(int32)*Description.NumChannels);
			}
			else
			{
				UE_LOGF(LogAudioMixer, Error, "LibSoundFile failed to load symbols for SoundFileCommand");
			}

			if (Result != 1)
			{
				if (!SoundFileStrError)
				{
					return ESoundFileError::Type::INVALID_DATA;
				}

				// The result is returning 0 (false), however 'No Error'
				// is provided and the file mapping is correct.
				FString StrErr = SoundFileStrError(nullptr);
				if (StrErr != TEXT("No Error."))
				{
					UE_LOGF(LogAudioMixer, Error, "Failed to set the channel map on empty file for writing: %ls", *StrErr);
					return SetError(ESoundFileError::Type::INVALID_CHANNEL_MAP);
				}
			}

			if ((Description.FormatFlags & ESoundFileFormat::MAJOR_FORMAT_MASK) == ESoundFileFormat::OGG)
			{
				int32 Result2 = 0;
				if (SoundFileCommand)
				{
					Result2 = SoundFileCommand(FileHandle, SET_ENCODING_QUALITY, &EncodingQuality, sizeof(double));
				}
				else
				{
					UE_LOGF(LogAudioMixer, Error, "LibSoundFile failed to load symbols for SoundFileCommand");
				}

				if (Result2 != 1)
				{
					if (!SoundFileStrError)
					{
						UE_LOGF(LogAudioMixer, Error, "LibSoundFile failed to load symbols for SoundFileStrError");
						return ESoundFileError::Type::INVALID_DATA;
					}

					FString StrErr = SoundFileStrError(FileHandle);
					UE_LOGF(LogAudioMixer, Error, "Failed to set encoding quality: %ls", *StrErr);
					return SetError(ESoundFileError::Type::BAD_ENCODING_QUALITY);
				}
			}

			return ESoundFileError::Type::NONE;
		}

		ESoundFileError::Type Release() override
		{
			if (!SoundFileClose)
			{
				UE_LOGF(LogAudioMixer, Error, "LibSoundFile failed to load symbols for SoundFileClose");
				FileHandle = nullptr;
				return ESoundFileError::Type::INVALID_STATE;
			}

			if (FileHandle)
			{
				int32 Result = SoundFileClose(FileHandle);
				check(Result == 0);
				FileHandle = nullptr;
			}
			return ESoundFileError::Type::NONE;
		}

		ESoundFileError::Type SeekFrames(SoundFileCount Offset, ESoundFileSeekMode::Type SeekMode, SoundFileCount& OutOffset) override
		{
			if (!SoundFileSeek)
			{
				UE_LOGF(LogAudioMixer, Error, "LibSoundFile failed to load symbols for SoundFileSeek");
				return ESoundFileError::Type::INVALID_STATE;
			}

			SoundFileCount Pos = SoundFileSeek(FileHandle, Offset, (int32)SeekMode);
			if (Pos == -1)
			{
				if (!SoundFileStrError)
				{
					return ESoundFileError::Type::INVALID_DATA;
				}

				FString StrErr = SoundFileStrError(FileHandle);
				UE_LOGF(LogAudioMixer, Error, "Failed to seek file: %ls", *StrErr);
				return SetError(ESoundFileError::Type::FAILED_TO_SEEK);
			}
			return ESoundFileError::Type::NONE;
		}

		ESoundFileError::Type WriteFrames(const float* DataPtr, SoundFileCount NumFrames, SoundFileCount& OutNumFramesWritten) override
		{
			if (!SoundFileWriteFramesFloat)
			{
				UE_LOGF(LogAudioMixer, Error, "LibSoundFile failed to load symbols for SoundFileWriteFramesFloat");
				return ESoundFileError::Type::INVALID_STATE;
			}

			OutNumFramesWritten = SoundFileWriteFramesFloat(FileHandle, DataPtr, NumFrames);
			return ESoundFileError::Type::NONE;
		}

		ESoundFileError::Type WriteFrames(const double* DataPtr, SoundFileCount NumFrames, SoundFileCount& OutNumFramesWritten) override
		{
			if (!SoundFileWriteFramesDouble)
			{
				UE_LOGF(LogAudioMixer, Error, "LibSoundFile failed to load symbols for SoundFileWriteFramesDouble");
				return ESoundFileError::Type::INVALID_STATE;
			}

			OutNumFramesWritten = SoundFileWriteFramesDouble(FileHandle, DataPtr, NumFrames);
			return ESoundFileError::Type::NONE;
		}

		ESoundFileError::Type WriteSamples(const float* DataPtr, SoundFileCount NumSamples, SoundFileCount& OutNumSampleWritten) override
		{
			if (!SoundFileWriteSamplesFloat)
			{
				UE_LOGF(LogAudioMixer, Error, "LibSoundFile failed to load symbols for SoundFileWriteSamplesFloat");
				return ESoundFileError::Type::INVALID_STATE;
			}

			OutNumSampleWritten = SoundFileWriteSamplesFloat(FileHandle, DataPtr, NumSamples);
			return ESoundFileError::Type::NONE;
		}

		ESoundFileError::Type WriteSamples(const double* DataPtr, SoundFileCount NumSamples, SoundFileCount& OutNumSampleWritten) override
		{
			if (!SoundFileWriteSamplesDouble)
			{
				UE_LOGF(LogAudioMixer, Error, "LibSoundFile failed to load symbols for SoundFileWriteSamplesDouble");
				return ESoundFileError::Type::INVALID_STATE;
			}

			OutNumSampleWritten = SoundFileWriteSamplesDouble(FileHandle, DataPtr, NumSamples);
			return ESoundFileError::Type::NONE;
		}

		ESoundFileError::Type GetData(TArray<uint8>** OutBulkData) override
		{
			*OutBulkData = &BulkData;
			return ESoundFileError::Type::NONE;
		}

		ESoundFileError::Type WriteOptionalChunks(const FSoundFileChunkArray& ChunkInfoArray) override
		{
			return WriteOptionalChunksInternal(FileHandle, ChunkInfoArray);
		}

		ESoundFileError::Type WriteByteArrayChunk(TArray<uint8>& InSoundFileChunk) override
		{
			return WriteByteArrayChunkInternal(FileHandle, InSoundFileChunk);
		}

		ESoundFileError::Type WriteCueCommandData(FSoundFileCues& InSoundFileData) override
		{
			return WriteCommandDataInternal(FileHandle, SET_CUE, &InSoundFileData, sizeof(InSoundFileData));
		}

	private:

		ESoundFileError::Type SetError(ESoundFileError::Type InError)
		{
			if (InError != ESoundFileError::Type::NONE)
			{
				State.Set(ESoundFileState::HAS_ERROR);
			}
			CurrentError.Set(static_cast<int32>(InError));
			return InError;
		}

		SoundFileCount						CurrentIndexBytes;
		LibSoundFileHandle*					FileHandle;
		FSoundFileDescription				Description;
		TArray<ESoundFileChannelMap::Type>	ChannelMap;
		TArray<uint8>						BulkData;
		double								EncodingQuality;
		FThreadSafeCounter					State;
		FThreadSafeCounter					CurrentError;	
	};

	//////////////////////////////////////////////////////////////////////////

	bool SoundFileIOManagerInit()
	{
		return LoadSoundFileLib();
	}

	bool SoundFileIOManagerShutdown()
	{
		return ShutdownSoundFileLib();
	}

	//////////////////////////////////////////////////////////////////////////
	FSoundFileIOManagerImpl::FSoundFileIOManagerImpl()
	{

	}

	FSoundFileIOManagerImpl::~FSoundFileIOManagerImpl()
	{

	}

	TSharedPtr<ISoundFileReader> FSoundFileIOManagerImpl::CreateSoundFileReader()
	{
		return TSharedPtr<ISoundFileReader>(new FSoundFileReader());
	}

	TSharedPtr<ISoundFileReader> FSoundFileIOManagerImpl::CreateSoundDataReader()
	{
		return TSharedPtr<ISoundFileReader>(new FSoundDataReader());
	}

	TSharedPtr<ISoundFileWriter> FSoundFileIOManagerImpl::CreateSoundFileWriter()
	{
		return TSharedPtr<ISoundFileWriter>(new FSoundFileWriter());
	}

	bool FSoundFileIOManagerImpl::GetSoundFileDescription(const FString& FilePath, FSoundFileDescription& OutputDescription, TArray<ESoundFileChannelMap::Type>& OutChannelMap)
	{
		LibSoundFileHandle* FileHandle = nullptr;
		ESoundFileError::Type Error = GetSoundDesriptionInternal(&FileHandle, FilePath, OutputDescription, OutChannelMap);
		if (Error == ESoundFileError::Type::NONE)
		{
			check(FileHandle != nullptr);

			if (!SoundFileClose)
			{
				UE_LOGF(LogAudioMixer, Error, "LibSoundFile failed to load symbols for SoundFileClose");
				return false;
			}

			SoundFileClose(FileHandle);
			return true;
		}
		return false;
	}

	bool FSoundFileIOManagerImpl::GetSoundFileDescription(const FString& FilePath, FSoundFileDescription& OutputDescription)
	{
		TArray<ESoundFileChannelMap::Type> OutChannelMap;
		return GetSoundFileDescription(FilePath, OutputDescription, OutChannelMap);
	}

	bool FSoundFileIOManagerImpl::GetFileExtensionForFormatFlags(int32 FormatFlags, FString& OutExtension)
	{
		if (FormatFlags & ESoundFileFormat::OGG)
		{
			OutExtension = TEXT("ogg");
		}
		else if (FormatFlags & ESoundFileFormat::WAV)
		{
			OutExtension = TEXT("wav");
		}
		else if (FormatFlags & ESoundFileFormat::AIFF)
		{
			OutExtension = TEXT("aiff");
		}
		else if (FormatFlags & ESoundFileFormat::FLAC)
		{
			OutExtension = TEXT("flac");
		}
		else
		{
			return false;
		}

		return true;
	}

	ESoundFileError::Type FSoundFileIOManagerImpl::GetSoundFileInfoFromPath(const FString& FilePath, FSoundFileDescription& Description, TArray<ESoundFileChannelMap::Type>& ChannelMap)
	{
		// Load the description and channel map info
		LibSoundFileHandle* FileHandle = nullptr;
		ESoundFileError::Type Error = GetSoundDesriptionInternal(&FileHandle, FilePath, Description, ChannelMap);
		if (FileHandle)
		{
			if (!SoundFileClose)
			{
				UE_LOGF(LogAudioMixer, Error, "LibSoundFile failed to load symbols for SoundFileClose");
				return Error;
			}

			SoundFileClose(FileHandle);
		}
		return Error;
	}

	ESoundFileError::Type FSoundFileIOManagerImpl::LoadSoundFileFromPath(const FString& FilePath, FSoundFileDescription& Description, TArray<ESoundFileChannelMap::Type>& ChannelMap, TArray<uint8>& BulkData)
	{
		ESoundFileError::Type Error = GetSoundFileInfoFromPath(FilePath, Description, ChannelMap);
		if (Error != ESoundFileError::Type::NONE)
		{
			return Error;
		}

		// Now read the data from disk into the bulk data array
		if (FFileHelper::LoadFileToArray(BulkData, *FilePath))
		{
			return ESoundFileError::Type::NONE;
		}
		else
		{
			return ESoundFileError::Type::FAILED_TO_LOAD_BYTE_DATA;
		}
	}
}

