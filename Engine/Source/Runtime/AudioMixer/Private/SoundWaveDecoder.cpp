// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundWaveDecoder.h"
#include "AudioThread.h"
#include "Misc/ScopeTryLock.h"
#include "AudioThread.h"
#include "AudioDecompress.h"
#include "AudioMixer.h"
#include "AudioMixerBuffer.h"
#include "AudioMixerSourceBuffer.h"

namespace Audio
{
	FDecodingSoundSource::FDecodingSoundSource(FAudioDevice* AudioDevice, const FSourceDecodeInit& InitData)
		: Handle(InitData.Handle)
		, AudioDeviceID(0)
		, SoundWave(InitData.SoundWave)
		, MixerBuffer(nullptr)
		, SampleRate(INDEX_NONE)
		, SeekTime(InitData.SeekTime)
		, bForceSyncDecode(InitData.bForceSyncDecode)
	{
		SourceInfo.VolumeParam.Init();
		SourceInfo.VolumeParam.SetValue(InitData.VolumeScale);
		SourceInfo.PitchScale = InitData.PitchScale;

		if (nullptr != AudioDevice)
		{
			AudioDeviceID = AudioDevice->DeviceID;
		}

		MixerBuffer = FMixerBuffer::Init(AudioDevice, InitData.SoundWave, true);
	}

	FDecodingSoundSource::~FDecodingSoundSource()
	{
		FScopeLock Lock(&MixerSourceBufferCritSec);

		if (MixerSourceBuffer.IsValid())
		{
			MixerSourceBuffer->OnEndGenerate();
			MixerSourceBuffer.Reset();
		}
	}

	bool FDecodingSoundSource::PreInit(int32 InSampleRate)
	{
		SampleRate = InSampleRate;

#if AUDIO_SOURCE_DECODER_DEBUG
		SineTone[0].Init(InSampleRate, 220.0f, 0.5f);
		SineTone[1].Init(InSampleRate, 440.0f, 0.5f);
#endif

		if (!SoundWave || !MixerBuffer)
		{
			return false;
		}

		const ELoopingMode LoopingMode = SoundWave->bLooping ? ELoopingMode::LOOP_Forever : ELoopingMode::LOOP_Never;
		const bool bIsSeeking = SeekTime > 0.0f;

		bool bIsValid = false;
		{
			FScopeLock Lock(&MixerSourceBufferCritSec);

			FMixerSourceBufferInitArgs Args;
			Args.AudioDeviceID = AudioDeviceID;
			Args.SampleRate = InSampleRate;
			Args.Buffer = MixerBuffer;
			Args.SoundWave = SoundWave;
			Args.LoopingMode = LoopingMode;
			Args.bIsSeeking = bIsSeeking;
			Args.bForceSyncDecode = bForceSyncDecode;
			Args.StartTime = SeekTime;
			MixerSourceBuffer = FMixerSourceBuffer::Create(Args);

			bIsValid = MixerSourceBuffer.IsValid();
		}

		return bIsValid;
	}

	bool FDecodingSoundSource::IsReadyToInit()
	{
		FScopeLock Lock(&MixerSourceBufferCritSec);
		if (!MixerSourceBuffer.IsValid())
		{
			return false;
		}

		if (MixerBuffer && MixerBuffer->IsRealTimeSourceReady())
		{
			// Check if we have a realtime audio task already (doing first decode)
			if (MixerSourceBuffer->IsAsyncTaskInProgress())
			{
				// not ready
				return MixerSourceBuffer->IsAsyncTaskDone();
			}
			else
			{
				// Now check to see if we need to kick off a decode the first chunk of audio
				const EBufferType::Type BufferType = MixerBuffer->GetType();
				if ((BufferType == EBufferType::PCMRealTime || BufferType == EBufferType::Streaming) && SoundWave)
				{
					// If any of these conditions meet, we need to do an initial async decode before we're ready to start playing the sound
					if (SeekTime > 0.0f || !SoundWave->CachedRealtimeFirstBuffer)
					{
						// Before reading more PCMRT data, we first need to seek the buffer
						if (SeekTime > 0.0f)
						{
							MixerBuffer->Seek(SeekTime);
						}

						ICompressedAudioInfo* CompressedAudioInfo = MixerBuffer->GetDecompressionState(false);

						MixerSourceBuffer->ReadMoreRealtimeData(CompressedAudioInfo, 0, EBufferReadMode::Asynchronous);

						// not ready
						return false;
					}
				}
			}

			return true;
		}
		return false;
	}

	void FDecodingSoundSource::Init()
	{
		if (MixerBuffer->GetNumChannels() > 0 && MixerBuffer->GetNumChannels() <= 2)
		{
			FScopeLock Lock(&MixerSourceBufferCritSec);
			if (MixerSourceBuffer.IsValid())
			{

				// Pass the decompression state off to the mixer source buffer if it hasn't already done so
				ICompressedAudioInfo* Decoder = MixerBuffer->GetDecompressionState(false);
				MixerSourceBuffer->SetDecoder(Decoder);

				if (!MixerSourceBuffer->IsAsyncTaskInProgress())
				{
					MixerSourceBuffer->ReadMoreRealtimeData(Decoder, 0, EBufferReadMode::Asynchronous);
				}

				SourceInfo.NumSourceChannels = MixerBuffer->GetNumChannels();
				SourceInfo.TotalNumFrames = MixerBuffer->GetNumFrames();

				SourceInfo.CurrentFrameValues.AddZeroed(SourceInfo.NumSourceChannels);
				SourceInfo.NextFrameValues.AddZeroed(SourceInfo.NumSourceChannels);

				SourceInfo.BasePitchScale = MixerBuffer->GetSampleRate() / SampleRate;

				SourceInfo.PitchParam.Init();
				SourceInfo.PitchParam.SetValue(SourceInfo.BasePitchScale * SourceInfo.PitchScale);

				MixerSourceBuffer->Init();
				MixerSourceBuffer->OnBeginGenerate();

				bInitialized = true;
			}
		}
	}

	void FDecodingSoundSource::SetPitchScale(float InPitchScale, uint32 NumFrames)
	{
		SourceInfo.PitchParam.SetValue(SourceInfo.BasePitchScale * InPitchScale, NumFrames);
		SourceInfo.PitchResetFrame = SourceInfo.NumFramesGenerated + NumFrames;
	}

	void FDecodingSoundSource::SetVolumeScale(float InVolumeScale, uint32 NumFrames)
	{
		SourceInfo.VolumeParam.SetValue(InVolumeScale, NumFrames);
		SourceInfo.VolumeResetFrame = SourceInfo.NumFramesGenerated + NumFrames;
	}

	void FDecodingSoundSource::SetForceSyncDecode(bool bShouldForceSyncDecode)
	{
		bForceSyncDecode = bShouldForceSyncDecode;
	}

	void FDecodingSoundSource::ReadFrame()
	{
		if (!MixerSourceBuffer.IsValid())
		{
			SourceInfo.bIsLastBuffer = true;
			return;
		}

		TSharedPtr<FMixerSourceBuffer, ESPMode::ThreadSafe>MixerSourceBufferLocal = MixerSourceBuffer;

		bool bNextFrameOutOfRange = (SourceInfo.CurrentFrameIndex + FMath::CeilToInt(SourceInfo.CurrentFrameAlpha)) >= SourceInfo.CurrentAudioChunkNumFrames;
		bool bCurrentFrameOutOfRange = SourceInfo.CurrentFrameIndex >= SourceInfo.CurrentAudioChunkNumFrames;

		bool bReadCurrentFrame = true;

		while (bNextFrameOutOfRange || bCurrentFrameOutOfRange)
		{
			if (bNextFrameOutOfRange && !bCurrentFrameOutOfRange)
			{
				bReadCurrentFrame = false;

				const float* AudioData = SourceInfo.CurrentPCMBuffer->GetData();

				if (!AudioData)
				{
					SourceInfo.bIsLastBuffer = true;
					return;
				}

				const int32 CurrentSampleIndex = SourceInfo.CurrentFrameIndex * SourceInfo.NumSourceChannels;

				for (int32 Channel = 0; Channel < SourceInfo.NumSourceChannels; ++Channel)
				{
					SourceInfo.CurrentFrameValues[Channel] = AudioData[CurrentSampleIndex + Channel];
				}

				MixerSourceBufferLocal->OnBufferEnd();
			}

			auto const NumBuffersQueued = MixerSourceBufferLocal->GetNumBuffersQueued();
			if (MixerSourceBufferLocal->GetNumBuffersQueued() > 0 && (SourceInfo.NumSourceChannels > 0))
			{
				check(MixerSourceBufferLocal.IsValid());
				SourceInfo.CurrentPCMBuffer = MixerSourceBufferLocal->GetNextBuffer();
				if (!SourceInfo.CurrentPCMBuffer)
				{
					SourceInfo.bIsLastBuffer = true;
					return;
				}

				SourceInfo.CurrentAudioChunkNumFrames = SourceInfo.CurrentPCMBuffer->Num() / SourceInfo.NumSourceChannels;

				if (bReadCurrentFrame)
				{
					SourceInfo.CurrentFrameIndex = FMath::Max(SourceInfo.CurrentFrameIndex - SourceInfo.CurrentAudioChunkNumFrames, 0);
				}
				else
				{
					SourceInfo.CurrentFrameIndex = INDEX_NONE;
				}
			}
			else
			{
				SourceInfo.bIsLastBuffer = true;
				return;
			}

			bNextFrameOutOfRange = (SourceInfo.CurrentFrameIndex + 1) >= SourceInfo.CurrentAudioChunkNumFrames;
			bCurrentFrameOutOfRange = SourceInfo.CurrentFrameIndex >= SourceInfo.CurrentAudioChunkNumFrames;
		}

		const float* AudioData = SourceInfo.CurrentPCMBuffer->GetData();

		if (!AudioData)
		{
			SourceInfo.bIsLastBuffer = true;
			return;
		}

		const int32 NextSampleIndex = (SourceInfo.CurrentFrameIndex + 1) * SourceInfo.NumSourceChannels;

		if (bReadCurrentFrame)
		{
			const int32 CurrentSampleIndex = SourceInfo.CurrentFrameIndex * SourceInfo.NumSourceChannels;
			for (int32 Channel = 0; Channel < SourceInfo.NumSourceChannels; ++Channel)
			{
				SourceInfo.CurrentFrameValues[Channel] = AudioData[CurrentSampleIndex + Channel];
				SourceInfo.NextFrameValues[Channel] = AudioData[NextSampleIndex + Channel];
			}
		}
		else
		{
			for (int32 Channel = 0; Channel < SourceInfo.NumSourceChannels; ++Channel)
			{
				SourceInfo.NextFrameValues[Channel] = AudioData[NextSampleIndex + Channel];
			}
		}
	}

	void FDecodingSoundSource::GetAudioBufferInternal(const int32 InNumFrames, const int32 InNumChannels, FAlignedFloatBuffer& OutAudioBuffer)
	{
#if AUDIO_SOURCE_DECODER_DEBUG
		int32 SampleIndex = 0;
		float* OutAudioBufferPtr = OutAudioBuffer.GetData();
		for (int32 FrameIndex = 0; FrameIndex < InNumFrames; ++FrameIndex)
		{
			for (int32 ChannelIndex = 0; ChannelIndex < InNumChannels; ++ChannelIndex)
			{
				OutAudioBufferPtr[SampleIndex++] = SineTone[ChannelIndex].ProcessAudio();
			}
		}
#else
		int32 SampleIndex = 0;
		float* OutAudioBufferPtr = OutAudioBuffer.GetData();
		float* CurrentFrameValuesPtr = SourceInfo.CurrentFrameValues.GetData();
		float* NextFrameValuesPtr = SourceInfo.NextFrameValues.GetData();

		for (int32 FrameIndex = 0; FrameIndex < InNumFrames; ++FrameIndex)
		{
			if (SourceInfo.bIsLastBuffer)
			{
				break;
			}

			bool bReadFrame = !SourceInfo.bHasStarted;
			SourceInfo.bHasStarted = true;

			while (SourceInfo.CurrentFrameAlpha >= 1.0f)
			{
				bReadFrame = true;
				SourceInfo.CurrentFrameIndex++;
				SourceInfo.NumFramesRead++;
				SourceInfo.CurrentFrameAlpha -= 1.0f;
			}


			if (!MixerSourceBuffer.IsValid())
			{
				bReadFrame = false;
				SourceInfo.bIsLastBuffer = true;
				break;
			}

			// assign "CurrentAlpha" before we update it so ReadFrame() can use the "next" value
			const float CurrentAlpha = SourceInfo.CurrentFrameAlpha;
			const float CurrentVolumeScale = SourceInfo.VolumeParam.Update();

			const float CurrentPitchScale = SourceInfo.PitchParam.Update();
			SourceInfo.CurrentFrameAlpha += CurrentPitchScale;

			if (bReadFrame)
			{
				ReadFrame();
				if (SourceInfo.bIsLastBuffer)
				{
					break;
				}
			}


			for (int32 Channel = 0; Channel < SourceInfo.NumSourceChannels; ++Channel)
			{
				const float CurrFrameValue = CurrentFrameValuesPtr[Channel];
				const float NextFrameValue = NextFrameValuesPtr[Channel];

				OutAudioBufferPtr[SampleIndex++] = CurrentVolumeScale * FMath::Lerp(CurrFrameValue, NextFrameValue, CurrentAlpha);
			}

			SourceInfo.NumFramesGenerated++;

			if (SourceInfo.NumFramesGenerated >= SourceInfo.PitchResetFrame)
			{
				SourceInfo.PitchResetFrame = INDEX_NONE;
				SourceInfo.PitchParam.Reset();
			}

			if (SourceInfo.NumFramesGenerated >= SourceInfo.VolumeResetFrame)
			{
				SourceInfo.VolumeResetFrame = INDEX_NONE;
				SourceInfo.VolumeParam.Reset();
			}
		}
#endif
	}

	bool FDecodingSoundSource::GetAudioBuffer(const int32 InNumFrames, const int32 InNumChannels, FAlignedFloatBuffer& OutAudioBuffer)
	{
		FScopeTryLock Lock(&MixerSourceBufferCritSec);

		if (!bInitialized || !Lock.IsLocked())
		{
			return false;
		}

		OutAudioBuffer.Reset();
		OutAudioBuffer.AddZeroed(InNumFrames * InNumChannels);

		if (SourceInfo.bIsLastBuffer)
		{
			return false;
		}

		if (InNumChannels == SourceInfo.NumSourceChannels)
		{
			GetAudioBufferInternal(InNumFrames, InNumChannels, OutAudioBuffer);
		}
		else
		{

			ScratchBuffer.Reset();
			ScratchBuffer.AddZeroed(InNumFrames * SourceInfo.NumSourceChannels);

			GetAudioBufferInternal(InNumFrames, InNumChannels, ScratchBuffer);

			float* BufferPtr = OutAudioBuffer.GetData();
			float* ScratchBufferPtr = ScratchBuffer.GetData();

			int32 OutputSampleIndex = 0;
			int32 InputSampleIndex = 0;

			// Need to upmix the audio
			if (InNumChannels == 2 && SourceInfo.NumSourceChannels == 1)
			{
				for (int32 FrameIndex = 0; FrameIndex < InNumFrames; ++FrameIndex, ++InputSampleIndex)
				{
					for (int32 ChannelIndex = 0; ChannelIndex < InNumChannels; ++ChannelIndex)
					{
						BufferPtr[OutputSampleIndex++] = 0.5f * ScratchBufferPtr[InputSampleIndex];
					}
				}
			}
			// Need to downmix the audio
			else
			{
				check(InNumChannels == 1 && SourceInfo.NumSourceChannels == 2);

				for (int32 FrameIndex = 0; FrameIndex < InNumFrames; ++FrameIndex, ++InputSampleIndex)
				{
					for (int32 ChannelIndex = 0; ChannelIndex < InNumChannels; ++ChannelIndex)
					{
						BufferPtr[OutputSampleIndex++] = 0.5f * (ScratchBufferPtr[InputSampleIndex] + ScratchBufferPtr[InputSampleIndex + 1]);
					}
				}
			}
		}

		return true;
	}

	FSoundSourceDecoder::FSoundSourceDecoder()
		: AudioThreadId(0)
		, AudioDevice(nullptr)
		, SampleRate(0)
	{
	}

	FSoundSourceDecoder::~FSoundSourceDecoder()
	{
		
	}

	void FSoundSourceDecoder::AddReferencedObjects(FReferenceCollector & Collector)
	{
		for (auto& Entry : PrecachingSources)
		{
			FSourceDecodeInit& DecodingSoundInitPtr = Entry.Value;
			Collector.AddReferencedObject(DecodingSoundInitPtr.SoundWave);
		}

		for (auto& Entry : InitializingDecodingSources)
		{
			FDecodingSoundSourcePtr DecodingSoundSourcePtr = Entry.Value;
			Collector.AddReferencedObject(DecodingSoundSourcePtr->GetSoundWavePtr());
		}

		FScopeLock Lock(&DecodingSourcesCritSec);
		for (auto& Entry : DecodingSources)
		{
			FDecodingSoundSourcePtr DecodingSoundSourcePtr = Entry.Value;
			Collector.AddReferencedObject(DecodingSoundSourcePtr->GetSoundWavePtr());
		}
	}

	void FSoundSourceDecoder::Init(FAudioDevice* InAudioDevice, int32 InSampleRate)
	{
		AudioDevice = InAudioDevice;
		SampleRate = InSampleRate;
	}

	FDecodingSoundSourceHandle FSoundSourceDecoder::CreateSourceHandle(USoundWave* InSoundWave)
	{
		// Init the handle ids
		static int32 SoundWaveDecodingHandles = 0;

		// Create a new handle
		FDecodingSoundSourceHandle Handle;
		Handle.Id = SoundWaveDecodingHandles++;
		Handle.SoundWaveName = InSoundWave->GetFName();
		return Handle;
	}

	void FSoundSourceDecoder::EnqueueDecoderCommand(TFunction<void()> Command)
	{
		CommandQueue.Enqueue(Command);
	}

	void FSoundSourceDecoder::PumpDecoderCommandQueue()
	{
		TFunction<void()> Command;
		while (CommandQueue.Dequeue(Command))
		{
			Command();
		}
	}

	bool FSoundSourceDecoder::InitDecodingSourceInternal(const FSourceDecodeInit& InitData)
	{
		FDecodingSoundSourcePtr DecodingSoundWaveDataPtr = FDecodingSoundSourcePtr(new FDecodingSoundSource(AudioDevice, InitData));

		if (DecodingSoundWaveDataPtr->PreInit(SampleRate))
		{
			DecodingSoundWaveDataPtr->SetForceSyncDecode(InitData.bForceSyncDecode);
			InitializingDecodingSources.Add(InitData.Handle.Id, DecodingSoundWaveDataPtr);

			// Add this decoding sound wave to a data structure we can access safely from audio render thread
			EnqueueDecoderCommand([this, InitData, DecodingSoundWaveDataPtr]()
			{
				FScopeLock Lock(&DecodingSourcesCritSec);
				DecodingSources.Add(InitData.Handle.Id, DecodingSoundWaveDataPtr);

				UE_LOGF(LogAudioMixer, Verbose, "Decoding SoundWave '%ls' (Num Decoding: %d)",
					*InitData.Handle.SoundWaveName.ToString(), DecodingSources.Num());
			});

			return true;
		}

		UE_LOGF(LogAudioMixer, Warning, "Failed to initialize sound wave %ls.", InitData.SoundWave ? *InitData.SoundWave->GetName() : TEXT("Unset"));
		return false;
	}

	bool FSoundSourceDecoder::InitDecodingSource(const FSourceDecodeInit& InitData)
	{
		check(IsInAudioThread());

		if (InitData.SoundWave == nullptr)
		{
			UE_LOGF(LogAudioMixer, Error, "Cannot Decode NULL SoundWave");
			return false;
		}

		if (InitData.SoundWave->NumChannels == 0)
		{
			UE_LOGF(LogAudioMixer, Error, "Cannot Decode invalid or corrupt sound wave %ls. NumChannels = 0", *InitData.SoundWave->GetName());
			return false;
		}

		if (InitData.SoundWave->NumChannels <= 0 || InitData.SoundWave->NumChannels > 2)
		{
			UE_LOGF(LogAudioMixer, Error, "Only supporting 1 or 2 channel decodes in sound source decoder.");
			return false;
		}


		if (InitData.SoundWave->bIsSourceBus || InitData.SoundWave->bProcedural)
		{
			UE_LOGF(LogAudioMixer, Warning, "Sound wave decoder does not support buses or procedural sounds.");
			return false;
		}

		// Start the soundwave precache
		const ESoundWavePrecacheState PrecacheState = InitData.SoundWave->GetPrecacheState();
		if (PrecacheState == ESoundWavePrecacheState::InProgress)
		{
			if (!PrecachingSources.Contains(InitData.Handle.Id))
			{
				PrecachingSources.Add(InitData.Handle.Id, InitData);
			}
			return true;
		}
		else
		{
			if (PrecacheState == ESoundWavePrecacheState::NotStarted)
			{
				AudioDevice->Precache(InitData.SoundWave, true, false, false);
			}
			check(InitData.SoundWave->GetPrecacheState() == ESoundWavePrecacheState::Done);
			return InitDecodingSourceInternal(InitData);
		}
	}

	void FSoundSourceDecoder::RemoveDecodingSource(const FDecodingSoundSourceHandle& Handle)
	{
		FScopeLock Lock(&DecodingSourcesCritSec);
		DecodingSources.Remove(Handle.Id);
	}

	void FSoundSourceDecoder::Reset()
	{
		PumpDecoderCommandQueue();

		FScopeLock Lock(&DecodingSourcesCritSec);

		DecodingSources.Reset();
		InitializingDecodingSources.Reset();
		PrecachingSources.Reset();
	}

	void FSoundSourceDecoder::SetSourcePitchScale(const FDecodingSoundSourceHandle& Handle, float InPitchScale)
	{

	}

	void FSoundSourceDecoder::SetSourceVolumeScale(const FDecodingSoundSourceHandle& InHandle, float InVolumeScale)
	{
		FScopeLock Lock(&DecodingSourcesCritSec);
		FDecodingSoundSourcePtr* DecodingSoundWaveDataPtr = DecodingSources.Find(InHandle.Id);
		if (!DecodingSoundWaveDataPtr)
		{
			return;
		}
		(*DecodingSoundWaveDataPtr)->SetVolumeScale(InVolumeScale);
	}

	void FSoundSourceDecoder::Update()
	{
		check(IsInAudioThread());

		TArray<int32> TempIds;

		for (auto& Entry : PrecachingSources)
		{
			int32 Id = Entry.Key;
			FSourceDecodeInit& InitData = Entry.Value;
			if (InitData.SoundWave->GetPrecacheState() == ESoundWavePrecacheState::Done)
			{
				InitDecodingSourceInternal(InitData);
				TempIds.Add(Id);
			}
		}

		// Remove the Id's that have initialized
		for (int32 Id : TempIds)
		{
			PrecachingSources.Remove(Id);
		}

		TempIds.Reset();
		for (auto& Entry : InitializingDecodingSources)
		{
			int32 Id = Entry.Key;
			FDecodingSoundSourcePtr DecodingSoundSourcePtr = Entry.Value;

			if (DecodingSoundSourcePtr->IsReadyToInit())
			{
				DecodingSoundSourcePtr->Init();

				// Add to local array here to clean up the map quickly
				TempIds.Add(Id);
			}
		}
		
		// Remove the Id's that have initialized
		for (int32 Id : TempIds)
		{
			InitializingDecodingSources.Remove(Id);
		}

	}

	void FSoundSourceDecoder::UpdateRenderThread()
	{
		PumpDecoderCommandQueue();
	}

	bool FSoundSourceDecoder::IsFinished(const FDecodingSoundSourceHandle& InHandle) const
	{
		FScopeLock Lock(&DecodingSourcesCritSec);

		const FDecodingSoundSourcePtr* DecodingSoundWaveDataPtr = DecodingSources.Find(InHandle.Id);
		if (!DecodingSoundWaveDataPtr || !DecodingSoundWaveDataPtr->IsValid())
		{
			return true;
		}

		return (*DecodingSoundWaveDataPtr)->IsFinished();
	}

	bool FSoundSourceDecoder::IsInitialized(const FDecodingSoundSourceHandle& InHandle) const
	{
		FScopeLock Lock(&DecodingSourcesCritSec);

		const FDecodingSoundSourcePtr* DecodingSoundWaveDataPtr = DecodingSources.Find(InHandle.Id);
		if (!DecodingSoundWaveDataPtr)
		{
			return true;
		}

		return (*DecodingSoundWaveDataPtr)->IsInitialized();
	}


	bool FSoundSourceDecoder::GetSourceBuffer(const FDecodingSoundSourceHandle& InHandle, const int32 NumOutFrames, const int32 NumOutChannels, FAlignedFloatBuffer& OutAudioBuffer)
	{
		check(InHandle.Id != INDEX_NONE);
		FScopeLock Lock(&DecodingSourcesCritSec);

		FDecodingSoundSourcePtr DecodingSoundWaveDataPtr = DecodingSources.FindRef(InHandle.Id);
		if (DecodingSoundWaveDataPtr.IsValid())
		{
			DecodingSoundWaveDataPtr->GetAudioBuffer(NumOutFrames, NumOutChannels, OutAudioBuffer);
			return true;
		}

		return false;
	}

}
