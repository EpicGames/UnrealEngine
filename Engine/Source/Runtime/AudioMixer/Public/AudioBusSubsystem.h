// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioMixerTrace.h"
#include "DSP/MultithreadedPatching.h"
#include "Misc/TVariant.h"
#include "Subsystems/AudioEngineSubsystem.h"
#include "Templates/TypeHash.h"
#include "UObject/StrongObjectPtr.h"

#include "AudioBusSubsystem.generated.h"

class UAudioBus;

namespace Audio
{
	// Forward declarations 
	class FMixerAudioBus;
	class FMixerSourceManager;

	struct FAudioBusKey
	{
		uint32 ObjectId = INDEX_NONE; // from a corresponding UObject (UAudioBus) if applicable
		uint32 InstanceId = INDEX_NONE;

		FAudioBusKey()
			: InstanceId(InstanceIdCounter++)
		{
		}

		// For construction with a given UObject unique id 
		FAudioBusKey(uint32 InObjectId)
			: ObjectId(InObjectId)
		{
		}

		const bool IsValid() const
		{
			return ObjectId != INDEX_NONE || InstanceId != INDEX_NONE;
		}

		inline friend uint32 GetTypeHash(const FAudioBusKey& Key)
		{
			return HashCombineFast(Key.ObjectId, Key.InstanceId);
		}
		 		
		inline friend bool operator==(const FAudioBusKey& InLHS, const FAudioBusKey& InRHS) 
		{
			return (InLHS.ObjectId == InRHS.ObjectId) && (InLHS.InstanceId == InRHS.InstanceId);
		}

		inline friend bool operator!=(const FAudioBusKey& InLHS, const FAudioBusKey& InRHS) 
		{
			return !(InLHS == InRHS);
		}
		

	private:
		static AUDIOMIXER_API std::atomic<uint32> InstanceIdCounter;
	};
}

/**
*  UAudioBusSubsystem
*/
UCLASS(MinimalAPI)
class UAudioBusSubsystem : public UAudioEngineSubsystem
{
	GENERATED_BODY()

public:
	AUDIOMIXER_API UAudioBusSubsystem();
	virtual ~UAudioBusSubsystem() = default;

	//~ Begin USubsystem interface
	AUDIOMIXER_API virtual bool ShouldCreateSubsystem(UObject* Outer) const override;
	AUDIOMIXER_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	AUDIOMIXER_API virtual void Deinitialize() override;
	//~ End USubsystem interface

	// Audio bus API from FMixerDevice
	UE_DEPRECATED(5.6, "Use the StartAudioBus version that requires an AudioBus name.")
	AUDIOMIXER_API void StartAudioBus(Audio::FAudioBusKey InAudioBusKey, int32 InNumChannels, bool bInIsAutomatic);

	AUDIOMIXER_API void StartAudioBus(Audio::FAudioBusKey InAudioBusKey, const FString& InAudioBusName, int32 InNumChannels, bool bInIsAutomatic);
	AUDIOMIXER_API void StopAudioBus(Audio::FAudioBusKey InAudioBusKey);
	AUDIOMIXER_API bool IsAudioBusActive(Audio::FAudioBusKey InAudioBusKey) const;
	
	AUDIOMIXER_API Audio::FPatchInput AddPatchInputForAudioBus(Audio::FAudioBusKey InAudioBusKey, int32 InFrames, int32 InChannels, float InGain = 1.f);
	AUDIOMIXER_API Audio::FPatchOutputStrongPtr AddPatchOutputForAudioBus(Audio::FAudioBusKey InAudioBusKey, int32 InFrames, int32 InChannels, float InGain = 1.f);

	AUDIOMIXER_API Audio::FPatchInput AddPatchInputForSoundAndAudioBus(uint64 SoundInstanceID, Audio::FAudioBusKey AudioBusKey, int32 InFrames, int32 NumChannels, float InGain = 1.f);
	AUDIOMIXER_API Audio::FPatchOutputStrongPtr AddPatchOutputForSoundAndAudioBus(uint64 SoundInstanceID, Audio::FAudioBusKey AudioBusKey, int32 InFrames, int32 NumChannels, float InGain = 1.f);
	AUDIOMIXER_API void ConnectPatches(uint64 SoundInstanceID);
	AUDIOMIXER_API void RemoveSound(uint64 SoundInstanceID);

	AUDIOMIXER_API void InitDefaultAudioBuses();
	AUDIOMIXER_API void ShutdownDefaultAudioBuses();

private:
	struct FActiveBusData
	{
		Audio::FAudioBusKey BusKey = 0;
		int32 NumChannels = 0;
		bool bIsAutomatic = false;
	};

	TArray<TStrongObjectPtr<UAudioBus>> DefaultAudioBuses; 
	// The active audio bus list accessible on the game thread
	TMap<Audio::FAudioBusKey, FActiveBusData> ActiveAudioBuses_GameThread;

	struct FPendingConnection
	{
		using FPatchVariant = TVariant<Audio::FPatchInput, Audio::FPatchOutputStrongPtr>;
		FPatchVariant PatchVariant;
		Audio::FAudioBusKey AudioBusKey;
		int32 BlockSizeFrames = 0;
		int32 NumChannels = 0;
		bool bIsAutomatic = false;

#if UE_AUDIO_PROFILERTRACE_ENABLED
		uint64 InstanceID = 0u; // MixerSourceBuffer instance ID, used for PlayOrder resolution in trace profiling
#endif // UE_AUDIO_PROFILERTRACE_ENABLED
	};

	void AddPendingConnection(uint64 SoundInstanceID, FPendingConnection&& PendingConnection);

	struct FSoundInstanceConnections
	{
		TArray<FPendingConnection> PendingConnections;
	};

	TArray<FPendingConnection> ExtractPendingConnectionsIfReady(uint64 SoundInstanceID);

	TMap<uint64, FSoundInstanceConnections> SoundInstanceConnectionMap;
	FCriticalSection Mutex;
};
