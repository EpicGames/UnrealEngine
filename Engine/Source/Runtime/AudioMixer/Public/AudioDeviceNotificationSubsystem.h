// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/EngineSubsystem.h"
#include "Engine/Engine.h"
#include "AudioMixer.h"
#include "Delegates/Delegate.h"
#include "AudioDeviceNotificationSubsystem.generated.h"

/**
 *	EAudioDeviceChangedRole
 */
UENUM(BlueprintType)
enum class EAudioDeviceChangedRole : uint8
{
	Invalid,
	Console,
	Multimedia,
	Communications,
	Count UMETA(Hidden)
};

/**
 *	EAudioDeviceChangedState
 */
UENUM(BlueprintType)
enum class EAudioDeviceChangedState : uint8
{
	Invalid,
	Active,
	Disabled,
	NotPresent,
	Unplugged,
	Count UMETA(Hidden)
};

DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnAudioDefaultDeviceChanged, EAudioDeviceChangedRole, AudioDeviceRole, FString, DeviceId);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnAudioDeviceStateChanged, FString, DeviceId, EAudioDeviceChangedState, NewState);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnAudioDeviceChange, FString, DeviceId);

DECLARE_MULTICAST_DELEGATE_TwoParams(FOnAudioDefaultDeviceChangedNative, EAudioDeviceChangedRole, FString);
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnAudioDeviceStateChangedNative, FString, EAudioDeviceChangedState);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnAudioDeviceChangeNative, FString);

/**
 *  UAudioDeviceNotificationSubsystem
 */
UCLASS(MinimalAPI)
class UAudioDeviceNotificationSubsystem : public UEngineSubsystem
													   , public Audio::IAudioMixerDeviceChangedListener
{
	GENERATED_BODY()

public: 

	virtual ~UAudioDeviceNotificationSubsystem() = default;

	static UAudioDeviceNotificationSubsystem* Get() { return GEngine->GetEngineSubsystem<UAudioDeviceNotificationSubsystem>(); }

	//~ Begin UEngineSubsystem Interface
	AUDIOMIXER_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	AUDIOMIXER_API virtual void Deinitialize() override;
	//~ End UEngineSubsystem Interface

	//~ Begin IAudioMixerDeviceChangedListener Interface
	AUDIOMIXER_API virtual void OnDefaultCaptureDeviceChanged(const Audio::EAudioDeviceRole InAudioDeviceRole, const FString& DeviceId) override;
	AUDIOMIXER_API virtual void OnDefaultRenderDeviceChanged(const Audio::EAudioDeviceRole InAudioDeviceRole, const FString& DeviceId) override;
	AUDIOMIXER_API virtual void OnDeviceAdded(const FString& DeviceId, bool bIsRenderDevice) override;
	AUDIOMIXER_API virtual void OnDeviceRemoved(const FString& DeviceId, bool bIsRenderDevice) override;
	AUDIOMIXER_API virtual void OnDeviceStateChanged(const FString& DeviceId, const Audio::EAudioDeviceState InState, bool bIsRenderDevice) override;
	//~ End IAudioMixerDeviceChangedListener Interface

	AUDIOMIXER_API virtual void OnDeviceSwitched(const FString& DeviceId);

	/** Multicast delegate triggered when default capture device changes */
	UPROPERTY(BlueprintAssignable, Category = "Audio Delegates")
	FOnAudioDefaultDeviceChanged DefaultCaptureDeviceChanged;
	/** Multicast delegate triggered when default capture device changes (native code only) */
	FOnAudioDefaultDeviceChangedNative DefaultCaptureDeviceChangedNative;

	/** Multicast delegate triggered when default render device changes */
	UPROPERTY(BlueprintAssignable, Category = "Audio Delegates")
	FOnAudioDefaultDeviceChanged DefaultRenderDeviceChanged;
	/** Multicast delegate triggered when default render device changes (native code only) */
	FOnAudioDefaultDeviceChangedNative DefaultRenderDeviceChangedNative;

	/** Multicast delegate triggered when a device is added */
	UPROPERTY(BlueprintAssignable, Category = "Audio Delegates")
	FOnAudioDeviceChange DeviceAdded;
	/** Multicast delegate triggered when a device is added (native code only) */
	FOnAudioDeviceChangeNative DeviceAddedNative;

	/** Multicast delegate triggered when a device is removed */
	UPROPERTY(BlueprintAssignable, Category = "Audio Delegates")
	FOnAudioDeviceChange DeviceRemoved;
	/** Multicast delegate triggered when a device is removed (native code only) */
	FOnAudioDeviceChangeNative DeviceRemovedNative;

	/** Multicast delegate triggered on device state change */
	UPROPERTY(BlueprintAssignable, Category = "Audio Delegates")
	FOnAudioDeviceStateChanged DeviceStateChanged;
	/** Multicast delegate triggered on device state change (native code only) */
	FOnAudioDeviceStateChangedNative DeviceStateChangedNative;

	/** Multicast delegate triggered on device switch */
	UPROPERTY(BlueprintAssignable, Category = "Audio Delegates")
	FOnAudioDeviceChange DeviceSwitched;
	/** Multicast delegate triggered on device switch (native code only) */
	FOnAudioDeviceChangeNative DeviceSwitchedNative;

protected:

	AUDIOMIXER_API EAudioDeviceChangedRole GetDeviceChangedRole(Audio::EAudioDeviceRole InRole) const;
	AUDIOMIXER_API EAudioDeviceChangedState GetDeviceChangedState(Audio::EAudioDeviceState InState) const;
};

