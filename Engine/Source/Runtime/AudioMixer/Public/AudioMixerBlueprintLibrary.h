// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "SubmixEffects/AudioMixerSubmixEffectDynamicsProcessor.h"
#include "Sound/SoundEffectSource.h"
#include "Sound/AudioBus.h"
#include "SampleBuffer.h"
#include "Sound/SoundCue.h"
#include "Sound/SoundSubmixSend.h"
#include "DSP/SpectrumAnalyzer.h"
#include "AudioMixer.h"
#include "AudioMixerTypes.h"
#include "AudioMixerBlueprintLibrary.generated.h"

class USoundSubmix;

/** 
* Called when a load request for a sound has completed.
*/
DECLARE_DYNAMIC_DELEGATE_TwoParams(FOnSoundLoadComplete, const class USoundWave*, LoadedSoundWave, const bool, WasCancelled);



UENUM(BlueprintType)
enum class EMusicalNoteName : uint8
{
	C  = 0,
	Db = 1,
	D  = 2,
	Eb = 3,
	E  = 4,
	F  = 5,
	Gb = 6,
	G  = 7,
	Ab = 8,
	A  = 9,
	Bb = 10,
	B  = 11,
};

//Duplicate of Audio::EAudioMixerStreamDataFormat::Type, to get around UHT's lack of namespace support
UENUM()
enum class EAudioMixerStreamDataFormatType : uint8
{
	Unknown,
	Float,
	Int16,
	Unsupported
};

FString DataFormatAsString(EAudioMixerStreamDataFormatType type);

//A copy of Audio::EAudioMixerChannel::Type to get around UHT's refusal of namespaces
UENUM()
enum class EAudioMixerChannelType : uint8
{
	FrontLeft,
	FrontRight,
	FrontCenter,
	LowFrequency,
	BackLeft,
	BackRight,
	FrontLeftOfCenter,
	FrontRightOfCenter,
	BackCenter,
	SideLeft,
	SideRight,
	TopCenter,
	TopFrontLeft,
	TopFrontCenter,
	TopFrontRight,
	TopBackLeft,
	TopBackCenter,
	TopBackRight,
	Unknown,
	ChannelTypeCount,
	DefaultChannel = FrontLeft
};

inline const TCHAR* ToString(EAudioMixerChannelType InType)
{
	switch (InType)
	{
		case EAudioMixerChannelType::FrontLeft:				return TEXT("FrontLeft");
		case EAudioMixerChannelType::FrontRight:			return TEXT("FrontRight");
		case EAudioMixerChannelType::FrontCenter:			return TEXT("FrontCenter");
		case EAudioMixerChannelType::LowFrequency:			return TEXT("LowFrequency");
		case EAudioMixerChannelType::BackLeft:				return TEXT("BackLeft");
		case EAudioMixerChannelType::BackRight:				return TEXT("BackRight");
		case EAudioMixerChannelType::FrontLeftOfCenter:		return TEXT("FrontLeftOfCenter");
		case EAudioMixerChannelType::FrontRightOfCenter:	return TEXT("FrontRightOfCenter");
		case EAudioMixerChannelType::BackCenter:			return TEXT("BackCenter");
		case EAudioMixerChannelType::SideLeft:				return TEXT("SideLeft");
		case EAudioMixerChannelType::SideRight:				return TEXT("SideRight");
		case EAudioMixerChannelType::TopCenter:				return TEXT("TopCenter");
		case EAudioMixerChannelType::TopFrontLeft:			return TEXT("TopFrontLeft");
		case EAudioMixerChannelType::TopFrontCenter:		return TEXT("TopFrontCenter");
		case EAudioMixerChannelType::TopFrontRight:			return TEXT("TopFrontRight");
		case EAudioMixerChannelType::TopBackLeft:			return TEXT("TopBackLeft");
		case EAudioMixerChannelType::TopBackCenter:			return TEXT("TopBackCenter");
		case EAudioMixerChannelType::TopBackRight:			return TEXT("TopBackRight");
		case EAudioMixerChannelType::Unknown:				return TEXT("Unknown");

		default:
			return TEXT("UNSUPPORTED");
	}
}

// Resulting State of SwapAudioOutputDevice call
UENUM(BlueprintType)
enum class ESwapAudioOutputDeviceResultState : uint8
{
	Failure, 
	Success, 
	None,
};

/**
 * Out structure for use with AudioMixerBlueprintLibrary::SwapAudioOutputDevice
 */
USTRUCT(BlueprintType)
struct FSwapAudioOutputResult
{
	GENERATED_USTRUCT_BODY()

	FSwapAudioOutputResult() = default;

	/** ID of the currently set device.  This is the device at the time of the call, NOT the resulting deviceId */
	UPROPERTY(BlueprintReadOnly, Category = "Audio")
	FString CurrentDeviceId;

	/** ID of the requested device. */
	UPROPERTY(BlueprintReadOnly, Category = "Audio")
	FString RequestedDeviceId;

	/** Result of the call */
	UPROPERTY(BlueprintReadOnly, Category = "Audio")
	ESwapAudioOutputDeviceResultState Result = ESwapAudioOutputDeviceResultState::None;
};

/**
 * Platform audio output device info, in a Blueprint-readable format
 */
USTRUCT(BlueprintType)
struct FAudioOutputDeviceInfo
{
	GENERATED_USTRUCT_BODY()

	FAudioOutputDeviceInfo()
		: Name("")
		, DeviceId("")
		, NumChannels(0)
		, SampleRate(0)
		, Format(EAudioMixerStreamDataFormatType::Unknown)
		, bIsSystemDefault(true)
		, bIsCurrentDevice(false)
	{};

	AUDIOMIXER_API FAudioOutputDeviceInfo(const Audio::FAudioPlatformDeviceInfo& InDeviceInfo);

	/** The name of the audio device */
	UPROPERTY(BlueprintReadOnly, Category="Audio")
	FString Name;

	/** ID of the device. */
	UPROPERTY(BlueprintReadOnly, Category = "Audio")
	FString DeviceId;

	/** The number of channels supported by the audio device */
	UPROPERTY(BlueprintReadOnly, Category = "Audio")
	int32 NumChannels = 0;

	/** The sample rate of the audio device */
	UPROPERTY(BlueprintReadOnly, Category = "Audio")
	int32 SampleRate = 0;

	/** The data format of the audio stream */
	UPROPERTY(BlueprintReadOnly, Category = "Audio")
	EAudioMixerStreamDataFormatType Format = EAudioMixerStreamDataFormatType::Unknown;

	/** The output channel array of the audio device */
	UPROPERTY(BlueprintReadOnly, Category = "Audio")
	TArray<EAudioMixerChannelType> OutputChannelArray;

	/** Whether or not this device is the system default */
	UPROPERTY(BlueprintReadOnly, Category = "Audio")
	uint8 bIsSystemDefault : 1;

	/** Whether or not this device is the device currently in use */
	UPROPERTY(BlueprintReadOnly, Category = "Audio")
	uint8 bIsCurrentDevice : 1;
};

/**
 * Called when a list of all available audio devices is retrieved
 */
DECLARE_DYNAMIC_DELEGATE_OneParam(FOnAudioOutputDevicesObtained, const TArray<FAudioOutputDeviceInfo>&, AvailableDevices);

/**
 * Called when a list of all available audio devices is retrieved
 */
DECLARE_DYNAMIC_DELEGATE_OneParam(FOnMainAudioOutputDeviceObtained, const FString&, CurrentDevice);

/**
 * Called when the system has swapped to another audio output device
 */
DECLARE_DYNAMIC_DELEGATE_OneParam(FOnCompletedDeviceSwap, const FSwapAudioOutputResult&, SwapResult);

UCLASS(meta=(ScriptName="AudioMixerLibrary"), MinimalAPI)
class UAudioMixerBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/**
	* Returns the device info in a human readable format
	* @param info - The audio device data to print
	* @return The data in a string format
	*/
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Audio Output Device Info To String", CompactNodeTitle = "->", BlueprintAutocast), Category = "Audio")
	static AUDIOMIXER_API FString Conv_AudioOutputDeviceInfoToString(const FAudioOutputDeviceInfo& Info);

	/** Adds a submix effect preset to the master submix. */
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects", meta=(WorldContext="WorldContextObject"))
	static AUDIOMIXER_API void AddMasterSubmixEffect(const UObject* WorldContextObject, USoundEffectSubmixPreset* SubmixEffectPreset);

	/** Removes a submix effect preset from the master submix. */
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects", meta=(WorldContext="WorldContextObject"))
	static AUDIOMIXER_API void RemoveMasterSubmixEffect(const UObject* WorldContextObject, USoundEffectSubmixPreset* SubmixEffectPreset);

	/** Clears all master submix effects. */
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects", meta = (WorldContext = "WorldContextObject"))
	static AUDIOMIXER_API void ClearMasterSubmixEffects(const UObject* WorldContextObject);

	/** Adds a submix effect preset to the given submix at the end of its submix effect chain. Returns the number of submix effects. */
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects", meta = (WorldContext = "WorldContextObject"))
	static AUDIOMIXER_API int32 AddSubmixEffect(const UObject* WorldContextObject, USoundSubmix* SoundSubmix, USoundEffectSubmixPreset* SubmixEffectPreset);

	UE_DEPRECATED(4.27, "RemoveSubmixEffectPreset is deprecated, use RemoveSubmixEffect.")
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects", meta = (WorldContext = "WorldContextObject", DeprecatedFunction))
	static AUDIOMIXER_API void RemoveSubmixEffectPreset(const UObject* WorldContextObject, USoundSubmix* SoundSubmix, USoundEffectSubmixPreset* SubmixEffectPreset);

	/** Removes all instances of a submix effect preset from the given submix. */
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects", meta = (WorldContext = "WorldContextObject"))
	static AUDIOMIXER_API void RemoveSubmixEffect(const UObject* WorldContextObject, USoundSubmix* SoundSubmix, USoundEffectSubmixPreset* SubmixEffectPreset);

	UE_DEPRECATED(4.27, "RemoveSubmixEffectPresetAtIndex is deprecated, use RemoveSubmixEffectAtIndex.")
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects", meta = (WorldContext = "WorldContextObject", DeprecatedFunction))
	static AUDIOMIXER_API void RemoveSubmixEffectPresetAtIndex(const UObject* WorldContextObject, USoundSubmix* SoundSubmix, int32 SubmixChainIndex);

	/** Removes the submix effect at the given submix chain index, if there is a submix effect at that index. */
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects", meta = (WorldContext = "WorldContextObject"))
	static AUDIOMIXER_API void RemoveSubmixEffectAtIndex(const UObject* WorldContextObject, USoundSubmix* SoundSubmix, int32 SubmixChainIndex);

	UE_DEPRECATED(4.27, "ReplaceSoundEffectSubmix is deprecated, use ReplaceSubmixEffect.")
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects", meta = (WorldContext = "WorldContextObject", DeprecatedFunction))
	static AUDIOMIXER_API void ReplaceSoundEffectSubmix(const UObject* WorldContextObject, USoundSubmix* InSoundSubmix, int32 SubmixChainIndex, USoundEffectSubmixPreset* SubmixEffectPreset);

	/** Replaces the submix effect at the given submix chain index, adds the effect if there is none at that index. */
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects", meta = (WorldContext = "WorldContextObject"))
	static AUDIOMIXER_API void ReplaceSubmixEffect(const UObject* WorldContextObject, USoundSubmix* InSoundSubmix, int32 SubmixChainIndex, USoundEffectSubmixPreset* SubmixEffectPreset);

	/** Clears all submix effects on the given submix. */
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects", meta = (WorldContext = "WorldContextObject"))
	static AUDIOMIXER_API void ClearSubmixEffects(const UObject* WorldContextObject, USoundSubmix* SoundSubmix);

	/** Sets a submix effect chain override on the given submix. The effect chain will cross fade from the base effect chain or current override to the new override. */
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects", meta = (WorldContext = "WorldContextObject"))
	static AUDIOMIXER_API void SetSubmixEffectChainOverride(const UObject* WorldContextObject, USoundSubmix* SoundSubmix, TArray<USoundEffectSubmixPreset*> SubmixEffectPresetChain, float FadeTimeSec);

	/** Clears all submix effect overrides on the given submix and returns it to the default effect chain. */
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects", meta = (WorldContext = "WorldContextObject"))
	static AUDIOMIXER_API void ClearSubmixEffectChainOverride(const UObject* WorldContextObject, USoundSubmix* SoundSubmix, float FadeTimeSec);

	/** Start recording audio. By leaving the Submix To Record field blank, you can record the master output of the game. */
	UFUNCTION(BlueprintCallable, Category = "Audio|Recording", meta = (WorldContext = "WorldContextObject", AdvancedDisplay = 1))
	static AUDIOMIXER_API void StartRecordingOutput(const UObject* WorldContextObject, float ExpectedDuration, USoundSubmix* SubmixToRecord = nullptr);
	
	/** Stop recording audio. Path can be absolute, or relative (to the /Saved/BouncedWavFiles folder). By leaving the Submix To Record field blank, you can record the master output of the game.  */
	UFUNCTION(BlueprintCallable, Category = "Audio|Recording", meta = (WorldContext = "WorldContextObject", DisplayName = "Finish Recording Output", AdvancedDisplay = 4))
	static AUDIOMIXER_API USoundWave* StopRecordingOutput(const UObject* WorldContextObject, EAudioRecordingExportType ExportType, const FString& Name, FString Path, USoundSubmix* SubmixToRecord = nullptr, USoundWave* ExistingSoundWaveToOverwrite= nullptr);

	/** Pause recording audio, without finalizing the recording to disk. By leaving the Submix To Record field blank, you can record the master output of the game. */
	UFUNCTION(BlueprintCallable, Category = "Audio|Recording", meta = (WorldContext = "WorldContextObject", AdvancedDisplay = 1))
	static AUDIOMIXER_API void PauseRecordingOutput(const UObject* WorldContextObject, USoundSubmix* SubmixToPause = nullptr);

	/** Resume recording audio after pausing. By leaving the Submix To Pause field blank, you can record the master output of the game. */
	UFUNCTION(BlueprintCallable, Category = "Audio|Recording", meta = (WorldContext = "WorldContextObject", AdvancedDisplay = 1))
	static AUDIOMIXER_API void ResumeRecordingOutput(const UObject* WorldContextObject, USoundSubmix* SubmixToPause = nullptr);

	/** Start spectrum analysis of the audio output. By leaving the Submix To Analyze blank, you can analyze the master output of the game. */
	UFUNCTION(BlueprintCallable, Category = "Audio|Analysis", meta = (WorldContext = "WorldContextObject", AdvancedDisplay = 1))
	static AUDIOMIXER_API void StartAnalyzingOutput(const UObject* WorldContextObject, USoundSubmix* SubmixToAnalyze = nullptr, EFFTSize FFTSize = EFFTSize::DefaultSize, EFFTPeakInterpolationMethod InterpolationMethod = EFFTPeakInterpolationMethod::Linear, EFFTWindowType WindowType = EFFTWindowType::Hann, float HopSize = 0, EAudioSpectrumType SpectrumType = EAudioSpectrumType::MagnitudeSpectrum);

	/** Stop spectrum analysis. */
	UFUNCTION(BlueprintCallable, Category = "Audio|Analysis", meta = (WorldContext = "WorldContextObject", AdvancedDisplay = 1))
	static AUDIOMIXER_API void StopAnalyzingOutput(const UObject* WorldContextObject, USoundSubmix* SubmixToStopAnalyzing = nullptr);

	/** Make an array of musically spaced bands with ascending frequency.
	 *
	 *  @param InNumSemitones - The number of semitones to represent.
	 *  @param InStartingMuiscalNote - The name of the first note in the array.
	 *  @param InStartingOctave - The octave of the first note in the array.
	 *  @param InAttackTimeMsec - The attack time (in milliseconds) to apply to each band's envelope tracker.
	 *  @param InReleaseTimeMsec - The release time (in milliseconds) to apply to each band's envelope tracker.
	 */
	UFUNCTION(BlueprintPure, Category = "Audio|Analysis", meta = (AdvancedDisplay = 3))
	static AUDIOMIXER_API TArray<FSoundSubmixSpectralAnalysisBandSettings> MakeMusicalSpectralAnalysisBandSettings(int32 InNumSemitones=60, EMusicalNoteName InStartingMusicalNote = EMusicalNoteName::C, int32 InStartingOctave = 2, int32 InAttackTimeMsec = 10, int32 InReleaseTimeMsec = 10);

	/** Make an array of logarithmically spaced bands. 
	 *
	 *  @param InNumBands - The number of bands to used to represent the spectrum.
	 *  @param InMinimumFrequency - The center frequency of the first band.
	 *  @param InMaximumFrequency - The center frequency of the last band.
	 *  @param InAttackTimeMsec - The attack time (in milliseconds) to apply to each band's envelope tracker.
	 *  @param InReleaseTimeMsec - The release time (in milliseconds) to apply to each band's envelope tracker.
	 */
	UFUNCTION(BlueprintPure, Category = "Audio|Analysis", meta = (AdvancedDisplay = 3))
	static AUDIOMIXER_API TArray<FSoundSubmixSpectralAnalysisBandSettings> MakeFullSpectrumSpectralAnalysisBandSettings(int32 InNumBands = 30, float InMinimumFrequency=40.f, float InMaximumFrequency=16000.f, int32 InAttackTimeMsec = 10, int32 InReleaseTimeMsec = 10);

	/** Make an array of bands which span the frequency range of a given EAudioSpectrumBandPresetType. 
	 *
	 *  @param InBandPresetType - The type audio content which the bands encompass.
	 *  @param InNumBands - The number of bands used to represent the spectrum.
	 *  @param InAttackTimeMsec - The attack time (in milliseconds) to apply to each band's envelope tracker.
	 *  @param InReleaseTimeMsec - The release time (in milliseconds) to apply to each band's envelope tracker.
	 */
	UFUNCTION(BlueprintPure, Category = "Audio|Analysis", meta = (AdvancedDisplay = 2))
	static AUDIOMIXER_API TArray<FSoundSubmixSpectralAnalysisBandSettings> MakePresetSpectralAnalysisBandSettings(EAudioSpectrumBandPresetType InBandPresetType, int32 InNumBands = 10, int32 InAttackTimeMsec = 10, int32 InReleaseTimeMsec = 10);

	/** Retrieve the magnitudes for the given frequencies. */
	UFUNCTION(BlueprintCallable, Category = "Audio|Analysis", meta = (WorldContext = "WorldContextObject", AdvancedDisplay = 3))
	static AUDIOMIXER_API void GetMagnitudeForFrequencies(const UObject* WorldContextObject, const TArray<float>& Frequencies, TArray<float>& Magnitudes, USoundSubmix* SubmixToAnalyze = nullptr);

	/** Retrieve the phases for the given frequencies. */
	UFUNCTION(BlueprintCallable, Category = "Audio|Analysis", meta = (WorldContext = "WorldContextObject", AdvancedDisplay = 3))
	static AUDIOMIXER_API void GetPhaseForFrequencies(const UObject* WorldContextObject, const TArray<float>& Frequencies, TArray<float>& Phases, USoundSubmix* SubmixToAnalyze = nullptr);

	/** Adds source effect entry to preset chain. Only effects the instance of the preset chain */
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects", meta = (WorldContext = "WorldContextObject"))
	static AUDIOMIXER_API void AddSourceEffectToPresetChain(const UObject* WorldContextObject, USoundEffectSourcePresetChain* PresetChain, FSourceEffectChainEntry Entry);

	/** Removes source effect entry from preset chain. Only affects the instance of preset chain. */
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects", meta = (WorldContext = "WorldContextObject"))
	static AUDIOMIXER_API void RemoveSourceEffectFromPresetChain(const UObject* WorldContextObject, USoundEffectSourcePresetChain* PresetChain, int32 EntryIndex);

	/** Set whether or not to bypass the effect at the source effect chain index. */
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects", meta = (WorldContext = "WorldContextObject"))
	static AUDIOMIXER_API void SetBypassSourceEffectChainEntry(const UObject* WorldContextObject, USoundEffectSourcePresetChain* PresetChain, int32 EntryIndex, bool bBypassed);

	/** Returns the number of effect chain entries in the given source effect chain. */
	UFUNCTION(BlueprintCallable, Category = "Audio|Effects", meta = (WorldContext = "WorldContextObject"))
	static AUDIOMIXER_API int32 GetNumberOfEntriesInSourceEffectChain(const UObject* WorldContextObject, USoundEffectSourcePresetChain* PresetChain);

	/** Begin loading a sound into the cache so that it can be played immediately. */
	UFUNCTION(BlueprintCallable, Category = "Audio|Cache")
	static AUDIOMIXER_API void PrimeSoundForPlayback(USoundWave* SoundWave, const FOnSoundLoadComplete OnLoadCompletion);

	/** Begin loading any sounds referenced by a sound cue into the cache so that it can be played immediately. */
	UFUNCTION(BlueprintCallable, Category = "Audio|Cache")
	static AUDIOMIXER_API void PrimeSoundCueForPlayback(USoundCue* SoundCue);

	/** Trim memory used by the audio cache. Returns the number of megabytes freed. */
	UFUNCTION(BlueprintCallable, Category = "Audio|Cache")
	static AUDIOMIXER_API float TrimAudioCache(float InMegabytesToFree);

	/** Starts the given audio bus. */
	UFUNCTION(BlueprintCallable, Category = "Audio|Bus", meta = (WorldContext = "WorldContextObject"))
	static AUDIOMIXER_API void StartAudioBus(const UObject* WorldContextObject, UAudioBus* AudioBus);

	/** Stops the given audio bus. */
	UFUNCTION(BlueprintCallable, Category = "Audio|Bus", meta = (WorldContext = "WorldContextObject"))
	static AUDIOMIXER_API void StopAudioBus(const UObject* WorldContextObject, UAudioBus* AudioBus);

	/** Queries if the given audio bus is active (and audio can be mixed to it). */
	UFUNCTION(BlueprintCallable, Category = "Audio|Bus", meta = (WorldContext = "WorldContextObject"))
	static AUDIOMIXER_API bool IsAudioBusActive(const UObject* WorldContextObject, UAudioBus* AudioBus);

	/** Registers an audio bus to a submix so the submix output can be routed to the audiobus. */
	UFUNCTION(BlueprintCallable, Category = "Audio|Bus", meta = (WorldContext = "WorldContextObject"))
	static AUDIOMIXER_API void RegisterAudioBusToSubmix(const UObject* WorldContextObject, USoundSubmix* SoundSubmix, UAudioBus* AudioBus);

	/** Unregisters an audio bus that could have been registered to a submix. */
	UFUNCTION(BlueprintCallable, Category = "Audio|Bus", meta = (WorldContext = "WorldContextObject"))
	static AUDIOMIXER_API void UnregisterAudioBusFromSubmix(const UObject* WorldContextObject, USoundSubmix* SoundSubmix, UAudioBus* AudioBus);

	/**
	* Gets information about all audio output devices available in the system
	* @param OnObtainDevicesEvent - the event to fire when the audio endpoint devices have been retrieved
	*/
	UFUNCTION(BlueprintCallable, Category = "Audio", meta = (WorldContext = "WorldContextObject"))
	static AUDIOMIXER_API void GetAvailableAudioOutputDevices(const UObject* WorldContextObject, const FOnAudioOutputDevicesObtained& OnObtainDevicesEvent);

	/**
	* Gets information about the currently used audio output device
	* @param OnObtainCurrentDeviceEvent - the event to fire when the audio endpoint devices have been retrieved
	*/
	UFUNCTION(BlueprintCallable, Category = "Audio", meta = (WorldContext = "WorldContextObject"))
	static AUDIOMIXER_API void GetCurrentAudioOutputDeviceName(const UObject* WorldContextObject, const FOnMainAudioOutputDeviceObtained& OnObtainCurrentDeviceEvent);

	/**
	* Hotswaps to the requested audio output device
	* @param NewDeviceId - the device Id to swap to
	* @param OnCompletedDeviceSwap - the event to fire when the audio endpoint devices have been retrieved
	*/
	UFUNCTION(BlueprintCallable, Category = "Audio", meta = (WorldContext = "WorldContextObject"))
	static AUDIOMIXER_API void SwapAudioOutputDevice(const UObject* WorldContextObject, const FString& NewDeviceId, const FOnCompletedDeviceSwap& OnCompletedDeviceSwap);
};

