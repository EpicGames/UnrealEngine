// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Audio.h"
#include "AudioBusSubsystem.h"
#include "AudioMixerTrace.h"
#include "DSP/InterpolatedOnePole.h"
#include "DSP/MultithreadedPatching.h"
#include "IAudioMixerRenderStep.h"
#include "Sound/AudioSettings.h"

#if UE_AUDIO_PROFILERTRACE_ENABLED
#include "DSP/EnvelopeFollower.h"
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

namespace Audio
{
	class FMixerSourceManager;

	// Consolidated filter settings for a single bus/submix send.
	// Passed between the game thread and render thread as a simple value type.
	struct FSendFilterSettings
	{
		uint8 bEnableLPF : 1 = 0;
		uint8 bEnableHPF : 1 = 0;
		float LpfCutoff = 20000.0f;
		float HpfCutoff = 20.0f;

		FSendFilterSettings() = default;
		FSendFilterSettings(bool bInEnableLPF, bool bInEnableHPF, float InLpfCutoff, float InHpfCutoff)
			: LpfCutoff(InLpfCutoff), HpfCutoff(InHpfCutoff)
		{
			bEnableLPF = bInEnableLPF;
			bEnableHPF = bInEnableHPF;
		}

		bool IsFiltering() const { return bEnableLPF || bEnableHPF; }

		// Returns true if any filter parameter differs significantly from Other.
		// Use to determine whether a render-thread update is needed.
		bool HasChanged(const FSendFilterSettings& Other) const
		{
			return bEnableLPF != Other.bEnableLPF
				|| bEnableHPF != Other.bEnableHPF
				|| (bEnableLPF && !FMath::IsNearlyEqual(LpfCutoff, Other.LpfCutoff))
				|| (bEnableHPF && !FMath::IsNearlyEqual(HpfCutoff, Other.HpfCutoff));
		}
	};

	// Struct holding mappings of runtime source ids (bus instances) to bus send level
	struct FAudioBusSend
	{
		int32 SourceId = INDEX_NONE;
		uint32 WaveInstancePlayOrder = INDEX_NONE;
		float SendLevel = 0.0f;

		FSendFilterSettings Filter;
		FInterpolatedLPF Lpf;
		FInterpolatedHPF Hpf;
	};

	// Bus instance data. Holds source id bus instances and bus sends data
	class FMixerAudioBus : public IAudioMixerRenderStep
	{
	public:
		// Creates an audio bus.
		// SourceManager		The owning source manager object.
		// bInIsAutomatic	Whether or not this audio bus was created automatically via source buses.
		// InNumChannels	The number of channels of the source bus.
		// InBusKey         The unique key used to identify this bus.
		FMixerAudioBus(FMixerSourceManager* SourceManager, bool bInIsAutomatic, int32 InNumChannels, const FAudioBusKey InBusKey);

		~FMixerAudioBus();

		// Allow anybody to add a pre-existing patch output object to the audio bus
		void AddNewPatchOutput(const FPatchOutputStrongPtr& InPatchOutputStrongPtr);

		// Allow anybody to write audio into this audio bus from any thread.
		void AddNewPatchInput(const FPatchInput& InPatchInput);

		// Allow anybody to write audio into this audio bus from any thread.
		void RemovePatchInput(const FPatchInput& InPatchInput);

	private:
	
		// Sets whether or not this audio bus is automatic.
		void SetAutomatic(bool bInIsAutomatic) { bIsAutomatic = bInIsAutomatic; }

		// Returns if this is a manual audio bus vs automatic
		bool IsAutomatic() const { return bIsAutomatic; }

		// Returns the number of channels of the audio bus
		int32 GetNumChannels() const { return NumChannels; }

		// Update the mixer bus after a render block
		void Update();

		// Adds a source id for instances of this bus
		void AddInstanceId(const int32 InSourceInstanceId, const uint32 WaveInstancePlayOrder, int32 InNumOutputChannels);

		// Removes the source id from this bus. Returns true if there are no more instances or sends.
		bool RemoveInstanceId(const int32 InSourceId, const uint32 WaveInstancePlayOrder);

		// Adds a bus send to the bus
		void AddSend(EBusSendType BusSendType, const FAudioBusSend& InBusSend);

		// Removes the source instance from this bus's send list
		bool RemoveSend(EBusSendType BusSendType, const int32 InSourceId);

		// Gets the current mixed bus buffer
		const float* GetCurrentBusBuffer() const;

		// Gets the previous mixed bus buffer
		const float* GetPreviousBusBuffer() const;

		// Compute the mixed buffer
		void MixBuffer();

		// Copies the current internal buffer to a provided output buffer. Only supports mono or stereo input/output formats.
		void CopyCurrentBuffer(Audio::FAlignedFloatBuffer& InChannelMap, int32 InNumOutputChannels, FAlignedFloatBuffer& OutBuffer, int32 NumOutputFrames) const;
		void CopyCurrentBuffer(int32 InNumOutputChannels, FAlignedFloatBuffer& OutBuffer, int32 NumOutputFrames) const;

		// If this bus was constructed before
		void SetNumOutputChannels(int32 InNumOutputChannels);

		// IAudioMixerRenderStep interface
		virtual void DoRenderStep() override;
		virtual const TCHAR* GetRenderStepName() override;

#if UE_AUDIO_PROFILERTRACE_ENABLED
		bool StartEnvelopeFollower(const float InAttackTime, const float InReleaseTime, const float InSampleRate);
		bool StopEnvelopeFollower();
		void ProcessEnvelopeFollower(const float* InBuffer);
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

		// Array of instance ids. These are sources which are instances of this.
		// It's possible for this data to have bus sends but no instance ids.
		// This means a source would send its audio to the bus if the bus had an instance.
		// Once and instance plays, it will then start sending its audio to the bus instances.
		TArray<int32> InstanceIds;

		// Bus sends to this instance
		TArray<FAudioBusSend> AudioBusSends[(int32)EBusSendType::Count];

		// The mixed source data. This is double-buffered to allow buses to send audio to themselves.
		// Buses feed audio to each other by storing their previous buffer. Current buses mix in previous other buses (including themselves)
		FAlignedFloatBuffer MixedSourceData[2];

		// Scratch buffer used for intermediate filter output in MixBuffer(). Kept as a member so its
		// heap allocation persists across render callbacks and avoids per-frame reallocation.
		FAlignedFloatBuffer FilterScratchBuffer;

		// The index of the bus data currently being rendered
		int32 CurrentBufferIndex;

		// The number of channels of this bus
		int32 NumChannels;

		// The number of output frames
		int32 NumFrames;

		// Owning source manager
		FMixerSourceManager* SourceManager;

		// Multiple places can produce and consume from audio buses
		Audio::FPatchMixer PatchMixer;
		Audio::FPatchSplitter PatchSplitter;

		// Unique bus identifier
		FAudioBusKey BusKey;

		// Was created manually, not via source buses.
		bool bIsAutomatic;

#if ENABLE_AUDIO_DEBUG
		FString BusName;
#endif // if ENABLE_AUDIO_DEBUG

#if UE_AUDIO_PROFILERTRACE_ENABLED
		Audio::FEnvelopeFollower EnvelopeFollower;
		float EnvelopeValues[AUDIO_MIXER_MAX_OUTPUT_CHANNELS];
		int32 EnvelopeNumChannels = 0;
		int32 EnvelopeFollowerRequestsRefCount = 0;
		bool bIsEnvelopeFollowing = false;
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

		friend FMixerSourceManager;
		friend FMixerSubmix;
	};

}
