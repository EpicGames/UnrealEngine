// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Audio.h"
#include "AudioMixer.h"
#include "AudioDevice.h"
#include "Containers/MpscQueue.h"
#include "Containers/SpscQueue.h"
#include "Sound/SoundSubmix.h"
#include "Sound/SoundGenerator.h"
#include "DSP/BufferVectorOperations.h"
#include "DSP/MultithreadedPatching.h"
#include "Quartz/AudioMixerClockManager.h"
#include "UObject/GCObject.h"
#include "UObject/StrongObjectPtr.h"

#if UE_AUDIO_PROFILERTRACE_ENABLED
#include "DSP/EnvelopeFollower.h"
#include "DSP/LKFSAnalyzer.h"
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

// Forward Declarations
class FOnSubmixEnvelopeBP;
class IAudioMixerPlatformInterface;
class USoundModulatorBase;
class IAudioLinkFactory;

#include "AudioMixerDevice.generated.h"

UENUM()
enum class ERequiredSubmixes : uint8
{
	Main = 0,
	BaseDefault = 1,
	Reverb = 2,
	EQ = 3,
	Count = 4 UMETA(Hidden)
};


namespace Audio
{
	AUDIOMIXER_API extern int32 GRemoveSubmixesFromRoot;
	
	// Audio Namespace Forward Declarations
	class FMixerSourceManager;
	class FMixerSourceVoice;
	class FMixerSubmix;
	class FAudioFormatSettings;
	class FAudioRenderScheduler;
	class IAudioMixerGeneratorSource;

	typedef TSharedPtr<FMixerSubmix, ESPMode::ThreadSafe> FMixerSubmixPtr;
	typedef TWeakPtr<FMixerSubmix, ESPMode::ThreadSafe> FMixerSubmixWeakPtr;

	/** Data used to schedule events automatically in the audio renderer in audio mixer. */
	struct FAudioThreadTimingData
	{
		/** The time since audio device started. */
		double StartTime;

		/** The clock of the audio thread, periodically synced to the audio render thread time. */
		double AudioThreadTime;

		/** The clock of the audio render thread. */
		double AudioRenderThreadTime;

		/** The current audio thread fraction for audio events relative to the render thread. */
		double AudioThreadTimeJitterDelta;

		FAudioThreadTimingData()
			: StartTime(0.0)
			, AudioThreadTime(0.0)
			, AudioRenderThreadTime(0.0)
			, AudioThreadTimeJitterDelta(0.05)
		{}
	};	

	/** Data used to interpolate the audio clock in between buffer callbacks */
	struct FAudioClockTimingData
	{
		/** Time in secods of previous audio clock update */
		double UpdateTime = 0.0;

		/** Interpolates the given clock based on the amount of platform time that has passed since last update */
		double GetInterpolatedAudioClock(const double InAudioClock, const double InAudioClockDelta) const;
	};

	// Deprecated, use ERequiredSubmixes above
	namespace EMasterSubmixType
	{
		enum Type
		{
			Master = static_cast<uint8>(ERequiredSubmixes::Main),
			BaseDefault = static_cast<uint8>(ERequiredSubmixes::BaseDefault),
			Reverb = static_cast<uint8>(ERequiredSubmixes::Reverb),
			EQ = static_cast<uint8>(ERequiredSubmixes::EQ),
			Count = static_cast<uint8>(ERequiredSubmixes::Count)
		};
	}

	struct FSubmixMap
	{
	public:
		using FObjectId = uint32;
		using FPair = TPair<FObjectId, FMixerSubmixPtr>;
		using FIterFunc = TUniqueFunction<void(const FPair&)>;

		void Add(const FObjectId InObjectId, FMixerSubmixPtr InMixerSubmix);
		void Iterate(FIterFunc InFunction);
		FMixerSubmixPtr FindRef(FObjectId InObjectId) const;
		int32 Remove(const FObjectId InObjectId);
		void Reset();
		TSet<FSubmixMap::FObjectId> GetKeys() const;
	private:
		TMap<FObjectId, FMixerSubmixPtr> SubmixMap;

		mutable FCriticalSection MutationLock;
	};


	class FMixerDevice :	public FAudioDevice,
							public IAudioMixer,
							public FGCObject
	{
	public:
		AUDIOMIXER_API FMixerDevice(IAudioMixerPlatformInterface* InAudioMixerPlatform);
		AUDIOMIXER_API ~FMixerDevice();

		//~ Begin FAudioDevice
		AUDIOMIXER_API virtual void UpdateDeviceDeltaTime() override;
		AUDIOMIXER_API virtual void GetAudioDeviceList(TArray<FString>& OutAudioDeviceNames) const override;
		AUDIOMIXER_API virtual bool InitializeHardware() override;
		AUDIOMIXER_API virtual void FadeIn() override;
		AUDIOMIXER_API virtual void FadeOut() override;
		AUDIOMIXER_API virtual void TeardownHardware() override;
		AUDIOMIXER_API virtual void UpdateHardwareTiming() override;
		AUDIOMIXER_API virtual void UpdateGameThread() override;
		AUDIOMIXER_API virtual void UpdateHardware() override;
		AUDIOMIXER_API virtual double GetAudioTime() const override;
		AUDIOMIXER_API virtual double GetInterpolatedAudioClock() const override;
		AUDIOMIXER_API virtual FAudioEffectsManager* CreateEffectsManager() override;
		AUDIOMIXER_API virtual FSoundSource* CreateSoundSource() override;
		AUDIOMIXER_API virtual bool HasCompressedAudioInfoClass(USoundWave* SoundWave) override;
		AUDIOMIXER_API virtual bool ValidateAPICall(const TCHAR* Function, uint32 ErrorCode) override;
#if UE_ALLOW_EXEC_COMMANDS
		AUDIOMIXER_API virtual bool Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;
#endif
		AUDIOMIXER_API virtual void CountBytes(class FArchive& Ar) override;
		AUDIOMIXER_API virtual bool IsExernalBackgroundSoundActive() override;
		AUDIOMIXER_API virtual void ResumeContext() override;
		AUDIOMIXER_API virtual void SuspendContext() override;
		AUDIOMIXER_API virtual void EnableDebugAudioOutput() override;
		AUDIOMIXER_API virtual FAudioPlatformSettings GetPlatformSettings() const override;
		AUDIOMIXER_API virtual void RegisterSoundSubmix(USoundSubmixBase* SoundSubmix, bool bInit = true) override;
		AUDIOMIXER_API virtual void UnregisterSoundSubmix(const USoundSubmixBase* SoundSubmix, const bool bReparentChildren) override;

		AUDIOMIXER_API virtual int32 GetNumActiveSources() const override;

		// Updates the source effect chain (using unique object id). 
		AUDIOMIXER_API virtual void UpdateSourceEffectChain(const uint32 SourceEffectChainId, const TArray<FSourceEffectChainEntry>& SourceEffectChain, const bool bPlayEffectChainTails) override;
		AUDIOMIXER_API virtual bool GetCurrentSourceEffectChain(const uint32 SourceEffectChainId, TArray<FSourceEffectChainEntry>& OutCurrentSourceEffectChainEntries) override;

		// Submix dry/wet settings
		AUDIOMIXER_API virtual void UpdateSubmixProperties(USoundSubmixBase* InSubmix) override;
		AUDIOMIXER_API virtual void SetSubmixWetDryLevel(USoundSubmix* InSoundSubmix, float InOutputVolume, float InWetLevel, float InDryLevel) override;
		AUDIOMIXER_API virtual void SetSubmixOutputVolume(USoundSubmix* InSoundSubmix, float InOutputVolume) override;
		AUDIOMIXER_API virtual void SetSubmixWetLevel(USoundSubmix* InSoundSubmix, float InWetLevel) override;
		AUDIOMIXER_API virtual void SetSubmixDryLevel(USoundSubmix* InSoundSubmix, float InDryLevel) override;

		// Submix auto-disable setteings
		AUDIOMIXER_API virtual void SetSubmixAutoDisable(USoundSubmix* InSoundSubmix, bool bInAutoDisable) override;
		AUDIOMIXER_API virtual void SetSubmixAutoDisableTime(USoundSubmix* InSoundSubmix, float InDisableTime) override;

		// Submix Modulation Settings
		AUDIOMIXER_API virtual void UpdateSubmixModulationSettings(USoundSubmix* InSoundSubmix, const TSet<TObjectPtr<USoundModulatorBase>>& InOutputModulation, const TSet<TObjectPtr<USoundModulatorBase>>& InWetLevelModulation, const TSet<TObjectPtr<USoundModulatorBase>>& InDryLevelModulation) override;
		AUDIOMIXER_API virtual void SetSubmixModulationBaseLevels(USoundSubmix* InSoundSubmix, float InVolumeModBase, float InWetModBase, float InDryModBase) override;

		// Submix effect chain override settings
		AUDIOMIXER_API virtual void SetSubmixEffectChainOverride(USoundSubmix* InSoundSubmix, const TArray<FSoundEffectSubmixPtr>& InSubmixEffectPresetChain, float InFadeTimeSec) override;
		AUDIOMIXER_API virtual void ClearSubmixEffectChainOverride(USoundSubmix* InSoundSubmix, float InFadeTimeSec) override;

		// Submix recording callbacks:
		AUDIOMIXER_API virtual void StartRecording(USoundSubmix* InSubmix, float ExpectedRecordingDuration) override;
		AUDIOMIXER_API virtual Audio::FAlignedFloatBuffer& StopRecording(USoundSubmix* InSubmix, float& OutNumChannels, float& OutSampleRate) override;

		AUDIOMIXER_API virtual void PauseRecording(USoundSubmix* InSubmix);
		AUDIOMIXER_API virtual void ResumeRecording(USoundSubmix* InSubmix);

		// Submix envelope following
		AUDIOMIXER_API virtual void StartEnvelopeFollowing(USoundSubmix* InSubmix) override;
		AUDIOMIXER_API virtual void StopEnvelopeFollowing(USoundSubmix* InSubmix) override;
		AUDIOMIXER_API virtual void AddEnvelopeFollowerDelegate(USoundSubmix* InSubmix, const FOnSubmixEnvelopeBP& OnSubmixEnvelopeBP) override;
		AUDIOMIXER_API virtual void RemoveEnvelopeFollowerDelegate(USoundSubmix* InSubmix, const FOnSubmixEnvelopeBP& OnSubmixEnvelopeBP) override;

		// Submix Spectrum Analysis
		AUDIOMIXER_API virtual void StartSpectrumAnalysis(USoundSubmix* InSubmix, const FSoundSpectrumAnalyzerSettings& InSettings) override;
		AUDIOMIXER_API virtual void StopSpectrumAnalysis(USoundSubmix* InSubmix) override;
		AUDIOMIXER_API virtual void GetMagnitudesForFrequencies(USoundSubmix* InSubmix, const TArray<float>& InFrequencies, TArray<float>& OutMagnitudes) override;
		AUDIOMIXER_API virtual void GetPhasesForFrequencies(USoundSubmix* InSubmix, const TArray<float>& InFrequencies, TArray<float>& OutPhases) override;
		AUDIOMIXER_API virtual void AddSpectralAnalysisDelegate(USoundSubmix* InSubmix, const FSoundSpectrumAnalyzerDelegateSettings& InDelegateSettings, const FOnSubmixSpectralAnalysisBP& OnSubmixSpectralAnalysisBP) override;
		AUDIOMIXER_API virtual void RemoveSpectralAnalysisDelegate(USoundSubmix* InSubmix, const FOnSubmixSpectralAnalysisBP& OnSubmixSpectralAnalysisBP) override;

		// Submix buffer listener callbacks
		AUDIOMIXER_API virtual void RegisterSubmixBufferListener(TSharedRef<ISubmixBufferListener, ESPMode::ThreadSafe> InSubmixBufferListener, USoundSubmix& InSubmix) override;

		// This is optional, and should only be used to manually unregister before the ISubmixBufferListener is getting destroyed
		// (do not call this in the destructor of ISubmixBufferListener or derived classes)
		AUDIOMIXER_API virtual void UnregisterSubmixBufferListener(TSharedRef<ISubmixBufferListener, ESPMode::ThreadSafe> InSubmixBufferListener, USoundSubmix& InSubmix) override;

		AUDIOMIXER_API virtual FPatchOutputStrongPtr AddPatchForSubmix(uint32 InObjectId, float InPatchGain) override;

		AUDIOMIXER_API virtual void FlushExtended(UWorld* WorldToFlush, bool bClearActivatedReverb);
		AUDIOMIXER_API virtual void FlushAudioRenderingCommands(bool bPumpSynchronously = false, bool bRenderThreadStopped = false) override;

		// Audio Device Properties
		AUDIOMIXER_API virtual bool IsNonRealtime() const override;

		//~ End FAudioDevice

		//~ Begin IAudioMixer
		AUDIOMIXER_API virtual bool OnProcessAudioStream(FAlignedFloatBuffer& OutputBuffer) override;
		AUDIOMIXER_API virtual void OnAudioStreamShutdown() override;
		//~ End IAudioMixer

		//~ Begin FGCObject
		AUDIOMIXER_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
		virtual FString GetReferencerName() const override
		{
			return TEXT("Audio::FMixerDevice");
		}
		//~End FGCObject

		AUDIOMIXER_API FMixerSubmixPtr FindSubmixInstanceByObjectId(uint32 InObjectId);

		AUDIOMIXER_API FMixerSubmixWeakPtr GetSubmixInstance(const USoundSubmixBase* SoundSubmix) const;

		// If SoundSubmix is a soundfield submix, this will return the factory used to encode 
		// source audio to it's soundfield format.
		// Otherwise, returns nullptr.
		AUDIOMIXER_API ISoundfieldFactory* GetFactoryForSubmixInstance(USoundSubmix* SoundSubmix);
		AUDIOMIXER_API ISoundfieldFactory* GetFactoryForSubmixInstance(FMixerSubmixWeakPtr& SoundSubmixPtr);

		// Functions which check the thread it's called on and helps make sure functions are called from correct threads
		AUDIOMIXER_API void CheckAudioThread() const;
		AUDIOMIXER_API void CheckAudioRenderingThread() const;
		AUDIOMIXER_API bool IsAudioRenderingThread() const;

		// Public Functions
		AUDIOMIXER_API FMixerSourceVoice* GetMixerSourceVoice();
		AUDIOMIXER_API void ReleaseMixerSourceVoice(FMixerSourceVoice* InSourceVoice);
		AUDIOMIXER_API int32 GetNumSources() const;

		// AudioLink
		AUDIOMIXER_API IAudioLinkFactory* GetAudioLinkFactory() const;

		const FAudioPlatformDeviceInfo& GetPlatformDeviceInfo() const { return PlatformInfo; };

		inline int32 GetNumDeviceChannels() const { return PlatformInfo.NumChannels; }
		inline int32 GetNumDirectOutChannels() const { return PlatformInfo.NumDirectOutChannels; }

		int32 GetNumOutputFrames() const { return PlatformSettings.CallbackBufferFrameSize; }

		int32 GetNumOutputBuffers() const { return PlatformSettings.NumBuffers; }

		// Retrieve a pointer to the currently active platform. Only use this if you know what you are doing. The returned IAudioMixerPlatformInterface will only be alive as long as this FMixerDevice is alive.
		IAudioMixerPlatformInterface* GetAudioMixerPlatform() const { return AudioMixerPlatform; }

		// Builds a 3D channel map for a spatialized source.
		AUDIOMIXER_API void Get3DChannelMap(const int32 InSubmixNumChannels, const FWaveInstance* InWaveInstance, const float EmitterAzimuth, const float NonSpatiliazedFactor, const TMap<EAudioMixerChannel::Type, float>* InOmniMap, float InDefaultOmniValue, Audio::FAlignedFloatBuffer& OutChannelMap);

		// Builds a channel gain matrix for a non-spatialized source. The non-static variation of this function queries AudioMixerDevice->NumOutputChannels directly which may not be thread safe.
		AUDIOMIXER_API void Get2DChannelMap(bool bIsVorbis, const int32 NumSourceChannels, const bool bIsCenterChannelOnly, Audio::FAlignedFloatBuffer& OutChannelMap) const;
		AUDIOMIXER_API static void Get2DChannelMap(bool bIsVorbis, const int32 NumSourceChannels, const int32 NumOutputChannels, const bool bIsCenterChannelOnly, Audio::FAlignedFloatBuffer& OutChannelMap);

		AUDIOMIXER_API int32 GetDeviceSampleRate() const;
		AUDIOMIXER_API int32 GetDeviceOutputChannels() const;

		AUDIOMIXER_API FMixerSourceManager* GetSourceManager();
		AUDIOMIXER_API const FMixerSourceManager* GetSourceManager() const;

		AUDIOMIXER_API virtual USoundSubmix& GetMainSubmixObject() const override;

		AUDIOMIXER_API FMixerSubmixWeakPtr GetBaseDefaultSubmix();
		AUDIOMIXER_API FMixerSubmixWeakPtr GetMainSubmix();
		AUDIOMIXER_API FMixerSubmixWeakPtr GetReverbSubmix();
		AUDIOMIXER_API FMixerSubmixWeakPtr GetEQSubmix();

		// Renamed Main submix: these functions will be deprecated in a future release
		AUDIOMIXER_API void AddMasterSubmixEffect(FSoundEffectSubmixPtr SoundEffect);
		AUDIOMIXER_API void RemoveMasterSubmixEffect(uint32 SubmixEffectId);
		AUDIOMIXER_API void ClearMasterSubmixEffects();
		AUDIOMIXER_API FMixerSubmixWeakPtr GetMasterSubmix();
		AUDIOMIXER_API FMixerSubmixWeakPtr GetMasterReverbSubmix();
		AUDIOMIXER_API FMixerSubmixWeakPtr GetMasterEQSubmix();

		// Add submix effect to main submix
		AUDIOMIXER_API void AddMainSubmixEffect(FSoundEffectSubmixPtr SoundEffect);

		// Remove submix effect from main submix
		AUDIOMIXER_API void RemoveMainSubmixEffect(uint32 SubmixEffectId);

		// Clear all submix effects from main submix
		AUDIOMIXER_API void ClearMainSubmixEffects();

		// Add submix effect to given submix
		AUDIOMIXER_API int32 AddSubmixEffect(USoundSubmix* InSoundSubmix, FSoundEffectSubmixPtr SoundEffect);

		// Remove submix effect to given submix
		AUDIOMIXER_API void RemoveSubmixEffect(USoundSubmix* InSoundSubmix, uint32 SubmixEffectId);

		// Remove submix effect at the given submix chain index
		AUDIOMIXER_API void RemoveSubmixEffectAtIndex(USoundSubmix* InSoundSubmix, int32 SubmixChainIndex);

		// Replace the submix effect of the given submix at the submix chain index with the new submix effect id and submix instance
		AUDIOMIXER_API void ReplaceSoundEffectSubmix(USoundSubmix* InSoundSubmix, int32 InSubmixChainIndex, FSoundEffectSubmixPtr SoundEffect);

		// Clear all submix effects from given submix
		AUDIOMIXER_API void ClearSubmixEffects(USoundSubmix* InSoundSubmix);

		// Returns the channel array for the given submix channel type
		AUDIOMIXER_API const TArray<EAudioMixerChannel::Type>& GetChannelArray() const;

		// Retrieves the listener transforms
		AUDIOMIXER_API const TArray<FTransform>* GetListenerTransforms();

		// Retrieves spherical locations of channels for a given submix format
		AUDIOMIXER_API const FChannelPositionInfo* GetDefaultChannelPositions() const;

		// Audio thread tick timing relative to audio render thread timing
		double GetAudioThreadTime() const { return AudioThreadTimingData.AudioThreadTime; }
		double GetAudioRenderThreadTime() const { return AudioThreadTimingData.AudioRenderThreadTime; }
		double GetAudioClockDelta() const { return AudioClockDelta; }

		EMonoChannelUpmixMethod GetMonoChannelUpmixMethod() const { return MonoChannelUpmixMethod; }

		AUDIOMIXER_API TArray<Audio::FChannelPositionInfo>* GetDefaultPositionMap(int32 NumChannels);

		static AUDIOMIXER_API bool IsEndpointSubmix(const USoundSubmixBase* InSubmix);

		AUDIOMIXER_API FPatchOutputStrongPtr MakePatch(int32 InFrames, int32 InChannels, float InGain) const;

		// Clock Manager for quantized event handling on Audio Render Thread
		FQuartzClockManager QuantizedEventClockManager;

		// Keep a reference alive to UQuartzSubsystem state that needs to persist across level transitions (UWorld destruction
		TSharedPtr<FPersistentQuartzSubsystemData, ESPMode::ThreadSafe> QuartzSubsystemData { nullptr };

		// Technically, in editor, multiple UQuartz(World)Subsystem's will reference the same FMixerDevice object.
		// We need to protect around mutation/access of the "shared" state.
		// (in practice this should be low/zero contention)
		FCriticalSection QuartzPersistentStateCritSec;

		// Pushes the command to a audio render thread command queue to be executed on render thread
		AUDIOMIXER_API void AudioRenderThreadCommand(TFunction<void()> Command);

		// Pushes the command to a MPSC queue to be executed on the game thread
		AUDIOMIXER_API void GameThreadMPSCCommand(TFunction<void()> InCommand);

		// Debug Commands
		AUDIOMIXER_API void DrawSubmixes(FOutputDevice& InOutput, const TArray<FString>& InArgs) const;

#if ENABLE_AUDIO_DEBUG
		void UpdateSubmixDebugVolumes();
#endif // ENABLE_AUDIO_DEBUG

		bool UseRenderScheduler() const
		{
			return bUseRenderScheduler;
		}

		FAudioRenderScheduler& GetRenderScheduler() const
		{
			return *RenderScheduler;
		}

		UE_EXPERIMENTAL(5.8, "AudioMixer Generator Source API is experimental and likely to change.")
		AUDIOMIXER_API TUniquePtr<IAudioMixerGeneratorSource> CreateGeneratorSource(ISoundGeneratorRef InGenerator);

	protected:
		AUDIOMIXER_API virtual void InitSoundSubmixes() override;

		AUDIOMIXER_API virtual void OnListenerUpdated(const TArray<FListener>& InListeners) override;

		TArray<FTransform> ListenerTransforms;

	private:
		// Resets the thread ID used for audio rendering
		void ResetAudioRenderingThreadId();

		void RebuildSubmixLinks(const USoundSubmixBase& SoundSubmix, FMixerSubmixPtr& SubmixInstance);

		void InitializeChannelMaps();
		static int32 GetChannelMapCacheId(const int32 NumSourceChannels, const int32 NumOutputChannels, const bool bIsCenterChannelOnly);
		void CacheChannelMap(const int32 NumSourceChannels, const int32 NumOutputChannels, const bool bIsCenterChannelOnly);
		void InitializeChannelAzimuthMap(const int32 NumChannels);

		void WhiteNoiseTest(FAlignedFloatBuffer& Output);
		void SineOscTest(FAlignedFloatBuffer& Output);

		bool IsMainAudioDevice() const;

		void LoadRequiredSubmix(ERequiredSubmixes InType, const FString& InDefaultName, bool bInDefaultMuteWhenBackgrounded, FSoftObjectPath& InOutObjectPath);
		void LoadPluginSoundSubmixes();
		void LoadSoundSubmix(USoundSubmixBase& SoundSubmix);

		void InitSoundfieldAndEndpointDataForSubmix(const USoundSubmixBase& InSoundSubmix, FMixerSubmixPtr MixerSubmix, bool bAllowReInit);

		void UnloadSoundSubmix(const USoundSubmixBase& SoundSubmix, const bool bReparentChildren);

		bool IsRequiredSubmixType(const USoundSubmixBase* InSubmix) const;
		FMixerSubmixPtr GetRequiredSubmixInstance(uint32 InSubmixId) const;
		FMixerSubmixPtr GetRequiredSubmixInstance(const USoundSubmixBase* InSubmix) const;
		
		// Pumps the audio render thread command queue
		void PumpCommandQueue();
		void PumpGameThreadCommandQueue();

		/** Updates the audio clock and the associated timing data */
		void UpdateAudioClock();

#if UE_AUDIO_PROFILERTRACE_ENABLED
		void OnTraceStarted(FTraceAuxiliary::EConnectionType TraceType, const FString& TraceDestination);
		void OnTraceStopped(FTraceAuxiliary::EConnectionType TraceType, const FString& TraceDestination);

		void InitializeMainSubmixTruePeakEnvelopeFollower(int32 InSampleRate, int32 InNumChannels, int32 InNumSamples);
		void InitializeMainSubmixLKFSAnalyzer(int32 InSampleRate, int32 InNumChannels);

		void SendMainSubmixLoadedMessage();
		void SendMainSubmixAlivePingMessage();

		void UpdateAndSendMainSubmixTruePeakMaxValueMessage(const FAlignedFloatBuffer& InMainSubmixAudioBuffer);
		void UpdateAndSendMainSubmixLKFSValuesMessage(const FAlignedFloatBuffer& InMainSubmixAudioBuffer);
	
		void SendSubmixesAlivePing();
		void UpdateSubmixesAlivePingTime();
		void SendSubmixesSendLevelTraceMessages();

		void RegisterSubmixEnvelopeFollowerCVars();
		void UnregisterSubmixEnvelopeFollowerCVars();

		void OnStartSubmixEnvelopeFollowerCVarChanged(IConsoleVariable* InStartSubmixEnvelopeFollowerCVar);
		void OnStopSubmixEnvelopeFollowerCVarChanged(IConsoleVariable* InStopSubmixEnvelopeFollowerCVar);

		void StartSubmixEnvelopeFollowing(const FMixerSubmixPtr& InSubmix);
		void StopSubmixEnvelopeFollowing(const FMixerSubmixPtr& InSubmix);

		void AutomaticallyStartSubmixEnvelopeFollowers();

		void RegisterResetMainSubmixAnalyzersCmd();
		void UnregisterResetMainSubmixAnalyzersCmd();

		void OnResetMainSubmixAnalyzersCmdInvoked();

		void ResetMainSubmixAnalyzers();
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

		TArray<USoundSubmix*> RequiredSubmixes;
		TArray<FMixerSubmixPtr> RequiredSubmixInstances;

		TArray<TStrongObjectPtr<UAudioBus>> DefaultAudioBuses;
		/** Ptr to the platform interface, which handles streaming audio to the hardware device. */
		IAudioMixerPlatformInterface* AudioMixerPlatform;
		
		/** Contains a map of channel/speaker azimuth positions. */
		FChannelPositionInfo DefaultChannelAzimuthPositions[EAudioMixerChannel::MaxSupportedChannel];

		/** The azimuth positions for submix channel types. */
		TArray<FChannelPositionInfo> DeviceChannelAzimuthPositions;

		int32 DeviceOutputChannels;

		/** What upmix method to use for mono channel upmixing. */
		EMonoChannelUpmixMethod MonoChannelUpmixMethod;

		/** What panning method to use for panning. */
		EPanningMethod PanningMethod;

		/** The audio output stream parameters used to initialize the audio hardware. */
		FAudioMixerOpenStreamParams OpenStreamParams;

		/** The time delta for each callback block. */
		double AudioClockDelta;

		/** The timing data used to interpolate the audio clock */
		FAudioClockTimingData AudioClockTimingData;

		/** What the previous master volume was. */
		float PreviousPrimaryVolume;

		/** Timing data for audio thread. */
		FAudioThreadTimingData AudioThreadTimingData;

		/** The platform device info for this mixer device. */
		FAudioPlatformDeviceInfo PlatformInfo;

		/** Map of USoundSubmix static data objects to the dynamic audio mixer submix. */
		FSubmixMap Submixes;

		// Submixes that will sum their audio and send it directly to AudioMixerPlatform.
		// Submixes are added to this list in RegisterSoundSubmix, and removed in UnregisterSoundSubmix.
		TArray<FMixerSubmixPtr> DefaultEndpointSubmixes;

		// Submixes that need to be processed, but will be sending their audio to external sends.
		// Submixes are added to this list in RegisterSoundSubmix and removed in UnregisterSoundSubmix.
		TArray<FMixerSubmixPtr> ExternalEndpointSubmixes;

		// Contended between RegisterSoundSubmix/UnregisterSoundSubmix on the audio thread and OnProcessAudioStream on the audio mixer thread.
		FCriticalSection EndpointSubmixesMutationLock;

		/** Submix Buffer Listeners which are trying to listen to submixes which have not been registered yet */
		TMap<FString, TMap<FString, TSharedRef<ISubmixBufferListener, ESPMode::ThreadSafe>>> PendingSubmixBufferListeners;
		
		/** Which submixes have been told to envelope follow with this audio device. */
		TArray<USoundSubmix*> DelegateBoundSubmixes;

		/** Queue of mixer source voices. */
		TSpscQueue<FMixerSourceVoice*> SourceVoices;

		TMap<uint32, TArray<FSourceEffectChainEntry>> SourceEffectChainOverrides;

		/** Scheduler responsible for launching rendering tasks */
		TUniquePtr<FAudioRenderScheduler> RenderScheduler;
		bool bUseRenderScheduler = true;

		/** The mixer source manager. Should be cleaned up before the scheduler so dependencies can be removed correctly. */
		TUniquePtr<FMixerSourceManager> SourceManager;

		/** ThreadId for the game thread (or if audio is running a separate thread, that ID) */
		mutable int32 GameOrAudioThreadId;

		/** ThreadId for the low-level platform audio mixer. */
		mutable std::atomic<int32> AudioPlatformThreadId;

		/** Command queue to send commands to audio render thread from game thread or audio thread. */
		TMpscQueue<TFunction<void()>> CommandQueue;

		/** MPSC command queue to send commands to the game thread */
		TMpscQueue<TFunction<void()>> GameThreadCommandQueue;

		IAudioLinkFactory* AudioLinkFactory = nullptr;

#if UE_AUDIO_PROFILERTRACE_ENABLED
		TSet<uint32> SubmixesWithAutoEnvelopeFollowers;
		TSet<FString> SubmixesWithCVarEnvelopeFollowers;

		IConsoleVariable* StartSubmixEnvelopeFollowerCVar = nullptr;
		IConsoleVariable* StopSubmixEnvelopeFollowerCVar  = nullptr;
		IConsoleObject*   ResetMainSubmixAnalyzersCmd     = nullptr;

		TUniquePtr<FEnvelopeFollower> MainSubmixTruePeakEnvelopeFollower;
		FAlignedFloatBuffer MainSubmixTruePeakOutputEnvelope;

		TUniquePtr<FLKFSAnalyzer> MainSubmixLKFSAnalyzer;

		double SubmixesAlivePingTimeSec = 0.0;
		static constexpr double SubmixesAlivePingWaitTimeSec = 1.0;
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

		/** Whether or not we generate output audio to test multi-platform mixer. */
		bool bDebugOutputEnabled;

		/** Whether or not initialization of the submix system is underway and submixes can be registered */
		bool bSubmixRegistrationDisabled;

	public:

		UE_DEPRECATED(5.8, "Synchronized audio task queues are deprecated.")
		AUDIOMIXER_API static void CreateSynchronizedAudioTaskQueue(AudioTaskQueueId QueueId);

		UE_DEPRECATED(5.8, "Synchronized audio task queues are deprecated.")
		AUDIOMIXER_API static void DestroySynchronizedAudioTaskQueue(AudioTaskQueueId QueueId, bool RunCurrentQueue = false);

		UE_DEPRECATED(5.8, "Synchronized audio task queues are deprecated.")
		AUDIOMIXER_API static int KickQueuedTasks(AudioTaskQueueId QueueId);
	};
}

