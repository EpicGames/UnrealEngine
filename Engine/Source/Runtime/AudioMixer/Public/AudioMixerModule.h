// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


/* Public dependencies
*****************************************************************************/

#include "CoreMinimal.h"


/* Public includes
*****************************************************************************/

#include "AudioDecompress.h"
#include "AudioEffect.h"


