// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <atomic>
#include "AudioBusSubsystem.h"
#include "AudioMixer.h"
#include "AudioMixerTrace.h"
#include "AudioDefines.h"
#include "CoreMinimal.h"
#include "SampleBuffer.h"
#include "IAudioEndpoint.h"
#include "ISoundfieldEndpoint.h"
#include "Sound/SoundSubmix.h"
#include "Sound/SoundModulationDestination.h"
#include "DSP/EnvelopeFollower.h"
#include "DSP/MultithreadedPatching.h"
#include "DSP/SpectrumAnalyzer.h"
#include "Templates/SharedPointer.h"
#include "AudioDynamicParameter.h"
#include "Stats/Stats.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "IAudioLinkFactory.h"

// The time it takes to process the submix graph. Process submix effects, mix into the submix buffer, etc.
DECLARE_CYCLE_STAT_EXTERN(TEXT("Submix Graph"), STAT_AudioMixerSubmixes, STATGROUP_AudioMixer, AUDIOMIXER_API);

// The time it takes to process the endpoint submixes.
DECLARE_CYCLE_STAT_EXTERN(TEXT("Submix Graph Endpoint"), STAT_AudioMixerEndpointSubmixes, STATGROUP_AudioMixer, AUDIOMIXER_API);

// The time it takes to process the submix graph. Process submix effects, mix into the submix buffer, etc.
DECLARE_CYCLE_STAT_EXTERN(TEXT("Submix Graph Child Processing"), STAT_AudioMixerSubmixChildren, STATGROUP_AudioMixer, AUDIOMIXER_API);

// The time it takes to process the submix graph. Process submix effects, mix into the submix buffer, etc.
DECLARE_CYCLE_STAT_EXTERN(TEXT("Submix Graph Source Mixing"), STAT_AudioMixerSubmixSource, STATGROUP_AudioMixer, AUDIOMIXER_API);

// The time it takes to process the submix graph. Process submix effects, mix into the submix buffer, etc.
DECLARE_CYCLE_STAT_EXTERN(TEXT("Submix Graph Effect Processing"), STAT_AudioMixerSubmixEffectProcessing, STATGROUP_AudioMixer, AUDIOMIXER_API);

// The time it takes to process the submix buffer listeners. 
DECLARE_CYCLE_STAT_WITH_FLAGS_EXTERN(TEXT("Submix Buffer Listeners"), STAT_AudioMixerSubmixBufferListeners, STATGROUP_AudioMixer, EStatFlags::Verbose, AUDIOMIXER_API);

// The time it takes to process the submix soundfield child submixes. 
DECLARE_CYCLE_STAT_EXTERN(TEXT("Submix Soundfield Children"), STAT_AudioMixerSubmixSoundfieldChildren, STATGROUP_AudioMixer, AUDIOMIXER_API);

// The time it takes to process the submix soundfield sources. 
DECLARE_CYCLE_STAT_EXTERN(TEXT("Submix Soundfield Sources"), STAT_AudioMixerSubmixSoundfieldSources, STATGROUP_AudioMixer, AUDIOMIXER_API);

// The time it takes to process the submix soundfield processors.. 
DECLARE_CYCLE_STAT_EXTERN(TEXT("Submix Soundfield Processors"), STAT_AudioMixerSubmixSoundfieldProcessors, STATGROUP_AudioMixer, AUDIOMIXER_API);

// Forward Declarations
class FOnSubmixEnvelopeBP;
class USoundEffectSubmix;
class USoundSubmix;
class USoundSubmixBase;
class USoundModulatorBase;

namespace Audio
{
	class IAudioMixerEffect;
	class FMixerSourceVoice;
	class FMixerDevice;

	enum EMixerSourceSubmixSendStage
	{
		// Whether to do the send pre distance attenuation
		PostDistanceAttenuation,

		// Whether to do the send post distance attenuation
		PreDistanceAttenuation,
	};

	struct FSubmixVoiceData
	{
		float SendLevel;
		EMixerSourceSubmixSendStage SubmixSendStage;

		FSubmixVoiceData()
			: SendLevel(1.0f)
			, SubmixSendStage(EMixerSourceSubmixSendStage::PostDistanceAttenuation)
		{
		}
	};

	class FMixerSubmix;

	struct FChildSubmixInfo
	{
		TWeakPtr<FMixerSubmix, ESPMode::ThreadSafe> SubmixPtr;

		// If the child submix is not a soundfield submix, we may need to encode its audio output in ProcessAudio.
		TUniquePtr<ISoundfieldEncoderStream> Encoder;

		// If this child submix is a soundfield submix that we can read the output of, we may need to transcode it's audio output.
		TUniquePtr<ISoundfieldTranscodeStream> Transcoder;

		// This is filled by either the Encoder or the Transcoder, and passed to this submix' mixer.
		TUniquePtr<ISoundfieldAudioPacket> IncomingPacketToTranscode;

		FChildSubmixInfo()
		{}

		FChildSubmixInfo(TWeakPtr<FMixerSubmix, ESPMode::ThreadSafe> SubmixWeakPtr)
			: SubmixPtr(SubmixWeakPtr)
		{
		}

		UE_NONCOPYABLE(FChildSubmixInfo);
	};

	class FMixerSubmix
	{
	public:
		AUDIOMIXER_API FMixerSubmix(FMixerDevice* InMixerDevice);
		AUDIOMIXER_API virtual ~FMixerSubmix();

		// Initialize the submix object with the USoundSubmix ptr. Sets up child and parent connects.
		AUDIOMIXER_API void Init(const USoundSubmixBase* InSoundSubmix, bool bAllowReInit = true);

		// Returns the mixer submix Id
		uint32 GetId() const { return Id; }

		// Return the owners name 
		const FString& GetName() const { return SubmixName; }

#if UE_AUDIO_PROFILERTRACE_ENABLED
		// Return the owners path name 
		const FString& GetPathName() const { return SubmixPathName; }
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

		// Sets the parent submix to the given submix
		AUDIOMIXER_API void SetParentSubmix(TWeakPtr<FMixerSubmix, ESPMode::ThreadSafe> Submix);

		// Adds the given submix to this submix's children
		AUDIOMIXER_API void AddChildSubmix(TWeakPtr<FMixerSubmix, ESPMode::ThreadSafe> Submix);

		// Removes the given submix from this submix's children
		AUDIOMIXER_API void RemoveChildSubmix(TWeakPtr<FMixerSubmix, ESPMode::ThreadSafe> SubmixWeakPtr);

		// Registers the given audiobus to this submix
		AUDIOMIXER_API void RegisterAudioBus(const Audio::FAudioBusKey& InAudioBusKey, Audio::FPatchInput&& InPatchInput);

		// Unregisters a registered audiobus from this submix (if any)
		AUDIOMIXER_API void UnregisterAudioBus(const Audio::FAudioBusKey& InAudioBusKey);

		// Updates the cached debug volume from the audio debugger (must be called on the AudioThread)
		void UpdateDebugVolume();

		// Returns the cached debug volume for this submix (0.0 if muted/soloed out, 1.0 otherwise)
		float GetDebugVolume() const { return DebugVolume.load(std::memory_order_relaxed); }

		// Sets the output level of the submix in linear gain
		AUDIOMIXER_API void SetOutputVolume(float InOutputLevel);

		// Sets the static output volume of the submix in linear gain
		AUDIOMIXER_API void SetDryLevel(float InDryLevel);

		// Sets the wet level of the submix in linear gain
		AUDIOMIXER_API void SetWetLevel(float InWetLevel);

		// Update modulation settings of the submix
		AUDIOMIXER_API void UpdateModulationSettings(const TSet<TObjectPtr<USoundModulatorBase>>& InOutputModulators, const TSet<TObjectPtr<USoundModulatorBase>>& InWetLevelModulators, const TSet<TObjectPtr<USoundModulatorBase>>& InDryLevelModulators);

		// Update modulation settings of the submix with Decibel values
		AUDIOMIXER_API void SetModulationBaseLevels(float InVolumeModBaseDb, float InWetModeBaseDb, float InDryModBaseDb);

		// Returns the cached output volume modulation value
		float GetOutputVolumeModulationValueThreadSafe() const;

		// Returns the cached wet level modulation value
		float GetWetVolumeModulationValueThreadSafe() const;

		// Gets the submix channels channels
		AUDIOMIXER_API int32 GetSubmixChannels() const;

		// Gets this submix's parent submix
		AUDIOMIXER_API TWeakPtr<FMixerSubmix, ESPMode::ThreadSafe> GetParentSubmix();

		// Returns the number of source voices currently a part of this submix.
		AUDIOMIXER_API int32 GetNumSourceVoices() const;

		// Returns the number of wet effects in this submix.
		AUDIOMIXER_API int32 GetNumEffects() const;

		// Returns the size of the submix chain. 
		AUDIOMIXER_API int32 GetSizeOfSubmixChain() const;

		UE_DEPRECATED(5.8, "Use FMixerSubmix::AddOrSetSourceById instead.")
		AUDIOMIXER_API void AddOrSetSourceVoice(FMixerSourceVoice* InSourceVoice, const float SendLevel, EMixerSourceSubmixSendStage InSubmixSendStage);

		// Add (if not already added) or sets the amount of the source voice's send amount
		void AddOrSetSourceById(const int32 InSourceId, const float SendLevel, EMixerSourceSubmixSendStage InSubmixSendStage);

		AUDIOMIXER_API FPatchOutputStrongPtr AddPatch(float InGain);

		UE_DEPRECATED(5.8, "Use FMixerSubmix::RemoveSourceById instead.")
		AUDIOMIXER_API void RemoveSourceVoice(FMixerSourceVoice* InSourceVoice);

		/** Removes the given source voice from the submix. */
		void RemoveSourceById(const int32 InSourceId);

		/** Appends the effect submix to the effect submix chain. */
		AUDIOMIXER_API void AddSoundEffectSubmix(FSoundEffectSubmixPtr InSoundEffectSubmix);

		/** Removes the submix effect from the effect submix chain. */
		AUDIOMIXER_API void RemoveSoundEffectSubmix(uint32 SubmixPresetId);

		/** Removes the submix effect from the effect submix chain at the given submix index. */
		AUDIOMIXER_API void RemoveSoundEffectSubmixAtIndex(int32 InIndex);

		/** Clears all submix effects from the effect submix chain. */
		AUDIOMIXER_API void ClearSoundEffectSubmixes();

		/** Sets a submix effect chain override with the given fade time in seconds. */
		AUDIOMIXER_API void SetSubmixEffectChainOverride(const TArray<FSoundEffectSubmixPtr>& InSubmixEffectPresetChain, float InFadeTimeSec);

		/** Clears any submix effect chain overrides in the given fade time in seconds. */
		AUDIOMIXER_API void ClearSubmixEffectChainOverride(float InFadeTimeSec);

		/** Swaps effect for provided submix at the given index.  Fails if effect at index doesn't exist */
		AUDIOMIXER_API void ReplaceSoundEffectSubmix(int32 InIndex, FSoundEffectSubmixPtr InEffectInstance);

		/** Whether or not this submix instance is muted. */
		AUDIOMIXER_API void SetBackgroundMuted(bool bInMuted);

		/** Checks to see if submix is valid.  Submix can be considered invalid if the OwningSubmix
		  * pointer is stale.
		  */
		AUDIOMIXER_API bool IsOwnerValid() const;

		UE_DEPRECATED(5.8, "Use IsOwnerValid() instead, to avoid confusion with TSharedPtr::IsValid().")
		AUDIOMIXER_API bool IsValid() const;

		// Function which processes audio.
		AUDIOMIXER_API void ProcessAudio(FAlignedFloatBuffer& OutAudio);
		AUDIOMIXER_API void ProcessAudio(ISoundfieldAudioPacket& OutputAudio);

		AUDIOMIXER_API void SendAudioToSubmixBufferListeners(FAlignedFloatBuffer& OutAudioBuffer);

		// This should be called if this submix doesn't send it's audio to a parent submix,
		// but rather an external endpoint.
		AUDIOMIXER_API void ProcessAudioAndSendToEndpoint();

		// Returns the device sample rate this submix is rendering to
		AUDIOMIXER_API int32 GetSampleRate() const;

		// Returns the output channels this submix is rendering to
		AUDIOMIXER_API int32 GetNumOutputChannels() const;

		// Returns the number of effects in this submix's effect chain
		AUDIOMIXER_API int32 GetNumChainEffects();

		// Returns the submix effect at the given effect chain index
		AUDIOMIXER_API FSoundEffectSubmixPtr GetSubmixEffect(const int32 InIndex);

		// This must be called on the entire submix graph before calling SetupSoundfieldStreams.
		AUDIOMIXER_API void SetSoundfieldFactory(ISoundfieldFactory* InSoundfieldFactory);

		// updates settings, potentially creating or removing ambisonics streams based on what types of submixes this submix is connected to.
		AUDIOMIXER_API void SetupSoundfieldStreams(const USoundfieldEncodingSettingsBase* SoundfieldSettings, TArray<USoundfieldEffectBase*>& Processors, ISoundfieldFactory* InSoundfieldFactory);
		AUDIOMIXER_API void TeardownSoundfieldStreams();

		AUDIOMIXER_API void SetupEndpoint(IAudioEndpointFactory* InFactory, const UAudioEndpointSettingsBase* InSettings);
		AUDIOMIXER_API void SetupEndpoint(ISoundfieldEndpointFactory* InFactory, const USoundfieldEndpointSettingsBase* InSettings);

		AUDIOMIXER_API void UpdateEndpointSettings(TUniquePtr<IAudioEndpointSettingsProxy>&& InSettings);
		AUDIOMIXER_API void UpdateEndpointSettings(TUniquePtr<ISoundfieldEndpointSettingsProxy>&& InSettings);

		// This is called by the corresponding USoundSubmix when StartRecordingOutput is called.
		AUDIOMIXER_API void OnStartRecordingOutput(float ExpectedDuration);

		// This is called by the corresponding USoundSubmix when StopRecordingOutput is called.
		AUDIOMIXER_API FAlignedFloatBuffer& OnStopRecordingOutput(float& OutNumChannels, float& OutSampleRate);

		// This is called by the corresponding USoundSubmix when PauseRecording is called.
		AUDIOMIXER_API void PauseRecordingOutput();

		// This is called by the corresponding USoundSubmix when ResumeRecording is called.
		AUDIOMIXER_API void ResumeRecordingOutput();

		// Register buffer listener with this submix
		AUDIOMIXER_API void RegisterBufferListener(TSharedRef<ISubmixBufferListener, ESPMode::ThreadSafe> BufferListener);

		// Unregister buffer listener with this submix
		AUDIOMIXER_API void UnregisterBufferListener(TSharedRef<ISubmixBufferListener, ESPMode::ThreadSafe> BufferListener);

		// Starts envelope following with the given attack time and release time
		AUDIOMIXER_API void StartEnvelopeFollowing(int32 AttackTime, int32 ReleaseTime);

		// Stops envelope following the submix
		AUDIOMIXER_API void StopEnvelopeFollowing();

		// Adds an envelope follower delegate
		AUDIOMIXER_API void AddEnvelopeFollowerDelegate(const FOnSubmixEnvelopeBP& OnSubmixEnvelopeBP);

		// Removes an existing envelope follower delegate
		AUDIOMIXER_API void RemoveEnvelopeFollowerDelegate(const FOnSubmixEnvelopeBP& OnSubmixEnvelopeBP);

		// Initializes a new FFT analyzer for this submix and immediately begins feeding audio to it.
		AUDIOMIXER_API void StartSpectrumAnalysis(const FSoundSpectrumAnalyzerSettings& InSettings);

		// Terminates whatever FFT Analyzer is being used for this submix.
		AUDIOMIXER_API void StopSpectrumAnalysis();

		// Adds an spectral analysis delegate
		AUDIOMIXER_API void AddSpectralAnalysisDelegate(const FSoundSpectrumAnalyzerDelegateSettings& InDelegateSettings, const FOnSubmixSpectralAnalysisBP& OnSubmixSpectralAnalysisBP);

		// Removes an existing spectral analysis delegate
		AUDIOMIXER_API void RemoveSpectralAnalysisDelegate(const FOnSubmixSpectralAnalysisBP& OnSubmixSpectralAnalysisBP);

		// Gets the most recent magnitude values for each corresponding value in InFrequencies (in Hz).
		// This requires StartSpectrumAnalysis to be called first.
		AUDIOMIXER_API void GetMagnitudeForFrequencies(const TArray<float>& InFrequencies, TArray<float>& OutMagnitudes);

		// Gets the most recent phase values for each corresponding value in InFrequencies (in Hz).
		// This requires StartSpectrumAnalysis to be called first.
		AUDIOMIXER_API void GetPhaseForFrequencies(const TArray<float>& InFrequencies, TArray<float>& OutPhases);

		// Broadcast the envelope and submix delegates on the game thread
		AUDIOMIXER_API void BroadcastDelegates();

		// returns true if this submix is encoded to a soundfield.
		AUDIOMIXER_API bool IsSoundfieldSubmix() const;

		// returns true if this submix sends it's audio to the default endpoint.
		AUDIOMIXER_API bool IsDefaultEndpointSubmix() const;

		// Returns true if this submix sends its audio to an IAudioEndpoint.
		AUDIOMIXER_API bool IsExternalEndpointSubmix() const;

		// returns true if this submix sends its audio to an ISoundfieldEndpoint.
		AUDIOMIXER_API bool IsSoundfieldEndpointSubmix() const;

		//Returns true if this is an endpoint type that should no-op for this platform
		AUDIOMIXER_API bool IsDummyEndpointSubmix() const;

		// Returns true if the submix is currently rendering audio. The current rendering time is passed in.
		AUDIOMIXER_API bool IsRenderingAudio() const;

		// Set whether or not this submix is told to auto disable. 
		AUDIOMIXER_API void SetAutoDisable(bool bInAutoDisable);

		// Sets the auto-disable time
		AUDIOMIXER_API void SetAutoDisableTime(float InAutoDisableTime);

		// Get a unique key for this submix's format and settings.
		// If another submix has an identical format and settings it will have an equivalent key.
		AUDIOMIXER_API FSoundfieldEncodingKey GetKeyForSubmixEncoding();

		AUDIOMIXER_API ISoundfieldFactory* GetSoundfieldFactory();

		AUDIOMIXER_API ISoundfieldEncodingSettingsProxy& GetSoundfieldSettings();

		AUDIOMIXER_API FAudioPluginInitializationParams GetInitializationParamsForSoundfieldStream();

		AUDIOMIXER_API FSoundfieldSpeakerPositionalData GetDefaultPositionalDataForAudioDevice();

		TWeakPtr<FMixerSubmix, ESPMode::ThreadSafe>  GetParent() const { return ParentSubmix; }
		const TMap<uint32, FChildSubmixInfo>& GetChildren() const { return ChildSubmixes; }

	protected:
		// Initialize the submix internal
		AUDIOMIXER_API void InitInternal();

		// Down mix the given buffer to the desired down mix channel count
		static AUDIOMIXER_API void DownmixBuffer(const int32 InChannels, const FAlignedFloatBuffer& InBuffer, const int32 OutChannels, FAlignedFloatBuffer& OutNewBuffer);

		AUDIOMIXER_API void MixBufferDownToMono(const FAlignedFloatBuffer& InBuffer, int32 NumInputChannels, FAlignedFloatBuffer& OutBuffer);

		AUDIOMIXER_API void SetupSoundfieldEncodersForChildren();
		AUDIOMIXER_API void SetupSoundfieldEncodingForChild(FChildSubmixInfo& InChild);

		// Check to see if we need to decode from ambisonics for parent
		AUDIOMIXER_API void SetupSoundfieldStreamForParent();

		// This sets up the ambisonics positional data for speakers, based on what new format we need to convert to.
		AUDIOMIXER_API void SetUpSoundfieldPositionalData(const TSharedPtr<Audio::FMixerSubmix, ESPMode::ThreadSafe>& InParentSubmix);

		// Encode a source and sum it into the mixed soundfield.
		AUDIOMIXER_API void MixInSource(const ISoundfieldAudioPacket& InAudio, const ISoundfieldEncodingSettingsProxy& InSettings, ISoundfieldAudioPacket& PacketToSumTo);

		AUDIOMIXER_API void UpdateListenerRotation(const FQuat& InRotation);

		// Calls ProcessAudio on the child submix, performs all necessary conversions and mixes in it's resulting audio.
		AUDIOMIXER_API void MixInChildSubmix(FChildSubmixInfo& Child, ISoundfieldAudioPacket& PacketToSumTo);

		AUDIOMIXER_API FName GetSoundfieldFormat() const;

		AUDIOMIXER_API TUniquePtr<ISoundfieldTranscodeStream> GetTranscoderForChildSubmix(const TSharedPtr<Audio::FMixerSubmix, ESPMode::ThreadSafe>& InChildSubmix);

	protected:
		struct FSubmixBufferListenerInfo
		{
			ISubmixBufferListener* Listener = nullptr;
			FString Descriptor;
		};

		// Pump command queue
		AUDIOMIXER_API void PumpCommandQueue();

		// Add command to the command queue
		AUDIOMIXER_API void SubmixCommand(TFunction<void()> Command);

		// Generates audio from the given effect chain into the given buffer
		AUDIOMIXER_API bool GenerateEffectChainAudio(FSoundEffectSubmixInputData& InputData, const FAlignedFloatBuffer& InAudioBuffer, TArray<FSoundEffectSubmixPtr>& InEffectChain, FAlignedFloatBuffer& OutBuffer);

#if UE_AUDIO_PROFILERTRACE_ENABLED
		AUDIOMIXER_API void TraceSubmixEnvelopeValues(int32 InNumChannels, float InOutputVolume);
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

		// The name of this submix (the owning USoundSubmix) (at top so we can see in debugger it's name)
		FString SubmixName;

#if UE_AUDIO_PROFILERTRACE_ENABLED
		// The path name of this submix (the owning USoundSubmix)
		FString SubmixPathName;
#endif // UE_AUDIO_PROFILERTRACE_ENABLED
		
		// This mixer submix's Id
		uint32 Id;

		// Parent submix. 
		TWeakPtr<FMixerSubmix, ESPMode::ThreadSafe> ParentSubmix;

		// Child submixes
		TMap<uint32, FChildSubmixInfo> ChildSubmixes;

		// Struct to hold record keeping data about effect chain overrides
		struct FSubmixEffectFadeInfo
		{
			TArray<FSoundEffectSubmixPtr> EffectChain;

			FDynamicParameter FadeVolume = FDynamicParameter(1.0f);

			// If true, this effect override will be fading in or all the way faded in
			bool bIsCurrentChain = false;

			// If this effect fade info is the base effect
			bool bIsBaseEffect = false;
		};

		// The array of submix effect overrides. There may be more than one if multiple are fading out. There should be only one fading in (the current override).
		TArray<FSubmixEffectFadeInfo> EffectChains;
		FAlignedFloatBuffer EffectChainOutputBuffer;

		// Owning mixer device. 
		FMixerDevice* MixerDevice;

		// Map of mixer source voices identified by SourceId with a given send level for this submix
		TMap<int32, FSubmixVoiceData> MixerSourceIds;

		FAlignedFloatBuffer ScratchBuffer;
		FAlignedFloatBuffer SubmixChainMixBuffer;
		FAlignedFloatBuffer FilterScratchBuffer;
		FAlignedFloatBuffer InputBuffer;
		FAlignedFloatBuffer DownmixedBuffer;
		FAlignedFloatBuffer SourceInputBuffer;

		int32 NumChannels;
		int32 NumSamples;

		/**
		 * Individual processor in our 
		 */
		struct FSoundfieldEffectProcessorData
		{
			TUniquePtr<ISoundfieldEffectSettingsProxy> Settings;
			TUniquePtr<ISoundfieldEffectInstance> Processor;

			FSoundfieldEffectProcessorData(ISoundfieldFactory* InFactory, ISoundfieldEncodingSettingsProxy& InSettings, USoundfieldEffectBase* InProcessorBase)
			{
				check(InFactory);

				// As a sanity check, make sure if we've gotten to this point, this DSP processor supports this submix's format.
				check(InProcessorBase->SupportsFormat(InFactory->GetSoundfieldFormatName()));

				Processor = InProcessorBase->PrivateGetNewProcessor(InSettings);
				
				// If the processor doesn't have any settings, get the default settings for a processor of this type.
				const USoundfieldEffectSettingsBase* ProcessorSettings = InProcessorBase->Settings;
				if (!ProcessorSettings)
				{
					ProcessorSettings = InProcessorBase->PrivateGetDefaultSettings();
				}

				Settings = ProcessorSettings->PrivateGetProxy();
			}
		};

		struct FSoundfieldStreams
		{
			ISoundfieldFactory* Factory;

			// This encoder is used for the mixed down audio from all non-soundfield submixes plugged into
			// this submix. Will not be set up if ISoundfieldFactory::ShouldEncodeAllStreamsIndependently 
			// returns true.
			TUniquePtr<ISoundfieldEncoderStream> DownmixedChildrenEncoder;
			
			// Encoder used if a normal submix outputs to this submix.
			TUniquePtr<ISoundfieldDecoderStream> ParentDecoder;

			// This is the positional data we are decoding 
			FSoundfieldSpeakerPositionalData CachedPositionalData;

			// Mixes all encoded child submix inputs.
			TUniquePtr<ISoundfieldMixerStream> Mixer;

			// This is the packet we mix all input sources and child submixes to.
			TUniquePtr<ISoundfieldAudioPacket> MixedDownAudio;

			// Current settings for this submix.
			TUniquePtr<ISoundfieldEncodingSettingsProxy> Settings;

			// All soundfield processors attached to this submix.  
			TArray<FSoundfieldEffectProcessorData> EffectProcessors;

			// This critical section is contended by the soundfield overload of ProcessAudio and SetupSoundfieldStreams.
			FCriticalSection StreamsLock;

			FSoundfieldStreams()
				: Factory(nullptr)
			{}

			void Reset()
			{
				Factory = nullptr;
				ParentDecoder.Reset();
				Mixer.Reset();
				Settings.Reset();
			}
		};

		FSoundfieldStreams SoundfieldStreams;

		struct FEndpointData
		{
			// For endpoint submixes,
			// this is the primary method of pushing audio to the endpoint.
			Audio::FPatchInput Input;

			TUniquePtr<IAudioEndpoint> NonSoundfieldEndpoint;
			TUniquePtr<ISoundfieldEndpoint> SoundfieldEndpoint;

			// for non-soundfield endpoints, we use these buffers for processing.
			FAlignedFloatBuffer AudioBuffer;
			FAlignedFloatBuffer ResampledAudioBuffer;
			FAlignedFloatBuffer DownmixedResampledAudioBuffer;
			FAlignedFloatBuffer DownmixChannelMap;

			// Number of channels and sample rate for the external endpoint.
			int32 NumChannels;
			float SampleRate;

			// This is used if the endpoint has a different sample rate than our audio engine.
			Audio::FResampler Resampler;
			bool bShouldResample;

			// for soundfield endpoints, this is the buffer we use to send audio to the endpoint.
			TUniquePtr<ISoundfieldAudioPacket> AudioPacket;

			FEndpointData()
				: NumChannels(0)
				, SampleRate(0.0f)
				, bShouldResample(false)
			{}

			void Reset()
			{
				AudioBuffer.Reset();
				ResampledAudioBuffer.Reset();
				DownmixedResampledAudioBuffer.Reset();
				DownmixChannelMap.Reset();
				NonSoundfieldEndpoint.Reset();
				SoundfieldEndpoint.Reset();
			}
		};

		FEndpointData EndpointData;
		
		float CurrentOutputVolume;
		float TargetOutputVolume;
		float CurrentWetLevel;
		float TargetWetLevel;
		float CurrentDryLevel;
		float TargetDryLevel;

		FModulationDestination VolumeMod;
		FModulationDestination DryLevelMod;
		FModulationDestination WetLevelMod;
		
		std::atomic<float> CachedVolumeModValue { 1.0f };
		std::atomic<float> CachedWetLevelModValue { 1.0f };

		float VolumeModBaseDb = 0.f;
		float DryModBaseDb = MIN_VOLUME_DECIBELS;
		float WetModBaseDb = 0.f;

		// modifiers set from BP code
		float VolumeModifier = 1.f;
		float DryLevelModifier = 1.f;
		float WetLevelModifier = 1.f;

		// Debug mute/solo volume
		std::atomic<float> DebugVolume { 1.f };

		// Envelope following data
		float EnvelopeValues[AUDIO_MIXER_MAX_OUTPUT_CHANNELS];
		Audio::FEnvelopeFollower EnvelopeFollower;
		int32 EnvelopeNumChannels;
		FCriticalSection EnvelopeCriticalSection;

		// Spectrum analyzer. Created and destroyed on the audio thread.
		FCriticalSection SpectrumAnalyzerCriticalSection;
		FSoundSpectrumAnalyzerSettings SpectrumAnalyzerSettings;
		TSharedPtr<FAsyncSpectrumAnalyzer, ESPMode::ThreadSafe> SpectrumAnalyzer;
		
		// This buffer is used to downmix the submix output to mono before submitting it to the SpectrumAnalyzer.
		FAlignedFloatBuffer MonoMixBuffer;

		// The dry channel buffer
		FAlignedFloatBuffer DryChannelBuffer;

		// Submix command queue to shuffle commands from audio thread to audio render thread.
		TQueue<TFunction<void()>> CommandQueue;

		// List of submix buffer listeners. (mutable for pruning stale weak references)

		mutable TArray<TWeakPtr<ISubmixBufferListener>> BufferListenerPtrs;
		// Critical section used for modifying and interacting with buffer listeners
		mutable FCriticalSection BufferListenerCriticalSection;

		// This buffer is used for recorded output of the submix.
		FAlignedFloatBuffer RecordingData;

		// Returns the number of submix effects
		int32 NumSubmixEffects;

		// Bool set to true when this submix is recording data.
		uint8 bIsRecording : 1;

		// Whether or not this submix is muted.
		uint8 bIsBackgroundMuted : 1;

		// Whether or not auto-disablement is enabled. If true, the submix will disable itself.
		uint8 bAutoDisable : 1;

		// Whether or not the submix is currently rendering audio. I.e. audio was sent to it and mixing it, or any of its child submixes are rendering audio.
		uint8 bIsSilent : 1;

		// Whether or not we're currently disabled (i.e. the submix has been silent)
		uint8 bIsCurrentlyDisabled : 1;

		// The time to wait to disable the submix if the auto-disablement is active.
		double AutoDisableTime;

		// The time that the first full silent buffer was detected in the submix. Submix will auto-disable if the timeout is reached and the submix has bAutoDisable set to true.
		double SilenceTimeStartSeconds;

		// Bool set to true when envelope following is enabled
		FThreadSafeBool bIsEnvelopeFollowing;

		// Ref count for the number of times a request for submixes to start envelope following
		FThreadSafeCounter EnvelopeFollowerRequestsRefCount = 0;

		// Multi-cast delegate to broadcast envelope data from this submix instance
		FOnSubmixEnvelope OnSubmixEnvelope;

		struct FSpectralAnalysisBandInfo
		{
			FInlineEnvelopeFollower EnvelopeFollower;
		};

		struct FSpectrumAnalysisDelegateInfo
		{
			FSoundSpectrumAnalyzerDelegateSettings DelegateSettings;

			FOnSubmixSpectralAnalysis OnSubmixSpectralAnalysis;

			TUniquePtr<ISpectrumBandExtractor> SpectrumBandExtractor;
			TArray<FSpectralAnalysisBandInfo> SpectralBands;

			float LastUpdateTime = -1.0f;
			float UpdateDelta = 0.0f;

			FSpectrumAnalysisDelegateInfo()
			{
			}

			FSpectrumAnalysisDelegateInfo(FSpectrumAnalysisDelegateInfo&& Other)
			{
				OnSubmixSpectralAnalysis = Other.OnSubmixSpectralAnalysis;
				SpectrumBandExtractor.Reset(Other.SpectrumBandExtractor.Release());
				DelegateSettings = Other.DelegateSettings;
				SpectralBands = Other.SpectralBands;
			}

			~FSpectrumAnalysisDelegateInfo() = default;
		};

		TArray<FSpectrumAnalysisDelegateInfo> SpectralAnalysisDelegates;

		// Bool set to true when spectrum analysis is enabled
		FThreadSafeBool bIsSpectrumAnalyzing;

		// Critical section used for when we are appending recorded data.
		FCriticalSection RecordingCriticalSection;

		// Critical section for mutation of the effect chain.
		FCriticalSection EffectChainMutationCriticalSection;

		// Handle back to the owning USoundSubmix. Used when the device is shutdown to prematurely end a recording.
		TWeakObjectPtr<const USoundSubmixBase> OwningSubmixObject;

		Audio::FPatchSplitter PatchSplitter;

		TUniquePtr<IAudioLink> AudioLinkInstance;

		friend class FMixerDevice;

	private:
		AUDIOMIXER_API void SendAudioToRegisteredAudioBuses(FAlignedFloatBuffer& OutAudioBuffer);

		void UnregisterBufferListenerInternal(UPTRINT ListenerBufferPtr);

		void PruneSubmixBufferListeners();

		// Registered audio buses
		TMap<Audio::FAudioBusKey, Audio::FPatchInput> AudioBuses;
		FAlignedFloatBuffer AudioBusSendBuffer;
	};
}
