// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_DEV_AUTOMATION_TESTS

#include "CoreMinimal.h"

// A simple test harness for the scheduler. Numbered steps and dependencies between them can be added and
//  the order the steps were run in is returned.
class IAudioRenderSchedulerTester
{
public:
	static AUDIOMIXER_API TUniquePtr<IAudioRenderSchedulerTester> Create();

	virtual ~IAudioRenderSchedulerTester() = default;

	// Add a numbered step to the scheduler.
	virtual void AddStep(int Step) = 0;

	// Automatically creates the steps if they don't exist yet. 
	virtual void AddDependency(int FirstStep, int SecondStep) = 0;

	virtual void RemoveDependency(int FirstStep, int SecondStep) = 0;

	// Runs the scheduler, and returns the order steps were executed in.
	virtual TArray<int> Run() = 0;
};

#endif //WITH_DEV_AUTOMATION_TESTS
