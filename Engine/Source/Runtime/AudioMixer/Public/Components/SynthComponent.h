// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioMixerTypes.h"
#include "Components/AudioComponent.h"
#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "IAudioExtensionPlugin.h"
#include "Sound/SoundWaveProcedural.h"
#include "Sound/SoundGenerator.h"
#include "UObject/ObjectMacros.h"

#include "SynthComponent.generated.h"

#define SYNTH_GENERATOR_TEST_TONE 0

#if SYNTH_GENERATOR_TEST_TONE
#include "DSP/SinOsc.h"
#endif

class UAudioBus;
class USoundSourceBus;

/** Simple interface class to allow objects to route audio between them. */
class IAudioBufferListener
{
public:
	virtual void OnGeneratedBuffer(const float* AudioBuffer, const int32 NumSamples, const int32 NumChannels) = 0;
};

class USynthComponent;
class USoundConcurrency;

/**
* Called by a synth component and returns the sound's envelope value (using an envelope follower in the audio renderer).
* This only works in the audio mixer.
*/
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnSynthEnvelopeValue, const float, EnvelopeValue);

/** shadow delegate declaration for above */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnSynthEnvelopeValueNative, const class UAudioComponent*, const float);


UCLASS(MinimalAPI)
class USynthSound : public USoundWaveProcedural
{
	GENERATED_UCLASS_BODY()

	AUDIOMIXER_API void Init(USynthComponent* InSynthComponent, const int32 InNumChannels, const int32 SampleRate, const int32 InCallbackSize);
	AUDIOMIXER_API void StartOnAudioDevice(FAudioDevice* InAudioDevice);

	/** Begin USoundWave */
	AUDIOMIXER_API virtual void OnBeginGenerate() override;
	AUDIOMIXER_API virtual int32 OnGeneratePCMAudio(TArray<uint8>& OutAudio, int32 NumSamples) override;
	AUDIOMIXER_API virtual void OnEndGenerate() override;
	AUDIOMIXER_API virtual Audio::EAudioMixerStreamDataFormat::Type GetGeneratedPCMDataFormat() const override;
	AUDIOMIXER_API virtual ISoundGeneratorPtr CreateSoundGenerator(const FSoundGeneratorInitParams& InParams) override;
	/** End USoundWave */

protected:
	UPROPERTY()
	TWeakObjectPtr<USynthComponent> OwningSynthComponent = nullptr;

	TArray<float> FloatBuffer;

public:
	USynthComponent* GetOwningSynthComponent()
	{
		return OwningSynthComponent.Get();
	}
	
	TWeakObjectPtr<USynthComponent>& GetOwningSynthComponentPtr()
	{
		return OwningSynthComponent;
	}
};

UCLASS(abstract, ClassGroup = Synth, hidecategories = (Object, ActorComponent, Physics, Rendering, Mobility, LOD), MinimalAPI)
class USynthComponent : public USceneComponent
{
	GENERATED_BODY()

public:
	AUDIOMIXER_API USynthComponent(const FObjectInitializer& ObjectInitializer);

	//~ Begin USceneComponent Interface
	AUDIOMIXER_API virtual void Activate(bool bReset = false) override;
	AUDIOMIXER_API virtual void Deactivate() override;
	//~ End USceneComponent Interface

	//~ Begin ActorComponent Interface.
	AUDIOMIXER_API virtual void OnRegister() override;
	AUDIOMIXER_API virtual void OnUnregister() override;
	AUDIOMIXER_API virtual bool IsReadyForOwnerToAutoDestroy() const override;
	AUDIOMIXER_API virtual void EndPlay(const EEndPlayReason::Type Reason) override;
	//~ End ActorComponent Interface.

	//~ Begin UObject Interface.
#if WITH_EDITOR
	AUDIOMIXER_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
	AUDIOMIXER_API virtual void PostLoad() override;
#endif //WITH_EDITORONLY_DATA

	AUDIOMIXER_API virtual void Serialize(FArchive& Ar) override;
	//~ End UObject Interface

	// Starts the synth generating audio.
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	AUDIOMIXER_API void Start();

	// Stops the synth generating audio.
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	AUDIOMIXER_API void Stop();

	/** Returns true if this component is currently playing. */
	UFUNCTION(BlueprintCallable, Category = "Synth|Components|Audio")
	AUDIOMIXER_API bool IsPlaying() const;

	/** Set a new volume multiplier */
	UFUNCTION(BlueprintCallable, Category = "Audio|Components|Audio")
	AUDIOMIXER_API void SetVolumeMultiplier(float VolumeMultiplier);

	/** Sets how much audio the sound should send to the given submix. */
	UFUNCTION(BlueprintCallable, Category = "Audio|Components|Audio")
	AUDIOMIXER_API void SetSubmixSend(USoundSubmixBase* Submix, float SendLevel);

	/** Sets how much audio the sound should send to the given SourceBus (pre effect). */
	UFUNCTION(BlueprintCallable, Category = "Audio|Components|Audio")
	AUDIOMIXER_API void SetSourceBusSendPreEffect(USoundSourceBus* SoundSourceBus, float SourceBusSendLevel);

	/** Sets how much audio the sound should send to the given SourceBus (post effect). */
	UFUNCTION(BlueprintCallable, Category = "Audio|Components|Audio")
	AUDIOMIXER_API void SetSourceBusSendPostEffect(USoundSourceBus* SoundSourceBus, float SourceBusSendLevel);

	/** Sets how much audio the sound should send to the given AudioBus (pre effect). */
	UFUNCTION(BlueprintCallable, Category = "Audio|Components|Audio")
	AUDIOMIXER_API void SetAudioBusSendPreEffect(UAudioBus* AudioBus, float AudioBusSendLevel);

	/** Sets how much audio the sound should send to the given AudioBus (post effect). */
	UFUNCTION(BlueprintCallable, Category = "Audio|Components|Audio")
	AUDIOMIXER_API void SetAudioBusSendPostEffect(UAudioBus* AudioBus, float AudioBusSendLevel);

	/** Sets whether or not the low pass filter is enabled on the audio component. */
	UFUNCTION(BlueprintCallable, Category = "Audio|Components|Audio")
	AUDIOMIXER_API void SetLowPassFilterEnabled(bool InLowPassFilterEnabled);

	/** Sets lowpass filter frequency of the audio component. */
	UFUNCTION(BlueprintCallable, Category = "Audio|Components|Audio")
	AUDIOMIXER_API virtual void SetLowPassFilterFrequency(float InLowPassFilterFrequency);

	/** Sets whether or not the synth component outputs its audio to any source or audio buses. */
	UFUNCTION(BlueprintCallable, Category = "Audio|Components|Audio")
	AUDIOMIXER_API void SetOutputToBusOnly(bool bInOutputToBusOnly);

	/**
	 * This function allows designers to call Play on an Audio Component instance while applying a volume curve over time. 
	 * Parameters allow designers to indicate the duration of the fade, the curve shape, and the start time if seeking into the sound.
	 *
	 * @param FadeInDuration How long it should take to reach the FadeVolumeLevel
	 * @param FadeVolumeLevel The percentage of the AudioComponents's calculated volume to fade to
	 * @param FadeCurve The curve to use when interpolating between the old and new volume
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio|Components|Audio")
	AUDIOMIXER_API void FadeIn(float FadeInDuration, float FadeVolumeLevel = 1.0f, float StartTime = 0.0f, const EAudioFaderCurve FadeCurve = EAudioFaderCurve::Linear) const;

	/**
	 * This function allows designers to call a delayed Stop on an Audio Component instance while applying a
	 * volume curve over time. Parameters allow designers to indicate the duration of the fade and the curve shape.
	 *
	 * @param FadeOutDuration how long it should take to reach the FadeVolumeLevel
	 * @param FadeVolumeLevel the percentage of the AudioComponents's calculated volume in which to fade to
	 * @param FadeCurve The curve to use when interpolating between the old and new volume
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio|Components|Audio")
	AUDIOMIXER_API void FadeOut(float FadeOutDuration, float FadeVolumeLevel, const EAudioFaderCurve FadeCurve = EAudioFaderCurve::Linear) const;

	/** This function allows designers to trigger an adjustment to the sound instance’s playback Volume with options for smoothly applying a curve over time.
     * @param AdjustVolumeDuration The length of time in which to interpolate between the initial volume and the new volume.
     * @param AdjustVolumeLevel The new volume to set the Audio Component to.
     * @param FadeCurve The curve used when interpolating between the old and new volume.
     */
	UFUNCTION(BlueprintCallable, Category = "Audio|Components|Audio")
	AUDIOMIXER_API void AdjustVolume(float AdjustVolumeDuration, float AdjustVolumeLevel, const EAudioFaderCurve FadeCurve = EAudioFaderCurve::Linear) const;

	/**
	* Sets the routing for one of the given Synth component's Modulation Destinations.
	* @param Modulators The set of modulators to apply to the given destination on the component.
	* @param Destination The destination to assign the modulators to.
	* @param RoutingMethod The routing method to use for the given modulator.
	*/
	UFUNCTION(BlueprintCallable, Category = "Audio|Components|Audio", DisplayName = "Set Modulation Routing")
	AUDIOMIXER_API void SetModulationRouting(const TSet<USoundModulatorBase*>& Modulators, const EModulationDestination Destination, const EModulationRouting RoutingMethod = EModulationRouting::Inherit);
	
	/**
	* Gets the set of currently active modulators for a given Modulation Destination.
	* @param Destination The Destination to retrieve the Modulators from.
	* @return The set of of Modulators applied to this component for the given Destination.
	*/
	UFUNCTION(BlueprintPure, Category = "Audio|Components|Audio", DisplayName = "Get Modulators")
	AUDIOMIXER_API UPARAM(DisplayName = "Modulators") TSet<USoundModulatorBase*> GetModulators(const EModulationDestination Destination);

	/** Auto destroy this component on completion */
	UPROPERTY()
	uint8 bAutoDestroy : 1;

	/** Stop sound when owner is destroyed */
	UPROPERTY()
	uint8 bStopWhenOwnerDestroyed : 1;

	/** Is this audio component allowed to be spatialized? */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attenuation)
	uint8 bAllowSpatialization : 1;

	/** Should the Attenuation Settings asset be used (false) or should the properties set directly on the component be used for attenuation properties */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attenuation)
	uint8 bOverrideAttenuation : 1;

#if WITH_EDITORONLY_DATA
	/** Whether or not to only send this audio's output to a bus. If true, this sound will not be audible except through bus sends. */
	UPROPERTY()
	uint32 bOutputToBusOnly_DEPRECATED : 1;
#endif //WITH_EDITORONLY_DATA

	/** Whether or not to enable sending this audio's output to buses.  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Effects)
	uint32 bEnableBusSends : 1;

	/** If enabled, sound will route to the Master Submix by default or to the Base Submix if defined. If disabled, sound will route ONLY to the Submix Sends and/or Bus Sends */
	UPROPERTY(EditAnywhere, Category = Effects)
	uint32 bEnableBaseSubmix : 1;

	/** Whether or not to enable Submix Sends other than the Base Submix.*/
	UPROPERTY(EditAnywhere, Category = Effects, meta = (DisplayAfter = "SoundSubmixObject"))
	uint32 bEnableSubmixSends : 1;

	/** If bOverrideSettings is false, the asset to use to determine attenuation properties for sounds generated by this component */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attenuation, meta = (EditCondition = "!bOverrideAttenuation"))
	TObjectPtr<class USoundAttenuation> AttenuationSettings;

	/** If bOverrideSettings is true, the attenuation properties to use for sounds generated by this component */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attenuation, meta = (EditCondition = "bOverrideAttenuation"))
	struct FSoundAttenuationSettings AttenuationOverrides;

	/** What sound concurrency to use for sounds generated by this audio component */
	UPROPERTY()
	TObjectPtr<USoundConcurrency> ConcurrencySettings_DEPRECATED;

	/** What sound concurrency to use for sounds generated by this audio component */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Concurrency)
	TSet<TObjectPtr<USoundConcurrency>> ConcurrencySet;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Modulation)
	FSoundModulationDefaultRoutingSettings ModulationRouting;

	/** Sound class this sound belongs to */
	UPROPERTY(EditAnywhere, Category = SoundClass)
	TObjectPtr<USoundClass> SoundClass;

	/** The source effect chain to use for this sound. */
	UPROPERTY(EditAnywhere, Category = Effects)
	TObjectPtr<USoundEffectSourcePresetChain> SourceEffectChain;

	/** Submix this sound belongs to */
	UPROPERTY(EditAnywhere, Category = Effects, meta = (EditCondition = "bEnableBaseSubmix", DisplayName = "Base Submix"))
	TObjectPtr<USoundSubmixBase> SoundSubmix;

	/** An array of submix sends. Audio from this sound will send a portion of its audio to these effects.  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Effects, meta = (EditCondition = "bEnableSubmixSends"))
	TArray<FSoundSubmixSendInfo> SoundSubmixSends;

	/** This sound will send its audio output to this list of buses if there are bus instances playing after source effects are processed.  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Effects, meta = (DisplayName = "Post-Effect Bus Sends", EditCondition = "bEnableBusSends"))
	TArray<FSoundSourceBusSendInfo> BusSends;

	/** This sound will send its audio output to this list of buses if there are bus instances playing before source effects are processed.  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Effects, meta = (DisplayName = "Pre-Effect Bus Sends", EditCondition = "bEnableBusSends"))
	TArray<FSoundSourceBusSendInfo> PreEffectBusSends;

	/** Whether or not this sound plays when the game is paused in the UI */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Sound)
	uint8 bIsUISound : 1;

	/** Whether or not this synth is playing as a preview sound */
	UPROPERTY()
	uint8 bIsPreviewSound : 1;

	/** Whether to artificially prioritize the component to play */
	uint8 bAlwaysPlay : 1;

	/** Call if creating this synth component not via an actor component in BP, but in code or some other location. 
	 *  Optionally override the sample rate of the sound wave, otherwise it uses the audio device's sample rate. 
	 */
	AUDIOMIXER_API void Initialize(int32 SampleRateOverride = INDEX_NONE);

	/** Creates the audio component if it hasn't already been created yet. This should only be used when trying to
	 *  assign explicit settings to the AudioComponent before calling Start(). 
	 */
	AUDIOMIXER_API void CreateAudioComponent();

	/** Retrieves this synth component's audio component. */
	AUDIOMIXER_API UAudioComponent* GetAudioComponent();

	/** The attack time in milliseconds for the envelope follower. Delegate callbacks can be registered to get the
	 *  envelope value of sounds played with this audio component. Only used in audio mixer.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Sound, meta = (ClampMin = "0", UIMin = "0"))
	int32 EnvelopeFollowerAttackTime;

	/** The release time in milliseconds for the envelope follower. Delegate callbacks can be registered to get the
	 *  envelope value of sounds played with this audio component. Only used in audio mixer. 
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Sound, meta = (ClampMin = "0", UIMin = "0"))
	int32 EnvelopeFollowerReleaseTime;

	UPROPERTY(BlueprintAssignable)
	FOnSynthEnvelopeValue OnAudioEnvelopeValue;

	/** Shadow delegate for non UObject subscribers */
	FOnSynthEnvelopeValueNative OnAudioEnvelopeValueNative;

	AUDIOMIXER_API void OnAudioComponentEnvelopeValue(const UAudioComponent* AudioComponent, const USoundWave* SoundWave, const float EnvelopeValue);

	// Adds and removes audio buffer listener
	AUDIOMIXER_API void AddAudioBufferListener(IAudioBufferListener* InAudioBufferListener);
	AUDIOMIXER_API void RemoveAudioBufferListener(IAudioBufferListener* InAudioBufferListener);

	AUDIOMIXER_API virtual USoundClass* GetSoundClass();

	AUDIOMIXER_API virtual void BeginDestroy() override;

protected:

	// Method to execute parameter changes on game thread in audio render thread
	AUDIOMIXER_API void SynthCommand(TFunction<void()> Command);

	// Called when synth is created.
	virtual bool Init(int32& SampleRate) { return true; }

	// Called when the synth component begins generating audio in render thread
	virtual void OnBeginGenerate() {}

	// Called when the synth has finished generating audio on the render thread
	virtual void OnEndGenerate() {}

	// Called when more audio is needed to be generated
	// This method of generating audio is soon to be deprecated. For all new synth components, create an FSoundGenerator instance and implement CreateSoundGenerator method to create an instance.
	virtual int32 OnGenerateAudio(float* OutAudio, int32 NumSamples) { return 0; }

	// Implemented by the synth component to create a generator object instead of generating audio directly on the synth component.
	// This method prevents UObjects from having to exist in the audio render thread.
	virtual ISoundGeneratorPtr CreateSoundGenerator(const FSoundGeneratorInitParams& InParams) { return nullptr; }

	// Called by procedural sound wave
	// Returns the number of samples actually generated
	AUDIOMIXER_API int32 OnGeneratePCMAudio(float* GeneratedPCMData, int32 NumSamples);

	// Gets the audio device associated with this synth component
	AUDIOMIXER_API FAudioDevice* GetAudioDevice() const;

	// Can be set by the derived class, defaults to 2
	int32 NumChannels;

	// Can be set by the derived class- sets the preferred callback size for the synth component.
	int32 PreferredBufferLength;

private:
	// Creates the synth component's sound generator, calls into overridden client code to create the instance.
	AUDIOMIXER_API ISoundGeneratorPtr CreateSoundGeneratorInternal(const FSoundGeneratorInitParams& InParams);

	UPROPERTY(Transient)
	TObjectPtr<USynthSound> Synth;

	UPROPERTY(Transient)
	TObjectPtr<UAudioComponent> AudioComponent;

	AUDIOMIXER_API void PumpPendingMessages();

#if SYNTH_GENERATOR_TEST_TONE
	Audio::FSineOsc TestSineLeft;
	Audio::FSineOsc TestSineRight;
#endif

	// Whether or not synth is playing
	bool bIsSynthPlaying;
	bool bIsInitialized;

	TQueue<TFunction<void()>> CommandQueue;

	// Synth component's handle to its sound generator instance.
	// used to forward BP functions to the instance directly.
	ISoundGeneratorPtr SoundGenerator;

	friend class USynthSound;
};
