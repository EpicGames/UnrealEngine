// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DSP/Dsp.h"
#include "SampleBuffer.h"
#include "AudioDecompress.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "Interfaces/IAudioFormat.h"

class FAudioFileReader
{
public:
	// Constructor. Takes a file path and immediately loads info.
	// Optionally, CallbackSize can be used to indicate the size of chunks
	// that will be popped off of this instance.
	// When set to 0, the entire file is decompressed into memory.
	AUDIOMIXER_API FAudioFileReader(const FString& InPath);

	// Returns file information.
	AUDIOMIXER_API void GetFileInfo(FSoundQualityInfo& OutInfo);

	AUDIOMIXER_API bool PopAudio(float* OutAudio, int32 NumSamples);

private:
	FAudioFileReader();

	// Handle back to the file this was constructed with.
	TUniquePtr<IFileHandle> FileHandle;

	// Actual decompressor in question.
	TUniquePtr<ICompressedAudioInfo> Decompressor;
	
	TArray<uint8> CompressedFile;
	TArray<Audio::DefaultUSoundWaveSampleType> DecompressionBuffer;

	FSoundQualityInfo QualityInfo;

	ICompressedAudioInfo* GetNewDecompressorForFile(const FString& InPath);

};
