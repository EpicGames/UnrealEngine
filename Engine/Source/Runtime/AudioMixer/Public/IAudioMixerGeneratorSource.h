// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace Audio
{
	UE_EXPERIMENTAL(5.8, "AudioMixer Generator Source API is experimental and likely to change.")
	class IAudioMixerGeneratorSource
	{
	public:
		virtual ~IAudioMixerGeneratorSource() = default;

		virtual void Play() = 0;
		virtual void Stop() = 0;

		virtual bool IsDone() = 0;
	};
}
