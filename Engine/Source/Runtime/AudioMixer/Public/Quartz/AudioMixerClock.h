// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/ThreadSafeBool.h"
#include "Sound/QuartzQuantizationUtilities.h"
#include "Quartz/QuartzMetronome.h"
#include "Sound/QuartzSubscription.h"
#include "Sound/QuartzInterfaces.h"
#include "Sound/QuartzCommandQueue.h"

#define UE_API AUDIOMIXER_API

namespace Audio
{
	// forwards
	class FMixerDevice;
	class FQuartzClock;
	class FMixerSourceManager;
	class FQuartzClockManager;

	using FQuartzClockCommandQueueType = Audio::Quartz::PrivateDefs::TQuartzCommandQueue<IQuartzClock>;
	using FQuartzClockCommandQueuePtr = TSharedPtr<FQuartzClockCommandQueueType, ESPMode::ThreadSafe>;
	using FQuartzClockCommandQueueWeakPtr = TWeakPtr<FQuartzClockCommandQueueType, ESPMode::ThreadSafe>;
	
	/**
	 *	FQuartzClockProxy:
	 *
	 *		This class is a C++ handle to the underlying clock.
	 *		
	 *		It is mostly a wrapper around a TWeakPtr<FQuartzClock> and
	 *		FQuartzClockCommandQueueType
	 *		
	 *		The getters query the underlying FQuartzClock directly,
	 *		which returns values updated during the last audio-engine tick
	 *
	 *		If you need to add more getters, add copies of the members in question to
	 *		FQuartzClock::FQuartzClockState and update FQuartzClock::UpdateCachedState()
	 *		for thread-safe access (or manually protect access w/ CachedClockStateCritSec)
	 *
	 *		SendCommandToClock() can be used to execute lambdas at the beginning
	 *		of the next clock tick.  These lambdas can call FQuartzClock's public methods safely.
	 *
	 *		Your lambda will take an FQuartzClock* as an argument, which will be passed in by the
	 *		FQuartzClock itself when it pumps the command queue.
	 *
	 */
	class FQuartzClockProxy
	{
	public:
		// ctor
		FQuartzClockProxy() {}
		FQuartzClockProxy(const FName& Name) : ClockId(Name){ } // conv ctor from FName
		UE_API FQuartzClockProxy(TSharedPtr<FQuartzClock, ESPMode::ThreadSafe> InClock);

		FName GetClockName() const { return ClockId; }

		UE_API bool IsValid() const;
		operator bool() const { return IsValid(); }

		bool operator==(const FName& Name) const { return ClockId == Name; }

		UE_API bool DoesClockExist() const;

		UE_API bool IsClockRunning() const;

		UE_API Audio::FQuartzClockTickRate GetTickRate() const;

		UE_API float GetEstimatedClockRunTimeSeconds() const;

		UE_API FQuartzTransportTimeStamp GetCurrentClockTimestamp() const;

		UE_API float GetDurationOfQuantizationTypeInSeconds(const EQuartzCommandQuantization& QuantizationType, float Multiplier) const;

		UE_API float GetBeatProgressPercent(const EQuartzCommandQuantization& QuantizationType) const;

		// returns false if the clock is not valid or has shut down
		UE_API bool SendCommandToClock(TFunction<void(FQuartzClock*)> InCommand);

		// implicit cast to underlying ID (FName)
		operator const FName&() const { return ClockId; }

	private:
		FName ClockId;

		FQuartzClockCommandQueueWeakPtr SharedQueue;

	protected:
		TWeakPtr<FQuartzClock, ESPMode::ThreadSafe> ClockWeakPtr;

	}; // class FQuartzClockProxy


	
	/**
	 *	FQuartzClock:
	 *
	 *		This class receives, schedules, and fires quantized commands. 
	 *		The underlying FQuartzMetronome handles all counting / timing logic.
	 *
	 *		This class gets ticked externally (i.e. by some Clock Manager)
	 *		and counts down the time-to-fire the commands in audio frames.
	 *
	 *
	 *		UpdateCachedState() updates a game-thread copy of data accessed via FQuartzClockProxy
	 *		(see FQuartzClockState)
	 */

	class FQuartzClock : public FQuartzClockCommandQueueType::TConsumerBase<Audio::Quartz::IQuartzClock>
	{
	public:

		// ctor
		UE_API FQuartzClock(const FName& InName, const FQuartzClockSettings& InClockSettings, FQuartzClockManager* InOwningClockManagerPtr = nullptr);

		// dtor
		UE_API virtual ~FQuartzClock() override;

		// Transport Control:
		// alter the tick rate (take by-value to make sample-rate adjustments in-place)
		UE_API void ChangeTickRate(FQuartzClockTickRate InNewTickRate, int32 NumFramesLeft = 0);

		UE_API void ChangeTimeSignature(const FQuartzTimeSignature& InNewTimeSignature);

		UE_API virtual void Resume() override;

		UE_API virtual void Pause() override;

		UE_API virtual void Restart(bool bPause = true) override;

		UE_API virtual void Stop(bool CancelPendingEvents) override;  // Pause + Restart

		UE_API void SetSampleRate(float InNewSampleRate);

		UE_API void ResetTransport(const int32 NumFramesToTickBeforeReset = 0);

		// (used for StartOtherClock command to handle the sub-tick as the target clock)
		UE_API void AddToTickDelay(int32 NumFramesOfDelayToAdd);

		// (used for StartOtherClock command to handle the sub-tick as the target clock)
		UE_API void SetTickDelay(int32 NumFramesOfDelay);

		UE_API void Shutdown();

		// Getters:
		UE_API FQuartzClockTickRate GetTickRate();

		UE_API FName GetName() const;

		UE_API bool IgnoresFlush() const;

		UE_API bool DoesMatchSettings(const FQuartzClockSettings& InClockSettings) const;

		UE_API bool HasPendingEvents() const;

		UE_API int32 NumPendingEvents() const;

		UE_API bool IsRunning() const;

		UE_API float GetDurationOfQuantizationTypeInSeconds(const EQuartzCommandQuantization& QuantizationType, float Multiplier);

		UE_API float GetBeatProgressPercent(const EQuartzCommandQuantization& QuantizationType) const;

		UE_API FQuartzTransportTimeStamp GetCurrentTimestamp();

		UE_API float GetEstimatedRunTime();

		UE_API FMixerDevice* GetMixerDevice();

		UE_API FMixerSourceManager* GetSourceManager();

		UE_API FQuartzClockManager* GetClockManager();

		UE_API FQuartzClockCommandQueueWeakPtr GetCommandQueue() const;

		// Metronome Event Subscription:
		UE_API virtual void SubscribeToTimeDivision(FQuartzGameThreadSubscriber InSubscriber, EQuartzCommandQuantization InQuantizationBoundary) override;

		UE_API virtual void SubscribeToAllTimeDivisions(FQuartzGameThreadSubscriber InSubscriber) override;

		UE_API virtual void UnsubscribeFromTimeDivision(FQuartzGameThreadSubscriber InSubscriber, EQuartzCommandQuantization InQuantizationBoundary) override;

		UE_API virtual void UnsubscribeFromAllTimeDivisions(FQuartzGameThreadSubscriber InSubscriber) override;

		// Quantized Command Management:
		UE_API virtual void AddQuantizedCommand(FQuartzQuantizedRequestData& InQuantizedRequestData) override;
		UE_API virtual void AddQuantizedCommand(FQuartzQuantizedCommandInitInfo& InQuantizationCommandInitInfo) override;

		UE_API virtual void AddQuantizedCommand(FQuartzQuantizationBoundary InQuantizationBoundary, TSharedPtr<IQuartzQuantizedCommand> InNewEvent) override;

		UE_API bool CancelQuantizedCommand(TSharedPtr<IQuartzQuantizedCommand> InCommandPtr);
		
		// low-resolution clock update
		// (not sample-accurate!, useful when running without an Audio Device)
		UE_API void LowResolutionTick(float InDeltaTimeSeconds);

		// sample accurate clock update
		UE_API void Tick(int32 InNumFramesUntilNextTick);

	private:
		// Contains the pending command and the number of frames it has to wait to fire
		struct PendingCommand
		{
			// ctor
			PendingCommand(TSharedPtr<IQuartzQuantizedCommand> InCommand, int32 InNumFramesUntilExec)
				: Command(InCommand)
				, NumFramesUntilExec(InNumFramesUntilExec)
			{
			}

			// Quantized Command Object
			TSharedPtr<IQuartzQuantizedCommand> Command;

			// Countdown to execution
			int32 NumFramesUntilExec{ 0 };
		}; // struct PendingCommand

		// mutex-protected update at the end of Tick()
		FCriticalSection CachedClockStateCritSec;
		void UpdateCachedState();

		// data is cached when an FQuartzClock is ticked
		struct FQuartzClockState
		{
			FQuartzClockTickRate TickRate;
			FQuartzTransportTimeStamp TimeStamp;
			float RunTimeInSeconds;
			float MusicalDurationPhases[static_cast<int32>(EQuartzCommandQuantization::Count)] { 0 };
			float MusicalDurationPhaseDeltas[static_cast<int32>(EQuartzCommandQuantization::Count)] { 0 };
			uint64 LastCacheTickCpuCycles64 = 0;
			uint64 LastCacheTickDeltaCpuCycles64 = 0;
			
		} CachedClockState;

		void TickInternal(int32 InNumFramesUntilNextTick, TArray<PendingCommand>& CommandsToTick, int32 FramesOfLatency = 0, int32 FramesOfDelay = 0);

		bool CancelQuantizedCommandInternal(TSharedPtr<IQuartzQuantizedCommand> InCommandPtr, TArray<PendingCommand>& CommandsToTick);

		// don't allow default ctor, a clock needs to be ready to be used
		// by the clock manager / FMixerDevice once constructed
		FQuartzClock() = delete;

		FQuartzMetronome Metronome;

		FQuartzClockManager* OwningClockManagerPtr{ nullptr };

		FName Name;

		float ThreadLatencyInMilliseconds{ 40.f };

		// Command queue handed out to GameThread objects to queue commands. These get executed at the top of Tick()
		mutable FQuartzClockCommandQueuePtr PreTickCommands; // (mutable for lazy init in GetQuartzSubscriber())

		// Container of external commands to be executed (TUniquePointer<QuantizedAudioCommand>)
		TArray<PendingCommand> ClockAlteringPendingCommands;
		TArray<PendingCommand> PendingCommands;

		FThreadSafeBool bIsRunning{ true };

		bool bIgnoresFlush{ false };

		int32 TickDelayLengthInFrames{ 0 };

	}; // class FQuartzClock
} // namespace Audio

#undef UE_API
