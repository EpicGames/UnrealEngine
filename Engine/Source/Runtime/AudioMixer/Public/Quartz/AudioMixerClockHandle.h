// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Curves/RichCurve.h"
#include "Curves/CurveBase.h"
#include "Sound/QuartzQuantizationUtilities.h"
#include "Sound/QuartzSubscription.h"
#include "Quartz/QuartzSubsystem.h"
#include "Quartz/QuartzMetronome.h"

#include "AudioMixerClockHandle.generated.h"

/**
 *  This class is a BP / Game thread wrapper around FQuartzClockProxy
 *	(to talk to the underlying clock)
 
 *  ...and inherits from FQuartzTickableObject
 *	(to listen to the underlying clock)
 *  
 *  It can subscribe to Quantized Event & Metronome delegates to synchronize
 *  gameplay & VFX to Quartz events fired from the Audio Engine
 */
UCLASS(BlueprintType, Blueprintable, Transient, ClassGroup = Quartz, meta = (BlueprintSpawnableComponent), MinimalAPI)
class UQuartzClockHandle : public UObject, public FQuartzTickableObject
{
	GENERATED_BODY()

public:
	// ctor
	AUDIOMIXER_API UQuartzClockHandle();

	// dtor
	AUDIOMIXER_API ~UQuartzClockHandle();

	// begin UObject interface
	AUDIOMIXER_API void BeginDestroy() override;
	// end UObject interface

private:
	// begin FQuartzTickableObject interface
	AUDIOMIXER_API virtual bool ShouldUnsubscribe() override;
	// end FQuartzTickableObject interface
	
public:

// Begin Blueprint Interface

	// Clock manipulation
	UFUNCTION(BlueprintCallable, Category = "Quartz Clock", meta = (WorldContext = "WorldContextObject"))
	AUDIOMIXER_API void StartClock(const UObject* WorldContextObject, UQuartzClockHandle*& ClockHandle);

	UFUNCTION(BlueprintCallable, Category = "Quartz Subsystem", meta = (WorldContext = "WorldContextObject"))
	AUDIOMIXER_API void StopClock(const UObject* WorldContextObject, bool CancelPendingEvents, UQuartzClockHandle*& ClockHandle);

	UFUNCTION(BlueprintCallable, Category = "Quartz Subsystem", meta = (WorldContext = "WorldContextObject"))
	AUDIOMIXER_API void PauseClock(const UObject* WorldContextObject, UQuartzClockHandle*& ClockHandle);

	UFUNCTION(BlueprintCallable, Category = "Quartz Clock", meta = (WorldContext = "WorldContextObject"))
	AUDIOMIXER_API void ResumeClock(const UObject* WorldContextObject, UQuartzClockHandle*& ClockHandle);

	UFUNCTION(BlueprintCallable, Category = "Quartz Clock", meta = (WorldContext = "WorldContextObject", AutoCreateRefTerm = "InDelegate", Keywords = "Transport, Counter"))
	AUDIOMIXER_API void ResetTransport(const UObject* WorldContextObject, const FOnQuartzCommandEventBP& InDelegate);

	UFUNCTION(BlueprintCallable, Category = "Quartz Clock", meta = (WorldContext = "WorldContextObject", AutoCreateRefTerm = "InDelegate", Keywords = "Transport, Counter"))
	AUDIOMIXER_API void ResetTransportQuantized(const UObject* WorldContextObject, FQuartzQuantizationBoundary InQuantizationBoundary, const FOnQuartzCommandEventBP& InDelegate, UQuartzClockHandle*& ClockHandle);

	UFUNCTION(BlueprintCallable, Category = "Quartz Clock", meta = (WorldContext = "WorldContextObject", Keywords = "Transport, Counter"))
	AUDIOMIXER_API bool IsClockRunning(const UObject* WorldContextObject);

	UFUNCTION(BlueprintCallable, Category = "Quartz Clock", meta = (WorldContext = "WorldContextObject", AutoCreateRefTerm = "InDelegate", Keywords = "Transport, Counter"))
	AUDIOMIXER_API void NotifyOnQuantizationBoundary(const UObject* WorldContextObject, FQuartzQuantizationBoundary InQuantizationBoundary, const FOnQuartzCommandEventBP& InDelegate, float InMsOffset = 0.f);

	/** Returns the duration in seconds of the given Quantization Type
	 *
	 * @param The Quantization type to measure
	 * @param The quantity of the Quantization Type to calculate the time of
	 * @return The duration, in seconds, of a multiplier amount of the Quantization Type, or -1 in the case the clock is invalid
	 */
	UFUNCTION(BlueprintCallable, Category = "Quartz Clock", meta = (WorldContext = "WorldContextObject"))
	AUDIOMIXER_API float GetDurationOfQuantizationTypeInSeconds(const UObject* WorldContextObject, const EQuartzCommandQuantization& QuantizationType, float Multiplier = 1.0f);

	//Retrieves a timestamp for the clock
	UFUNCTION(BlueprintCallable, Category = "Quartz Clock Handle", meta = (WorldContext = "WorldContextObject"))
	AUDIOMIXER_API FQuartzTransportTimeStamp GetCurrentTimestamp(const UObject* WorldContextObject);

	// Returns the amount of time, in seconds, the clock has been running. Caution: due to latency, this will not be perfectly accurate
	UFUNCTION(BlueprintCallable, Category = "Quartz Clock Handle", meta = (WorldContext = "WorldContextObject"))
	AUDIOMIXER_API float GetEstimatedRunTime(const UObject* WorldContextObject);

	// "other" clock manipulation
	UFUNCTION(BlueprintCallable, Category = "Quartz Clock", meta = (WorldContext = "WorldContextObject", AutoCreateRefTerm = "InDelegate", Keywords = "Transport, Counter"))
	AUDIOMIXER_API void StartOtherClock(const UObject* WorldContextObject, FName OtherClockName, FQuartzQuantizationBoundary InQuantizationBoundary, const FOnQuartzCommandEventBP& InDelegate);

	// Metronome subscription
	UFUNCTION(BlueprintCallable, Category = "Quartz Clock", meta = (WorldContext = "WorldContextObject"))
	AUDIOMIXER_API void SubscribeToQuantizationEvent(const UObject* WorldContextObject, EQuartzCommandQuantization InQuantizationBoundary, const FOnQuartzMetronomeEventBP& OnQuantizationEvent, UQuartzClockHandle*& ClockHandle);

	UFUNCTION(BlueprintCallable, Category = "Quartz Clock", meta = (WorldContext = "WorldContextObject"))
	AUDIOMIXER_API void SubscribeToAllQuantizationEvents(const UObject* WorldContextObject, const FOnQuartzMetronomeEventBP& OnQuantizationEvent, UQuartzClockHandle*& ClockHandle);

	UFUNCTION(BlueprintCallable, Category = "Quartz Clock", meta = (WorldContext = "WorldContextObject"))
	AUDIOMIXER_API void UnsubscribeFromTimeDivision(const UObject* WorldContextObject, EQuartzCommandQuantization InQuantizationBoundary, UQuartzClockHandle*& ClockHandle);

	UFUNCTION(BlueprintCallable, Category = "Quartz Clock", meta = (WorldContext = "WorldContextObject"))
	AUDIOMIXER_API void UnsubscribeFromAllTimeDivisions(const UObject* WorldContextObject, UQuartzClockHandle*& ClockHandle);

	// Metronome Alteration (setters)
	UFUNCTION(BlueprintCallable, Category = "Quantization", meta = (WorldContext = "WorldContextObject", AdvancedDisplay = "QuantizationBoundary, Delegate", AutoCreateRefTerm = "QuantizationBoundary, Delegate", Keywords = "BPM, Tempo"))
	AUDIOMIXER_API void SetMillisecondsPerTick(const UObject* WorldContextObject, UPARAM(ref) const FQuartzQuantizationBoundary& QuantizationBoundary, const FOnQuartzCommandEventBP& Delegate, UQuartzClockHandle*& ClockHandle, float MillisecondsPerTick = 100.f);

	UFUNCTION(BlueprintCallable, Category = "Quantization", meta = (WorldContext = "WorldContextObject", AdvancedDisplay = "QuantizationBoundary, Delegate", AutoCreateRefTerm = "QuantizationBoundary, Delegate", Keywords = "BPM, Tempo"))
	AUDIOMIXER_API void SetTicksPerSecond(const UObject* WorldContextObject, UPARAM(ref) const FQuartzQuantizationBoundary& QuantizationBoundary, const FOnQuartzCommandEventBP& Delegate, UQuartzClockHandle*& ClockHandle, float TicksPerSecond = 10.f);

	UFUNCTION(BlueprintCallable, Category = "Quantization", meta = (WorldContext = "WorldContextObject", AdvancedDisplay = "QuantizationBoundary, Delegate", AutoCreateRefTerm = "QuantizationBoundary, Delegate", Keywords = "BPM, Tempo"))
	AUDIOMIXER_API void SetSecondsPerTick(const UObject* WorldContextObject, UPARAM(ref) const FQuartzQuantizationBoundary& QuantizationBoundary, const FOnQuartzCommandEventBP& Delegate, UQuartzClockHandle*& ClockHandle, float SecondsPerTick = 0.25f);

	UFUNCTION(BlueprintCallable, Category = "Quantization", meta = (WorldContext = "WorldContextObject", AdvancedDisplay = "QuantizationBoundary, Delegate", AutoCreateRefTerm = "QuantizationBoundary, Delegate", Keywords = "BPM, Tempo"))
	AUDIOMIXER_API void SetThirtySecondNotesPerMinute(const UObject* WorldContextObject, UPARAM(ref) const FQuartzQuantizationBoundary& QuantizationBoundary, const FOnQuartzCommandEventBP& Delegate, UQuartzClockHandle*& ClockHandle, float ThirtySecondsNotesPerMinute = 960.f);

	UFUNCTION(BlueprintCallable, Category = "Quantization", meta = (WorldContext = "WorldContextObject", AdvancedDisplay = "QuantizationBoundary, Delegate", AutoCreateRefTerm = "QuantizationBoundary, Delegate", Keywords = "BPM, Tempo"))
	AUDIOMIXER_API void SetBeatsPerMinute(const UObject* WorldContextObject, UPARAM(ref) const FQuartzQuantizationBoundary& QuantizationBoundary, const FOnQuartzCommandEventBP& Delegate, UQuartzClockHandle*& ClockHandle, float BeatsPerMinute = 60.f);

	// Metronome getters
	UFUNCTION(BlueprintCallable, Category = "Quantization", meta = (WorldContext = "WorldContextObject", AutoCreateRefTerm = "InDelegate", Keywords = "BPM, Tempo"))
	AUDIOMIXER_API float GetMillisecondsPerTick(const UObject* WorldContextObject) const;

	UFUNCTION(BlueprintCallable, Category = "Quantization", meta = (WorldContext = "WorldContextObject", AutoCreateRefTerm = "InDelegate", Keywords = "BPM, Tempo"))
	AUDIOMIXER_API float GetTicksPerSecond(const UObject* WorldContextObject) const;

	UFUNCTION(BlueprintCallable, Category = "Quantization", meta = (WorldContext = "WorldContextObject", AutoCreateRefTerm = "InDelegate", Keywords = "BPM, Tempo"))
	AUDIOMIXER_API float GetSecondsPerTick(const UObject* WorldContextObject) const;

	UFUNCTION(BlueprintCallable, Category = "Quantization", meta = (WorldContext = "WorldContextObject", AutoCreateRefTerm = "InDelegate", Keywords = "BPM, Tempo"))
	AUDIOMIXER_API float GetThirtySecondNotesPerMinute(const UObject* WorldContextObject) const;

	UFUNCTION(BlueprintCallable, Category = "Quantization", meta = (WorldContext = "WorldContextObject", AutoCreateRefTerm = "InDelegate", Keywords = "BPM, Tempo"))
	AUDIOMIXER_API float GetBeatsPerMinute(const UObject* WorldContextObject) const;

	/**
	 * Returns the current progress until the next occurrence of the provided musical duration as a float value from 0 (previous beat) to 1 (next beat).
	 * This is useful for indexing into curves to animate parameters to musical time.
	 * Ms and Phase offsets are combined internally.
	 */
	UFUNCTION(BlueprintCallable, Category = "Quantization", meta = ( AutoCreateRefTerm = "PhaseOffset", Keywords = "BPM, Tempo"))
	AUDIOMIXER_API float GetBeatProgressPercent(EQuartzCommandQuantization QuantizationBoundary = EQuartzCommandQuantization::Beat, float PhaseOffset = 0.f, float MsOffset = 0.f);

	// todo: un-comment when metronome events support the offset
	// Set how early we would like to receive Metronome (not yet supported) and "About To Start" Delegates. (all other command delegates will execute as normal)
	// UFUNCTION(BlueprintCallable, Category = "Quantization", meta = (WorldContext = "WorldContextObject", AutoCreateRefTerm = "InDelegate", Keywords = "BPM, Tempo"))
	// void SetNotificationAnticipationAmountInMilliseconds(const UObject* WorldContextObject, UQuartzClockHandle*& ClockHandle, const double Milliseconds = 0.0);

	// // Set how early we would like to receive Metronome (not yet supported) and "About To Start" Delegates. (all other command delegates will execute as normal)
	// UFUNCTION(BlueprintCallable, Category = "Quantization", meta = (WorldContext = "WorldContextObject", Keywords = "BPM, Tempo"))
	// void SetNotificationAnticipationAmountAsMusicalDuration(const UObject* WorldContextObject, UQuartzClockHandle*& ClockHandle, const EQuartzCommandQuantization MusicalDuration = EQuartzCommandQuantization::QuarterNote, const double Multiplier = 1.0);

// End Blueprint Interface
	AUDIOMIXER_API void QueueQuantizedSound(const UObject* WorldContextObject, UQuartzClockHandle*& ClockHandle, const FAudioComponentCommandInfo& AudioComponentData, const FOnQuartzCommandEventBP& InDelegate, const FQuartzQuantizationBoundary& InTargetBoundary);

	AUDIOMIXER_API UQuartzClockHandle* SubscribeToClock(const UObject* WorldContextObject, FName ClockName, Audio::FQuartzClockProxy const* InHandlePtr = nullptr);

	FName GetClockName() const { return CurrentClockId; }

	bool DoesClockExist(const UObject* WorldContextObject) const
	{
		return RawHandle.DoesClockExist();
	}

	AUDIOMIXER_API bool GetCurrentTickRate(const UObject* WorldContextObject, Audio::FQuartzClockTickRate& OutTickRate) const;

private:
	AUDIOMIXER_API void SetTickRateInternal(const FQuartzQuantizationBoundary& InQuantizationBoundary, const FOnQuartzCommandEventBP& InDelegate, const Audio::FQuartzClockTickRate& NewTickRate);

	Audio::FQuartzClockProxy RawHandle;

	FName CurrentClockId;

}; // class UQuartzClockHandle
