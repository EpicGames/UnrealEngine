// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioMixerClock.h"
#include "Sound/QuartzQuantizationUtilities.h"

namespace Audio
{
	// forwards
	class FMixerDevice;
	class FQuartzClock;
	class FQuartzClockManager;

	// Class that owns, updates, and provides access to all active clocks
	// All methods are thread-safe. The method locks if it returns a value, and stages a command if it returns void
	class FQuartzClockManager : public FQuartLatencyTracker
	{
	public:
		// ctor
		AUDIOMIXER_API FQuartzClockManager(Audio::FMixerDevice* InOwner = nullptr);

		// dtor
		AUDIOMIXER_API ~FQuartzClockManager();

		int32 GetNumClocks() const { return ActiveClocks.Num(); }

		// Called on AudioRenderThread
		AUDIOMIXER_API void Update(int32 NumFramesUntilNextUpdate);
		AUDIOMIXER_API void UpdateClock(FName InClockToAdvance, int32 NumFramesToAdvance);

		// can be called from any thread for low-resolution clock updates
		// (i.e. used when running without an audio device)
		// not sample-accurate!
		AUDIOMIXER_API void LowResoultionUpdate(float DeltaTimeSeconds);

		// add (and take ownership of) a new clock
		// safe to call from AudioThread (uses critical section)
		AUDIOMIXER_API FQuartzClockProxy GetOrCreateClock(const FName& InClockName, const FQuartzClockSettings& InClockSettings, bool bOverrideTickRateIfClockExists = false);
		AUDIOMIXER_API FQuartzClockProxy GetClock(const FName& InClockName);

		// returns true if a clock with the given name already exists.
		AUDIOMIXER_API bool DoesClockExist(const FName& InClockName);

		// returns true if the name is running
		AUDIOMIXER_API bool IsClockRunning(const FName& InClockName);

		// Returns the duration in seconds of the given Quantization Type, or -1 if the Clock is invalid or nonexistent
		AUDIOMIXER_API float GetDurationOfQuantizationTypeInSeconds(const FName& InClockName, const EQuartzCommandQuantization& QuantizationType, float Multiplier);

		// Returns the current location of the clock in the transport
		AUDIOMIXER_API FQuartzTransportTimeStamp GetCurrentTimestamp(const FName& InClockName);

		// Returns the amount of time, in seconds, the clock has been running. Caution: due to latency, this will not be perfectly accurate
		AUDIOMIXER_API float GetEstimatedRunTime(const FName& InClockName);

		// remove existing clock
		// safe to call from AudioThread (uses Audio Render Thread Command)
		AUDIOMIXER_API void RemoveClock(const FName& InName, bool bForceSynchronous = false);

		// get Tick rate for clock
		// safe to call from AudioThread (uses critical section)
		AUDIOMIXER_API FQuartzClockTickRate GetTickRateForClock(const FName& InName);

		AUDIOMIXER_API void SetTickRateForClock(const FQuartzClockTickRate& InNewTickRate, const FName& InName);

		// start the given clock
		// safe to call from AudioThread (uses Audio Render Thread command)
		AUDIOMIXER_API void ResumeClock(const FName& InName, int32 NumFramesToDelayStart = 0);

		// stop the given clock
		// safe to call from AudioThread (uses Audio Render Thread command)
		AUDIOMIXER_API void StopClock(const FName& InName, bool CancelPendingEvents);

		// stop the given clock
		// safe to call from AudioThread (uses Audio Render Thread command)
		AUDIOMIXER_API void PauseClock(const FName& InName);

		// shutdown all clocks that don't ignore Flush() (i.e. level change)
		AUDIOMIXER_API void Flush();

		// stop all clocks and cancel all pending events
		AUDIOMIXER_API void Shutdown();

		// add a new command to a given clock
		// safe to call from AudioThread (uses Audio Render Thread command)
		AUDIOMIXER_API FQuartzQuantizedCommandHandle AddCommandToClock(FQuartzQuantizedCommandInitInfo& InQuantizationCommandInitInfo);

		// subscribe to a specific time division on a clock
		// TODO: update the metronome subscription functions to take an FQuartzGameThreadSubscriber instead of the Command queue ptr
		// (to support metronome event offset)
		AUDIOMIXER_API void SubscribeToTimeDivision(FName InClockName, MetronomeCommandQueuePtr InListenerQueue, EQuartzCommandQuantization InQuantizationBoundary);

		// subscribe to all time divisions on a clock
		AUDIOMIXER_API void SubscribeToAllTimeDivisions(FName InClockName, MetronomeCommandQueuePtr InListenerQueue);

		// un-subscribe from a specific time division on a clock
		AUDIOMIXER_API void UnsubscribeFromTimeDivision(FName InClockName, MetronomeCommandQueuePtr InListenerQueue, EQuartzCommandQuantization InQuantizationBoundary);

		// un-subscribe from all time divisions on a specific clock
		AUDIOMIXER_API void UnsubscribeFromAllTimeDivisions(FName InClockName, MetronomeCommandQueuePtr InListenerQueue);

		// cancel a queued command on a clock (i.e. cancel a PlayQuantized command if the sound is stopped before it is played)
		AUDIOMIXER_API bool CancelCommandOnClock(FName InOwningClockName, TSharedPtr<IQuartzQuantizedCommand> InCommandPtr);

		AUDIOMIXER_API bool HasClockBeenTickedThisUpdate(FName InClockName);

		int32 GetLastUpdateSizeInFrames() const { return LastUpdateSizeInFrames; }

		// get access to the owning FMixerDevice
		AUDIOMIXER_API FMixerDevice* GetMixerDevice() const;

	private:
		// updates all active clocks
		AUDIOMIXER_API void TickClocks(int32 NumFramesToTick);

		// find clock with a given key
		AUDIOMIXER_API TSharedPtr<FQuartzClock> FindClock(const FName& InName);

		// pointer to owning FMixerDevice
		FMixerDevice* MixerDevice;

		// Container of active clocks
		FCriticalSection ActiveClockCritSec;

		// Our array of active clocks (mutation/access acquires clock)
		TArray<TSharedPtr<FQuartzClock>> ActiveClocks;

		FThreadSafeCounter LastClockTickedIndex{ 0 };
		int32 LastUpdateSizeInFrames{ 0 };

		// allow a clock that is queuing a command directly use FindClock() to retrieve the TSharedPtr<FQuartzClock>
		friend void FQuartzClock::AddQuantizedCommand(FQuartzQuantizedCommandInitInfo& InQuantizationCommandInitInfo);
	};

	// data that the UQuartzSubsystem needs to persist on the AudioDevice across UWorld shutdown/startup
	struct FPersistentQuartzSubsystemData
	{
		// internal clock manager for game-thread-ticked clocks
		FQuartzClockManager SubsystemClockManager;

		// array of active clock handles (update FindProxyByName() if more are added later)
		TArray<Audio::FQuartzClockProxy> ActiveExternalClockProxies;
		TArray<Audio::FQuartzClockProxy> ActiveAudioMixerClockProxies;
	};
} // namespace Audio
