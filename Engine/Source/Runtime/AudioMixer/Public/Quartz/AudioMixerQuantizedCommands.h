// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioMixerClock.h"

namespace Audio
{
	// QuartzQuantizedCommand that plays a sound on a sample-accurate boundary
	class FQuantizedPlayCommand : public IQuartzQuantizedCommand
	{
	public:
		// ctor
		AUDIOMIXER_API FQuantizedPlayCommand();

		// dtor
		~FQuantizedPlayCommand() = default;

		AUDIOMIXER_API virtual TSharedPtr<IQuartzQuantizedCommand> GetDeepCopyOfDerivedObject() const override;

		AUDIOMIXER_API virtual void OnQueuedCustom(const FQuartzQuantizedCommandInitInfo& InCommandInitInfo) override;

		AUDIOMIXER_API virtual void OnFinalCallbackCustom(int32 InNumFramesLeft) override;

		AUDIOMIXER_API virtual void CancelCustom() override;

		virtual bool RequiresAudioDevice() const override { return true; }

		AUDIOMIXER_API virtual FName GetCommandName() const override;
		
		// for your implementation, a new EQuartzCommandType needs to be defined in QuartzQuantizationUtilities.h
		virtual EQuartzCommandType GetCommandType() const { return EQuartzCommandType::PlaySound; };

		virtual int32 OverrideFramesUntilExec(int32 NumFramesUntilExec);

	protected:
		TSharedPtr<FQuartzClock> OwningClockPtr{ nullptr };

		int32 SourceID{ -1 };

		bool bIsCanceled = false;

	}; // class FQuantizedPlayCommand 


	class FQuantizedQueueCommand : public IQuartzQuantizedCommand
	{
	public:
		AUDIOMIXER_API virtual TSharedPtr<IQuartzQuantizedCommand> GetDeepCopyOfDerivedObject() const override;

		AUDIOMIXER_API virtual void OnQueuedCustom(const FQuartzQuantizedCommandInitInfo& InCommandInitInfo) override;

		AUDIOMIXER_API virtual int32 OverrideFramesUntilExec(int32 NumFramesUntilExec) override;

		AUDIOMIXER_API virtual void OnFinalCallbackCustom(int32 InNumFramesLeft) override;

		virtual bool RequiresAudioDevice() const override { return true; }

		AUDIOMIXER_API virtual FName GetCommandName() const override;
		virtual EQuartzCommandType GetCommandType() const { return EQuartzCommandType::QueueSoundToPlay; };

		AUDIOMIXER_API void SetQueueCommand(const FAudioComponentCommandInfo& InAudioCommandData);

		FQuantizedQueueCommand() {}

	private:
		TSharedPtr<FQuartzClock> OwningClockPtr{ nullptr };

		//Data for the quantization event
		FAudioComponentCommandInfo AudioComponentData;
	}; // class FQuantizedQueueCommand 
	
	// QuartzQuantizedCommand that changes the TickRate of a clock on a sample-accurate boundary (i.e. BPM changes)
	class FQuantizedTickRateChange : public IQuartzQuantizedCommand
	{
	public:
		void SetTickRate(const FQuartzClockTickRate& InTickRate)
		{
			TickRate = InTickRate;
		}

		AUDIOMIXER_API virtual TSharedPtr<IQuartzQuantizedCommand> GetDeepCopyOfDerivedObject() const override;

		AUDIOMIXER_API virtual void OnQueuedCustom(const FQuartzQuantizedCommandInitInfo& InCommandInitInfo) override;

		AUDIOMIXER_API virtual void OnFinalCallbackCustom(int32 InNumFramesLeft) override;

		virtual bool IsClockAltering() override { return true; }

		AUDIOMIXER_API virtual FName GetCommandName() const override;
		virtual EQuartzCommandType GetCommandType() const { return EQuartzCommandType::TickRateChange; };

	private:
		FQuartzClockTickRate TickRate;
		TSharedPtr<FQuartzClock> OwningClockPtr{ nullptr };

	}; // class FQuantizedTickRateChange 


	// QuartzQuantizedCommand that resets the transport of a clock's metronome on a sample-accurate boundary
	class FQuantizedTransportReset : public IQuartzQuantizedCommand
	{
	public:
		AUDIOMIXER_API virtual TSharedPtr<IQuartzQuantizedCommand> GetDeepCopyOfDerivedObject() const override;

		AUDIOMIXER_API virtual void OnQueuedCustom(const FQuartzQuantizedCommandInitInfo& InCommandInitInfo) override;

		AUDIOMIXER_API virtual void OnFinalCallbackCustom(int32 InNumFramesLeft) override;

		virtual bool IsClockAltering() override { return true; }

		AUDIOMIXER_API virtual FName GetCommandName() const override;
		virtual EQuartzCommandType GetCommandType() const { return EQuartzCommandType::TransportReset; };

	private:
		TSharedPtr<FQuartzClock> OwningClockPtr{ nullptr };

	}; // class FQuantizedTransportReset 


	// QuartzQuantizedCommand that starts a second clock on a sample-accurate boundary
	class FQuantizedOtherClockStart : public IQuartzQuantizedCommand
	{
	public:
		AUDIOMIXER_API virtual TSharedPtr<IQuartzQuantizedCommand> GetDeepCopyOfDerivedObject() const override;

		AUDIOMIXER_API virtual void OnQueuedCustom(const FQuartzQuantizedCommandInitInfo& InCommandInitInfo) override;

		AUDIOMIXER_API virtual void OnFinalCallbackCustom(int32 InNumFramesLeft) override;

		virtual bool IsClockAltering() override { return true; }

		AUDIOMIXER_API virtual FName GetCommandName() const override;
		virtual EQuartzCommandType GetCommandType() const { return EQuartzCommandType::StartOtherClock; };

	private:
		TSharedPtr<FQuartzClock> OwningClockPtr{ nullptr };
		FName NameOfClockToStart;

	}; // class FQuantizedOtherClockStart


	// QuartzQuantizedCommand that basically no-ops, so the game thread can get notified on a musical boundary
	class FQuantizedNotify : public IQuartzQuantizedCommand
	{
	public:
		// ctor
		AUDIOMIXER_API FQuantizedNotify(float InMsOffset = 0.f);

		// dtor
		virtual ~FQuantizedNotify() override = default;

		AUDIOMIXER_API virtual TSharedPtr<IQuartzQuantizedCommand> GetDeepCopyOfDerivedObject() const override;

		virtual bool RequiresAudioDevice() const override { return true; }

		AUDIOMIXER_API virtual FName GetCommandName() const override;

		virtual EQuartzCommandType GetCommandType() const override { return EQuartzCommandType::Notify; };

		AUDIOMIXER_API virtual void OnQueuedCustom(const FQuartzQuantizedCommandInitInfo& InCommandInitInfo) override;

		AUDIOMIXER_API virtual int32 OverrideFramesUntilExec(int32 NumFramesUntilExec) override;

	protected:
		TSharedPtr<FQuartzClock> OwningClockPtr{ nullptr };
		float OffsetInMs = 0.f;
		float SampleRate = 0.f;
		bool bIsCanceled = false;

	}; // class FQuantizedNotify

} // namespace Audio
