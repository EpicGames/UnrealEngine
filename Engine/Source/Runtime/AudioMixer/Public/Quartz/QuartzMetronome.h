// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "Sound/QuartzQuantizationUtilities.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class FQuartzTickableObject;

namespace Audio
{
	using MetronomeCommandQueuePtr = Quartz::FQuartzGameThreadCommandQueuePtr; // todo: rename, "GameThread is not accurate"

	// Class to track the passage of musical time, and allow subscribers to be notified when these musical events take place
	class FQuartzMetronome
	{
	public:
		// ctor
		FQuartzMetronome(FName InClockName = {});
		FQuartzMetronome(const FQuartzTimeSignature& InTimeSignature, FName InClockName = {});

		// dtor
		~FQuartzMetronome();

		// Transport Control:
		void Tick(int32 InNumSamples, int32 FramesOfLatency = 0);

		void SetTickRate(FQuartzClockTickRate InNewTickRate, int32 NumFramesLeft = 0);

		void SetSampleRate(float InNewSampleRate);

		void SetTimeSignature(const FQuartzTimeSignature& InNewTimeSignature);

		void ResetTransport();

		// Getters
		const FQuartzClockTickRate& GetTickRate() const { return CurrentTickRate; }

		double GetFramesUntilBoundary(FQuartzQuantizationBoundary InQuantizationBoundary) const;

		const FQuartzTimeSignature & GetTimeSignature() const { return CurrentTimeSignature; }

		FQuartzTransportTimeStamp GetTimeStamp() const { return CurrentTimeStamp; }

		double GetTimeSinceStart() const { return TimeSinceStart; }

		uint64 GetLastTickCpuCycles64() const { return LastTickCpuCycles64; }

		void CalculateDurationPhases(float (&OutPhases)[static_cast<int32>(EQuartzCommandQuantization::Count)]) const;

		// Event Subscription
		void SubscribeToTimeDivision(MetronomeCommandQueuePtr InListenerQueue, EQuartzCommandQuantization InQuantizationBoundary);

		void SubscribeToAllTimeDivisions(MetronomeCommandQueuePtr InListenerQueue);

		void UnsubscribeFromTimeDivision(MetronomeCommandQueuePtr InListenerQueue, EQuartzCommandQuantization InQuantizationBoundary);

		void UnsubscribeFromAllTimeDivisions(MetronomeCommandQueuePtr InListenerQueue);


	private:

		// Helpers:
		void RecalculateDurations();

		void FireEvents();

		float CountNumSubdivisionsPerBar(EQuartzCommandQuantization InSubdivision) const;

		float CountNumSubdivisionsSinceBarStart(EQuartzCommandQuantization InSubdivision) const;

		float CountNumSubdivisionsSinceStart(EQuartzCommandQuantization InSubdivision) const;

		uint64 LastTickCpuCycles64{ 0 };

		int32 ListenerFlags{ 0 };

		FQuartzTransportTimeStamp CurrentTimeStamp;

		FQuartzTimeSignature CurrentTimeSignature;

		FQuartzClockTickRate CurrentTickRate;

		TArray<MetronomeCommandQueuePtr> MetronomeSubscriptionMatrix[static_cast<int32>(EQuartzCommandQuantization::Count)];
		
		// wrapper around our array so it can be indexed into by different Enums that represent musical time
		struct FFramesInTimeValue
		{
			void Reset();
			
			double& operator[](EQuartzCommandQuantization InTimeValue);
			const double& operator[](EQuartzCommandQuantization InTimeValue) const;
			double& operator[](int32 Index);
			const double& operator[](int32 Index) const;

			double FramesInTimeValueInternal[static_cast<int32>(EQuartzCommandQuantization::Count)]{ 0.0 };
		};

		struct FMetronomeEventEntry
		{
			TArray<int32> EventFrames;
			void Reset();
		};

		struct FPendingMetronomeEvents
		{
			void Add(const EQuartzCommandQuantization InDuration, const int32 InFrame);
			void Reset();
			bool HasPendingEvent(const EQuartzCommandQuantization InDuration) const;
			
			FMetronomeEventEntry CurrentMetronomeEvents[static_cast<int32>(EQuartzCommandQuantization::Count)];
		} PendingMetronomeEvents;

		// array of lengths of musical durations (in audio frames)
		FFramesInTimeValue MusicalDurationsInFrames;

		// array of the number of audio frames left until the respective musical duration
		FFramesInTimeValue FramesLeftInMusicalDuration;

		// optional array of pulse duration overrides (for odd meters)
		TArray<double> PulseDurations;

		// the index of the active pulse duration override
		int32 PulseDurationIndex{ -1 };

		int32 LastFramesOfLatency{ 0 };

		//Keeps track of time in seconds since the Clock was last reset
		double TimeSinceStart;

		FName ClockName;

	}; // class QuartzMetronome
} // namespace Audio