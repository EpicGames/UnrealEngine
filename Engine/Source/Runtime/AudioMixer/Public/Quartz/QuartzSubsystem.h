// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Subsystems/WorldSubsystem.h"
#include "Quartz/AudioMixerClockManager.h"
#include "Sound/QuartzQuantizationUtilities.h"

#include "QuartzSubsystem.generated.h"


// forwards
namespace Audio
{
	class FMixerDevice;
	class FQuartzClockManager;
	class FQuartzShareableCommandQueue;

}

class FQuartzTickableObject;
class UQuartzClockHandle;
using MetronomeCommandQueuePtr = TSharedPtr<Audio::FQuartzShareableCommandQueue, ESPMode::ThreadSafe>;



struct FQuartzTickableObjectsManager : public FQuartLatencyTracker
{
public:
	AUDIOMIXER_API void Tick(float DeltaTime);
	AUDIOMIXER_API bool IsTickable() const;
	AUDIOMIXER_API void SubscribeToQuartzTick(FQuartzTickableObject* InObjectToTick);
	AUDIOMIXER_API void UnsubscribeFromQuartzTick(FQuartzTickableObject* InObjectToTick);

private:
	// list of objects needing to be ticked by Quartz
	TArray<FQuartzTickableObject *> QuartzTickSubscribers;

	// index to track the next subscriber to tick (if updates are being amortized across multiple UObject Ticks)
	int32 UpdateIndex{ 0 };
};


UCLASS(DisplayName = "Quartz", MinimalAPI)
class UQuartzSubsystem : public UTickableWorldSubsystem
{
	GENERATED_BODY()

public:
	// ctor/dtor
	UQuartzSubsystem() = default;
	virtual ~UQuartzSubsystem() override = default;

	//~ Begin UWorldSubsystem Interface
	AUDIOMIXER_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	AUDIOMIXER_API virtual void Deinitialize() override;
	AUDIOMIXER_API virtual bool DoesSupportWorldType(EWorldType::Type WorldType) const override;
	AUDIOMIXER_API void virtual BeginDestroy() override;
	//~ End UWorldSubsystem Interface

	//~ Begin FTickableGameObject Interface
	AUDIOMIXER_API virtual void Tick(float DeltaTime) override;
	AUDIOMIXER_API virtual bool IsTickableWhenPaused() const override;
	AUDIOMIXER_API virtual bool IsTickable() const override;
	AUDIOMIXER_API virtual TStatId GetStatId() const override;
	//~ End FTickableGameObject Interface

	// these calls are forwarded to the internal FQuartzTickableObjectsManager
	AUDIOMIXER_API void SubscribeToQuartzTick(FQuartzTickableObject* InObjectToTick);
	AUDIOMIXER_API void UnsubscribeFromQuartzTick(FQuartzTickableObject* InObjectToTick);

	// get C++ handle (proxy) to a clock if it exists
	AUDIOMIXER_API Audio::FQuartzClockProxy GetProxyForClock(FName ClockName) const;

	// allow an external clock (not ticked by the Audio Mixer or QuartzSubsystem) to be accessible via this subsystem
	AUDIOMIXER_API void AddProxyForExternalClock(const Audio::FQuartzClockProxy& InProxy);

	// static methods
	static AUDIOMIXER_API UQuartzSubsystem* Get(const UWorld* const World);

	// Helper functions for initializing quantized command initialization struct (to consolidate eyesore)
	static AUDIOMIXER_API Audio::FQuartzQuantizedRequestData CreateRequestDataForTickRateChange(UQuartzClockHandle* InClockHandle, const FOnQuartzCommandEventBP& InDelegate, const Audio::FQuartzClockTickRate& InNewTickRate, const FQuartzQuantizationBoundary& InQuantizationBoundary);
	static AUDIOMIXER_API Audio::FQuartzQuantizedRequestData CreateRequestDataForTransportReset(UQuartzClockHandle* InClockHandle, const FQuartzQuantizationBoundary& InQuantizationBoundary, const FOnQuartzCommandEventBP& InDelegate);
	static AUDIOMIXER_API Audio::FQuartzQuantizedRequestData CreateRequestDataForStartOtherClock(UQuartzClockHandle* InClockHandle, FName InClockToStart, const FQuartzQuantizationBoundary& InQuantizationBoundary, const FOnQuartzCommandEventBP& InDelegate);
	static AUDIOMIXER_API Audio::FQuartzQuantizedRequestData CreateRequestDataForSchedulePlaySound(UQuartzClockHandle* InClockHandle, const FOnQuartzCommandEventBP& InDelegate, const FQuartzQuantizationBoundary& InQuantizationBoundary);
	static AUDIOMIXER_API Audio::FQuartzQuantizedRequestData CreateRequestDataForQuantizedNotify(UQuartzClockHandle* InClockHandle, const FQuartzQuantizationBoundary& InQuantizationBoundary, const FOnQuartzCommandEventBP& InDelegate, float InMsOffset = 0.f);

	UFUNCTION(BlueprintCallable, Category = "Quartz Clock Handle")
	AUDIOMIXER_API bool IsQuartzEnabled();

	// Clock Creation
	// create a new clock (or return handle if clock already exists)
	UFUNCTION(BlueprintCallable, Category = "Quartz Clock Handle", meta = (WorldContext = "WorldContextObject", AdvancedDisplay = "bUseAudioEngineClockManager"))
	AUDIOMIXER_API UQuartzClockHandle* CreateNewClock(const UObject* WorldContextObject, FName ClockName, FQuartzClockSettings InSettings, bool bOverrideSettingsIfClockExists = false, bool bUseAudioEngineClockManager = true);

	// delete an existing clock given its name
	UFUNCTION(BlueprintCallable, Category = "Quartz Clock Handle", meta = (WorldContext = "WorldContextObject"))
	AUDIOMIXER_API void DeleteClockByName(const UObject* WorldContextObject, FName ClockName);

	// delete an existing clock given its clock handle
	UFUNCTION(BlueprintCallable, Category = "Quartz Clock Handle", meta = (WorldContext = "WorldContextObject"))
	AUDIOMIXER_API void DeleteClockByHandle(const UObject* WorldContextObject, UPARAM(ref) UQuartzClockHandle*& InClockHandle);

	// get handle for existing clock
	UFUNCTION(BlueprintCallable, Category = "Quartz Clock Handle", meta = (WorldContext = "WorldContextObject"))
	AUDIOMIXER_API UQuartzClockHandle* GetHandleForClock(const UObject* WorldContextObject, FName ClockName);

	// returns true if the clock exists
	UFUNCTION(BlueprintCallable, Category = "Quartz Clock Handle", meta = (WorldContext = "WorldContextObject"))
	AUDIOMIXER_API bool DoesClockExist(const UObject* WorldContextObject, FName ClockName);

	// returns true if the clock is running
	UFUNCTION(BlueprintCallable, Category = "Quartz Clock Handle", meta = (WorldContext = "WorldContextObject"))
	AUDIOMIXER_API bool IsClockRunning(const UObject* WorldContextObject, FName ClockName);

	// Returns the duration in seconds of the given Quantization Type
	UFUNCTION(BlueprintCallable, Category = "Quartz Clock Handle", meta = (WorldContext = "WorldContextObject"))
	AUDIOMIXER_API float GetDurationOfQuantizationTypeInSeconds(const UObject* WorldContextObject, FName ClockName, const EQuartzCommandQuantization& QuantizationType, float Multiplier = 1.0f);

	// Retrieves a timestamp for the clock
	UFUNCTION(BlueprintCallable, Category = "Quartz Clock Handle", meta = (WorldContext = "WorldContextObject"))
	AUDIOMIXER_API FQuartzTransportTimeStamp GetCurrentClockTimestamp(const UObject* WorldContextObject, const FName& InClockName);

	// Returns the amount of time, in seconds, the clock has been running. Caution: due to latency, this will not be perfectly accurate
	UFUNCTION(BlueprintCallable, Category = "Quartz Clock Handle", meta = (WorldContext = "WorldContextObject"))
	AUDIOMIXER_API float GetEstimatedClockRunTime(const UObject* WorldContextObject, const FName& InClockName);

	// latency data (Game thread -> Audio Render Thread)
	UFUNCTION(BlueprintCallable, Category = "Quartz Subsystem", meta = (WorldContext = "WorldContextObject"))
	AUDIOMIXER_API float GetGameThreadToAudioRenderThreadAverageLatency(const UObject* WorldContextObject);

	UFUNCTION(BlueprintCallable, Category = "Quartz Subsystem", meta = (WorldContext = "WorldContextObject"))
	AUDIOMIXER_API float GetGameThreadToAudioRenderThreadMinLatency(const UObject* WorldContextObject);

	UFUNCTION(BlueprintCallable, Category = "Quartz Subsystem", meta = (WorldContext = "WorldContextObject"))
	AUDIOMIXER_API float GetGameThreadToAudioRenderThreadMaxLatency(const UObject* WorldContextObject);

	// latency data (Audio Render Thread -> Game thread)
	UFUNCTION(BlueprintCallable, Category = "Quartz Subsystem")
	AUDIOMIXER_API float GetAudioRenderThreadToGameThreadAverageLatency();

	UFUNCTION(BlueprintCallable, Category = "Quartz Subsystem")
	AUDIOMIXER_API float GetAudioRenderThreadToGameThreadMinLatency();

	UFUNCTION(BlueprintCallable, Category = "Quartz Subsystem")
	AUDIOMIXER_API float GetAudioRenderThreadToGameThreadMaxLatency();

	// latency data (Round trip)
	UFUNCTION(BlueprintCallable, Category = "Quartz Subsystem", meta = (WorldContext = "WorldContextObject"))
	AUDIOMIXER_API float GetRoundTripAverageLatency(const UObject* WorldContextObject);

	UFUNCTION(BlueprintCallable, Category = "Quartz Subsystem", meta = (WorldContext = "WorldContextObject"))
	AUDIOMIXER_API float GetRoundTripMinLatency(const UObject* WorldContextObject);

	UFUNCTION(BlueprintCallable, Category = "Quartz Subsystem", meta = (WorldContext = "WorldContextObject"))
	AUDIOMIXER_API float GetRoundTripMaxLatency(const UObject* WorldContextObject);
	
	UFUNCTION(BlueprintCallable, Category = "Quartz Subsystem")
	AUDIOMIXER_API void SetQuartzSubsystemTickableWhenPaused(const bool bInTickableWhenPaused);

	// sharable to allow non-UObjects to un-subscribe if the Subsystem is going to outlive them
	AUDIOMIXER_API TWeakPtr<FQuartzTickableObjectsManager> GetTickableObjectManager() const;

private:
	// deletes proxies to clocks that no longer exists
	AUDIOMIXER_API void PruneStaleProxies();
	static AUDIOMIXER_API void PruneStaleProxiesInternal(TArray<Audio::FQuartzClockProxy>& ContainerToPrune);

	// sharable tickable object manager to allow for non-UObject subscription / un-subscription
	TSharedPtr<FQuartzTickableObjectsManager> TickableObjectManagerPtr { MakeShared<FQuartzTickableObjectsManager>() };

	// Clock manager/proxy-related data that lives on the AudioDevice for persistence.
	TSharedPtr<Audio::FPersistentQuartzSubsystemData> ClockManagerDataPtr { nullptr };

	bool bTickEvenWhenPaused = false;

	// helpers
	AUDIOMIXER_API Audio::FQuartzClockProxy* FindProxyByName(const FName& ClockName);
	AUDIOMIXER_API Audio::FQuartzClockProxy const* FindProxyByName(const FName& ClockName) const;
	AUDIOMIXER_API Audio::FQuartzClockManager* GetClockManager(const UObject* WorldContextObject, bool bUseAudioEngineClockManager = true);

}; // class UQuartzGameSubsystem
