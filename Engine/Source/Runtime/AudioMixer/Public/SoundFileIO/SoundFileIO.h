// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Audio.h"
#include "CoreMinimal.h"

namespace Audio::SoundFileUtils
{
	bool AUDIOMIXER_API InitSoundFileIOManager();
	bool AUDIOMIXER_API ShutdownSoundFileIOManager();
	uint32 AUDIOMIXER_API GetNumSamples(const TArray<uint8>& InAudioData);
	bool AUDIOMIXER_API ConvertAudioToWav(const TArray<uint8>& InAudioData, TArray<uint8>& OutWaveData);
	bool AUDIOMIXER_API CreateCueAndSampleChunks(const TArray<uint8>& InAudioData, TArray<uint8>& OutWaveData, const TArray<FWaveCue>& InWaveCues, const TArray<FWaveSampleLoop>& InSampleLoops);
}
