// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AudioDevice.h"
#include "Sound/SoundWave.h"
#include "DSP/SinOsc.h"
#include "DSP/ParamInterpolator.h"
#include "Misc/ScopeLock.h"



class FAudioDevice;

#define AUDIO_SOURCE_DECODER_DEBUG 0

namespace Audio
{
	class FMixerBuffer;
	class FMixerSourceBuffer;

	struct FDecodingSoundSourceHandle
	{
		FDecodingSoundSourceHandle()
			: Id(INDEX_NONE)
		{}

		int32 Id;
		FName SoundWaveName;
	};

	struct FSourceDecodeInit
	{
		FSourceDecodeInit()
			: SoundWave(nullptr)
			, SeekTime(0.0f)
			, PitchScale(1.0f)
			, VolumeScale(1.0f)
			, bForceSyncDecode(false)
		{}

		TObjectPtr<USoundWave> SoundWave;
		float SeekTime;
		float PitchScale;
		float VolumeScale;
		bool bForceSyncDecode;
		FDecodingSoundSourceHandle Handle;
	};

	class FDecodingSoundSource
	{
	public:
		FDecodingSoundSource(FAudioDevice* AudioDevice, const FSourceDecodeInit& InitData);
		~FDecodingSoundSource();

		// Called before we initialize
		bool PreInit(int32 OutputSampleRate);

		// Queries if we're ready to initialize
		bool IsReadyToInit();

		// Initializes the decoding source
		void Init();

		// Returns if we've been initialized
		bool IsInitialized() const { return bInitialized; }

		// If the sound source finished playing all its source. Will only return true for non-looping sources.
		bool IsFinished() const { return !bInitialized || SourceInfo.bIsLastBuffer; }
		
		// Sets the pitch scale
		void SetPitchScale(float InPitchScale, uint32 NumFrames = 512);

		// Sets the volume scale
		void SetVolumeScale(float InVolumeScale, uint32 NumFrames = 512);

		// Sets the ForceSyncDecode flag. (Decodes for this soundwave will not happen in an async task if true)
		void SetForceSyncDecode(bool bShouldForceSyncDecode);

		// Get audio buffer
		bool GetAudioBuffer(const int32 InNumFrames, const int32 InNumChannels, FAlignedFloatBuffer& OutAudioBuffer);

		// Return the underlying sound wave
		USoundWave* GetSoundWave() { return SoundWave; }
    
		TObjectPtr<USoundWave>& GetSoundWavePtr() { return SoundWave; }

	private:

		void ReadFrame();
		void GetAudioBufferInternal(const int32 InNumFrames, const int32 InNumChannels, FAlignedFloatBuffer& OutAudioBuffer);

		// Handle to the decoding source
		FDecodingSoundSourceHandle Handle;

		FDeviceId AudioDeviceID;

		// The sound wave object with which this sound is generating
		TObjectPtr<USoundWave> SoundWave;

		// Mixer buffer object which is a convenience wrapper around some buffer initialization and management
		FMixerBuffer* MixerBuffer;

		// Critical Section around mixer source buffer access
		FCriticalSection MixerSourceBufferCritSec;

		// Object which handles bulk of decoding operations
		TSharedPtr<FMixerSourceBuffer, ESPMode::ThreadSafe> MixerSourceBuffer;

		// Scratch buffer used for upmixing and downmixing the audio
		FAlignedFloatBuffer ScratchBuffer;

		// Sample rate of the source
		int32 SampleRate;

		// Current seek time
		float SeekTime;

		// If we've initialized	
		FThreadSafeBool bInitialized;

		// force the decoding of this sound to be synchronous
		bool bForceSyncDecode{false};

		// Object used for source generation from decoded buffers
		struct FSourceInfo
		{
			// Number of channels of source file
			int32 NumSourceChannels;

			// Total number of frames of source file
			uint32 TotalNumFrames;

			// Total number of frames played (or read from decoded buffers) so far. Will always be less than TotalNumFrames
			uint32 NumFramesRead;

			// Total number of frames generated (could be larger or smaller than number of frames read)
			uint32 NumFramesGenerated;

			// The current frame alpha (how far we are between current and next frame)
			float CurrentFrameAlpha;

			// The current frame index
			int32 CurrentFrameIndex;

			// Number of frames of current decoded chunk
			int32 CurrentAudioChunkNumFrames;

			// The pitch scale to use to account for sample rate differences of source to output sample rate
			float BasePitchScale;
			float PitchScale;

			// The pitch param object, allows easy pitch interpolation
			FParam PitchParam;

			// The frame count (from frames generated) to reset the pitch param
			uint32 PitchResetFrame;

			// The volume param object, allows easy volume interpolation
			FParam VolumeParam;

			// The frame count (from frames generated) to reset the volume param
			uint32 VolumeResetFrame;

			// Buffer to store current decoded audio frame
			TArray<float> CurrentFrameValues;

			// Buffer to store next decoded audio frame
			TArray<float> NextFrameValues;

			// The current decoded PCM buffer we are reading from
			TSharedPtr<FAlignedFloatBuffer, ESPMode::ThreadSafe> CurrentPCMBuffer;

			// If this sound is done (has decoded all data)
			bool bIsDone;

			// If this sound hasn't yet started rendering audio
			bool bHasStarted;

			// If this is the last decoded buffer
			bool bIsLastBuffer;

			FSourceInfo()
				: NumSourceChannels(0)
				, TotalNumFrames(0)
				, NumFramesRead(0)
				, NumFramesGenerated(0)
				, CurrentFrameAlpha(0.0f)
				, CurrentFrameIndex(0)
				, CurrentAudioChunkNumFrames(0)
				, BasePitchScale(1.0f)
				, PitchScale(1.0f)
				, PitchResetFrame(0)
				, VolumeResetFrame(0)
				, bIsDone(false)
				, bHasStarted(false)
				, bIsLastBuffer(false)
			{}

		};

		FSourceInfo SourceInfo;

#if AUDIO_SOURCE_DECODER_DEBUG
		FSineOsc SineTone[2];
#endif

	};

	typedef TSharedPtr<FDecodingSoundSource> FDecodingSoundSourcePtr;
	
	class FSoundSourceDecoder : public FGCObject
	{
	public:
		AUDIOMIXER_API FSoundSourceDecoder();
		AUDIOMIXER_API virtual ~FSoundSourceDecoder();

		//~ Begin FGCObject
		AUDIOMIXER_API virtual void AddReferencedObjects(FReferenceCollector & Collector) override;
		virtual FString GetReferencerName() const override
		{
			return TEXT("Audio::FSoundSourceDecoder");
		}
		//~ End FGCObject

		// Initialize the source decoder at the given output sample rate
		// Sources will automatically sample rate convert to match this output
		AUDIOMIXER_API void Init(FAudioDevice* InAudioDevice, int32 SampleRate);

		// Creates a new decoding sound source handle
		AUDIOMIXER_API FDecodingSoundSourceHandle CreateSourceHandle(USoundWave* InSoundWave);

		// Called from the audio thread
		AUDIOMIXER_API void Update();

		// Called from the audio render thread
		AUDIOMIXER_API void UpdateRenderThread();

		// Initialize a decoding instance of this sound wave object. Call only from game thread.
		AUDIOMIXER_API bool InitDecodingSource(const FSourceDecodeInit& InitData);

		// Removes the decoding source from the decoder
		AUDIOMIXER_API void RemoveDecodingSource(const FDecodingSoundSourceHandle& Handle);

		// Resets internal state of decoder
		AUDIOMIXER_API void Reset();

		// Sets the source pitch scale
		AUDIOMIXER_API void SetSourcePitchScale(const FDecodingSoundSourceHandle& Handle, float InPitchScale);

		// Sets the source volume scale
		AUDIOMIXER_API void SetSourceVolumeScale(const FDecodingSoundSourceHandle& Handle, float InVolumeScale);

		// Get a decoded buffer for the given decoding sound wave handle. Call only from audio render thread or audio render thread task.
		AUDIOMIXER_API bool GetSourceBuffer(const FDecodingSoundSourceHandle& InHandle, const int32 NumOutFrames, const int32 NumOutChannels, FAlignedFloatBuffer& OutAudioBuffer);

		// Queries if the decoding source is finished
		AUDIOMIXER_API bool IsFinished(const FDecodingSoundSourceHandle& InHandle) const;

		AUDIOMIXER_API bool IsInitialized(const FDecodingSoundSourceHandle& InHandle) const;

	private:
		// Sends a command to the audio render thread from audio thread
		AUDIOMIXER_API void EnqueueDecoderCommand(TFunction<void()> Command);
		AUDIOMIXER_API void PumpDecoderCommandQueue();
		AUDIOMIXER_API bool InitDecodingSourceInternal(const FSourceDecodeInit& InitData);

		int32 AudioThreadId;
		FAudioDevice* AudioDevice;
		int32 SampleRate;
		mutable FCriticalSection DecodingSourcesCritSec;
		TMap<int32, FDecodingSoundSourcePtr> InitializingDecodingSources;
		TMap<int32, FDecodingSoundSourcePtr> DecodingSources;
		TMap<int32, FSourceDecodeInit> PrecachingSources;

		TQueue<TFunction<void()>> CommandQueue;
	};

}
